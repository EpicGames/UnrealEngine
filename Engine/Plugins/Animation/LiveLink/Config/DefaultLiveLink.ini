[CoreRedirects]

;
; 4.23 Redirects
;

; Plugin/Module redirects
+ClassRedirects=(OldName="MovieSceneLiveLinkTrack", NewName="/Script/LiveLinkMovieScene.MovieSceneLiveLinkTrack")
+ClassRedirects=(OldName="MovieSceneLiveLinkTrackRecorder", NewName="/Script/LiveLinkSequencer.MovieSceneLiveLinkTrackRecorder")
+FunctionRedirects=(OldName="/Script/LiveLink.LiveLinkBlueprintLibrary.RequestShutdown", NewName="/Script/LiveLink.LiveLinkBlueprintLibrary.RemoveSource")

; LiveLink MovieScene redirects
 +ClassRedirects=(OldName="MovieSceneLiveLinkSection", NewName="/Script/LiveLinkMovieScene.MovieSceneLiveLinkSection")
 +StructRedirects=(OldName="MovieSceneLiveLinkSectionTemplate", NewName="/Script/LiveLinkMovieScene.MovieSceneLiveLinkSectionTemplate")

; LiveLink Graph
+ClassRedirects=(OldName="K2Node_EvaluateLiveLinkFrame", NewName="/Script/LiveLinkGraphNode.K2Node_EvaluateLiveLinkFrameWithSpecificRole")
+ClassRedirects=(OldName="K2Node_EvaluateLiveLinkFrameAtWorldTime", NewName="/Script/LiveLinkGraphNode.K2Node_EvaluateLiveLinkFrameAtWorldTime")
+ClassRedirects=(OldName="K2Node_EvaluateLiveLinkFrameAtSceneTime", NewName="/Script/LiveLinkGraphNode.K2Node_EvaluateLiveLinkFrameAtSceneTime")
+ClassRedirects=(OldName="AnimGraphNode_LiveLinkPose", NewName="/Script/LiveLinkGraphNode.AnimGraphNode_LiveLinkPose")

;
; 4.24 Redirects
;

; Frame processors
+EnumRedirects=(OldName="ELiveLinkAxis",NewName="/Script/LiveLink.ELiveLinkAxis")
+ClassRedirects=(OldName="LiveLinkAxisSwitchPreProcessor", NewName="/Script/LiveLink.LiveLinkTransformAxisSwitchPreProcessor")
+ClassRedirects=(OldName="LiveLinkBasicFrameInterpolateProcessor", NewName="/Script/LiveLink.LiveLinkBasicFrameInterpolationProcessor")
+ClassRedirects=(OldName="LiveLinkAnimationFrameInterpolateProcessor", NewName="/Script/LiveLink.LiveLinkAnimationFrameInterpolationProcessor")
+ClassRedirects=(OldName="LiveLinkAnimationRoleToTransform", NewName="/Script/LiveLink.LiveLinkAnimationRoleToTransform")

;
; 4.25 Redirects
;

; Axis Switch processors
+PropertyRedirects=(OldName="LiveLinkTransformAxisSwitchPreProcessor.AxisX",NewName="LiveLinkTransformAxisSwitchPreProcessor.OrientationAxisX")
+PropertyRedirects=(OldName="LiveLinkTransformAxisSwitchPreProcessor.AxisY",NewName="LiveLinkTransformAxisSwitchPreProcessor.OrientationAxisY")
+PropertyRedirects=(OldName="LiveLinkTransformAxisSwitchPreProcessor.AxisZ",NewName="LiveLinkTransformAxisSwitchPreProcessor.OrientationAxisZ")