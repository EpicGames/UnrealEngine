{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "2.0",
	"FriendlyName": "Live Link",
	"Description": "LiveLink allows streaming of animated data into Unreal Engine",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Plugins": [
		{
			"Name": "LiveLinkDevice",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": true,
			"Optional": true
		},
		{
			"Name": "ContentBrowserAssetDataSource",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "LiveLink",
			"Type": "Runtime",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkComponents",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"ProgramAllowList": [ "LiveLinkHub" ]
		},
		{
			"Name": "LiveLinkGraphNode",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkMovieScene",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkSequencer",
			"Type": "UncookedOnly",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkMultiUser",
			"Type": "UncookedOnly",
			"LoadingPhase": "PostEngineInit"
		}
	]
}
