// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLink : ModuleRules
	{
		public LiveLink(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"LiveLinkInterface",
				"LiveLinkMessageBusFramework",
			});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"HeadMountedDisplay",
				"InputCore",
				"LiveLinkAnimationCore", //Adding dependency to make sure this runtime module is pulled when packaging if LiveLink plugin is enabled. 
				"Media",
				"Messaging",
				"Projects",
				"SlateCore",
				"Slate",
				"TimeManagement",
				"CinematicCamera" // Used for the broadcast component to populate camera data.
			});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MessageLog",
					"UnrealEd",
				});
			}
		}
	}
}
