// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientNetworkStatisticsModel.h"

#include "Features/IModularFeatures.h"
#include "INetworkMessagingExtension.h"
#include "Misc/ScopeExit.h"

namespace UE::LiveLinkHub::Private
{
	static INetworkMessagingExtension* GetMessagingStatistics()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (IsInGameThread())
		{
			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}
		else
		{
			ModularFeatures.LockModularFeatureList();
			ON_SCOPE_EXIT
			{
				ModularFeatures.UnlockModularFeatureList();
			};
			
			if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
			{
				return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
			}
		}
		

		ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
		return nullptr;
	}
}

TOptional<FMessageTransportStatistics> UE::LiveLinkHub::FClientNetworkStatisticsModel::GetLatestNetworkStatistics(const FMessageAddress& ClientAddress)
{
	if (INetworkMessagingExtension* Statistics = UE::LiveLinkHub::Private::GetMessagingStatistics())
	{
		const FGuid NodeId = Statistics->GetNodeIdFromAddress(ClientAddress);
		return NodeId.IsValid() ? Statistics->GetLatestNetworkStatistics(NodeId) : TOptional<FMessageTransportStatistics>();
	}
	return {};
}

bool UE::LiveLinkHub::FClientNetworkStatisticsModel::IsOnline(const FMessageAddress& ClientAddress)
{
	INetworkMessagingExtension* Statistics = UE::LiveLinkHub::Private::GetMessagingStatistics();
	return Statistics && Statistics->GetNodeIdFromAddress(ClientAddress).IsValid();
}
