// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"

struct FMessageAddress;
struct FMessageTransportStatistics;

/** Synchronizes the network statistics (the statistics are updated async). */
namespace UE::LiveLinkHub::FClientNetworkStatisticsModel
{
	TOptional<FMessageTransportStatistics> GetLatestNetworkStatistics(const FMessageAddress& ClientAddress);
	bool IsOnline(const FMessageAddress& ClientAddress);
}
