// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClockOffsetEstimatorRamp.h"
#include "CoreGlobals.h"


double FClockOffsetEstimatorRamp::GetEstimatedOffset() const
{
	return EstimatedOffset;
}

void FClockOffsetEstimatorRamp::UpdateEstimation(double Atime, double Btime)
{
	const double CurrentOffset = Btime - Atime;
	const double Error = EstimatedOffset - CurrentOffset;

	// If our estimation is too far, increase our counter to satisfy margin of acceptance before clamping the estimation.
	if (FMath::Abs(Error) > MaxAllowedEstimationError)
	{
		++ConsecutiveBigErrorCount;
		if (ConsecutiveBigErrorCount > ConsecutiveTooBigErrorRequired)
		{
			EstimatedOffset = CurrentOffset;
			return;
		}
	}
	else
	{
		ConsecutiveBigErrorCount = 0;
	}

	// Avoid overshoot if estimation is within a correction step from current offset.
	if (FMath::Abs(Error) < CorrectionStep)
	{
		EstimatedOffset = CurrentOffset;
		return;
	}

	// Apply correction step
	if (EstimatedOffset < CurrentOffset)
	{
		EstimatedOffset += CorrectionStep;
	}
	else
	{
		EstimatedOffset -= CorrectionStep;
	}
}

void FClockOffsetEstimatorRamp::SetRequiredConsecutiveBigError(int32 NewCount)
{
	// Only allow positive value.
	if (!ensure(NewCount >= 0))
	{
		return;
	}

	ConsecutiveTooBigErrorRequired = NewCount;
}

void FClockOffsetEstimatorRamp::SetMaxAllowedEstimationError(double NewValue)
{
	// Only allow positive value because will only compare to absolute values.
	if (!ensure(NewValue > 0.0))
	{
		return;
	}

	MaxAllowedEstimationError = NewValue;
}

void FClockOffsetEstimatorRamp::SetCorrectionStep(double NewValue)
{
	// Avoid control system correcting in the wrong direction.
	if (!ensure(NewValue > 0.0))
	{
		return;
	}

	CorrectionStep = NewValue;
}

void FClockOffsetEstimatorRamp::Reset()
{
	EstimatedOffset = 0.0;
	ConsecutiveBigErrorCount = ConsecutiveTooBigErrorRequired;
}

