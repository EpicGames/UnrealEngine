// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterpolationProcessor/LiveLinkAnimationFrameInterpolateProcessor.h"
#include "Roles/LiveLinkAnimationRole.h"

#include "Roles/LiveLinkAnimationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkAnimationFrameInterpolateProcessor)

namespace LiveLinkAnimationBlendingUtil
{
	void BlendItem(const FTransform& A, const FTransform& B, FTransform& Output, float BlendWeight)
	{
		const ScalarRegister ABlendWeight(1.0f - BlendWeight);
		const ScalarRegister BBlendWeight(BlendWeight);

		Output = A * ABlendWeight;
		Output.AccumulateWithShortestRotation(B, BBlendWeight);
		Output.NormalizeRotation();
	}

	void BlendItem(const float& A, const float& B, float& Output, float BlendWeight)
	{
		Output = (A * (1.0f - BlendWeight)) + (B * BlendWeight);
	}

	template<class Type>
	void Blend(const TArray<Type>& A, const TArray<Type>& B, TArray<Type>& Output, float BlendWeight)
	{
		check(A.Num() == B.Num());
		Output.SetNum(A.Num(), EAllowShrinking::No);

		for (int32 BlendIndex = 0; BlendIndex < A.Num(); ++BlendIndex)
		{
			BlendItem(A[BlendIndex], B[BlendIndex], Output[BlendIndex], BlendWeight);
		}
	}

	void CopyFrameDataBlended(const FLiveLinkAnimationFrameData& PreFrame, const FLiveLinkAnimationFrameData& PostFrame, float BlendWeight, FLiveLinkSubjectFrameData& OutFrame)
	{
		check(OutFrame.FrameData.IsValid());

		FLiveLinkAnimationFrameData* BlendedFrame = OutFrame.FrameData.Cast<FLiveLinkAnimationFrameData>();
		check(BlendedFrame);

		Blend(PreFrame.Transforms, PostFrame.Transforms, BlendedFrame->Transforms, BlendWeight);
	}

	template<class TTimeType>
	void Interpolate(TTimeType InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, bool bInInterpolatePropertyValues, FLiveLinkInterpolationInfo& OutInterpolationInfo)
	{
		//Validate inputs
		check(InStaticData.Cast<FLiveLinkSkeletonStaticData>());

		int32 FrameDataIndexA = INDEX_NONE;
		int32 FrameDataIndexB = INDEX_NONE;
		if (ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::FindInterpolateIndex(InTime, InSourceFrames, FrameDataIndexA, FrameDataIndexB, OutInterpolationInfo))
		{
			if (FrameDataIndexA == FrameDataIndexB)
			{
				// Copy over the frame directly
				OutBlendedFrame.FrameData.InitializeWith(InSourceFrames[FrameDataIndexA]);
			}
			else
			{
				//Initialize the output frame for animation. It will be filled during blended values copied
				OutBlendedFrame.FrameData.InitializeWith(FLiveLinkAnimationFrameData::StaticStruct(), nullptr);

				const FLiveLinkFrameDataStruct& FrameDataA = InSourceFrames[FrameDataIndexA];
				const FLiveLinkFrameDataStruct& FrameDataB = InSourceFrames[FrameDataIndexB];

				const double BlendWeight = ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::GetBlendFactor(InTime, FrameDataA, FrameDataB);
				if (FMath::IsNearlyZero(BlendWeight))
				{
					OutBlendedFrame.FrameData.InitializeWith(FrameDataA);
				}
				else if (FMath::IsNearlyEqual(1.0, BlendWeight))
				{
					OutBlendedFrame.FrameData.InitializeWith(FrameDataB);
				}
				else
				{
					const FLiveLinkAnimationFrameData* AnimationFrameDataPtrA = FrameDataA.Cast<FLiveLinkAnimationFrameData>();
					const FLiveLinkAnimationFrameData* AnimationFrameDataPtrB = FrameDataB.Cast<FLiveLinkAnimationFrameData>();
					const FLiveLinkAnimationFrameData* AnimationFrameDataPtrOutput = OutBlendedFrame.FrameData.Cast<FLiveLinkAnimationFrameData>();
					check(AnimationFrameDataPtrA && AnimationFrameDataPtrB && AnimationFrameDataPtrOutput);

					ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::FGenericInterpolateOptions InterpolationOptions;
					InterpolationOptions.bCopyClosestFrame = false; // Do not copy all the Transforms
					InterpolationOptions.bInterpolateInterpProperties = bInInterpolatePropertyValues;
					ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::GenericInterpolate(BlendWeight, InterpolationOptions, FrameDataA, FrameDataB, OutBlendedFrame.FrameData);
					LiveLinkAnimationBlendingUtil::CopyFrameDataBlended(*AnimationFrameDataPtrA, *AnimationFrameDataPtrB, BlendWeight, OutBlendedFrame);
				}
			}
		}
		else
		{
			//If we could not find a sample, tag it as an overflow. i.e Asking for the future
			OutInterpolationInfo.bOverflowDetected = true;
		}
	}
}

/**
 * ULiveLinkFrameAnimationInterpolateProcessor::FLiveLinkFrameAnimationInterpolateProcessorWorker
 */
ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::FLiveLinkAnimationFrameInterpolationProcessorWorker(bool bInInterpolatePropertyValues)
	: ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker(bInInterpolatePropertyValues)
{}

TSubclassOf<ULiveLinkRole> ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::GetRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

void ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::Interpolate(double InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo)
{
	LiveLinkAnimationBlendingUtil::Interpolate(InTime, InStaticData, InSourceFrames, OutBlendedFrame, bInterpolatePropertyValues, OutInterpolationInfo);
}

void ULiveLinkAnimationFrameInterpolationProcessor::FLiveLinkAnimationFrameInterpolationProcessorWorker::Interpolate(const FQualifiedFrameTime& InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo)
{
	LiveLinkAnimationBlendingUtil::Interpolate(InTime, InStaticData, InSourceFrames, OutBlendedFrame, bInterpolatePropertyValues, OutInterpolationInfo);
}


/**
 * ULiveLinkFrameAnimationInterpolateProcessor
 */
TSubclassOf<ULiveLinkRole> ULiveLinkAnimationFrameInterpolationProcessor::GetRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

ULiveLinkFrameInterpolationProcessor::FWorkerSharedPtr ULiveLinkAnimationFrameInterpolationProcessor::FetchWorker()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShared<FLiveLinkAnimationFrameInterpolationProcessorWorker, ESPMode::ThreadSafe>(bInterpolatePropertyValues);
	}

	return Instance;
}

