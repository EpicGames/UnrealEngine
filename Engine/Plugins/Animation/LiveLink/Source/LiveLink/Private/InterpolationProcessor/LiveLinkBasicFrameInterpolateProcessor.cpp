// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterpolationProcessor/LiveLinkBasicFrameInterpolateProcessor.h"

#include "Roles/LiveLinkBasicRole.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkBasicFrameInterpolateProcessor)


/**
 * ULiveLinkFrameInterpolationProcessor
 */
TSubclassOf<ULiveLinkRole> ULiveLinkBasicFrameInterpolationProcessor::GetRole() const
{
	return ULiveLinkBasicRole::StaticClass();
}

ULiveLinkFrameInterpolationProcessor::FWorkerSharedPtr ULiveLinkBasicFrameInterpolationProcessor::FetchWorker()
{
	if (!BaseInstance.IsValid())
	{
		BaseInstance = MakeShared<FLiveLinkBasicFrameInterpolationProcessorWorker, ESPMode::ThreadSafe>(bInterpolatePropertyValues);
	}

	return BaseInstance;
}

ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::FLiveLinkBasicFrameInterpolationProcessorWorker(bool bInInterpolatePropertyValues)
	: bInterpolatePropertyValues(bInInterpolatePropertyValues)
{}

TSubclassOf<ULiveLinkRole> ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::GetRole() const
{
	return ULiveLinkBasicRole::StaticClass();
}

namespace LiveLinkInterpolation
{
	void Interpolate(const UStruct* InStruct, bool bCheckForInterpFlag, float InBlendWeight, const void* InFrameDataA, const void* InFrameDataB, void* OutFrameData);
	void InterpolateProperty(FProperty* Property, float InBlendWeight, const void* InFrameDataA, const void* InFrameDataB, void* OutFrameData);

	template<class T>
	T BlendValue(const T& A, const T& B, float InBlendWeight)
	{
		return FMath::Lerp(A, B, InBlendWeight);
	}

	template<>
	FTransform BlendValue(const FTransform& A, const FTransform& B, float InBlendWeight)
	{
		const ScalarRegister ABlendWeight(1.0f - InBlendWeight);
		const ScalarRegister BBlendWeight(InBlendWeight);

		FTransform Output = A * ABlendWeight;
		Output.AccumulateWithShortestRotation(B, BBlendWeight);
		Output.NormalizeRotation();
		return Output;
	}

	template<class T>
	void Interpolate(const FStructProperty* StructProperty, float InBlendWeight, const void* DataA, const void* DataB, void* DataResult)
	{
		for (int32 ArrayIndex = 0; ArrayIndex < StructProperty->ArrayDim; ++ArrayIndex)
		{
			const T* ValuePtrA = StructProperty->ContainerPtrToValuePtr<T>(DataA, ArrayIndex);
			const T* ValuePtrB = StructProperty->ContainerPtrToValuePtr<T>(DataB, ArrayIndex);
			T* ValuePtrResult = StructProperty->ContainerPtrToValuePtr<T>(DataResult, ArrayIndex);

			T ValueResult = BlendValue(*ValuePtrA, *ValuePtrB, InBlendWeight);
			StructProperty->CopySingleValue(ValuePtrResult, &ValueResult);
		}
	}

	void Interpolate(const UStruct* InStruct, bool bCheckForInterpFlag, float InBlendWeight, const void* InFrameDataA, const void* InFrameDataB, void* OutFrameData)
	{
		for (TFieldIterator<FProperty> Itt(InStruct); Itt; ++Itt)
		{
			FProperty* Property = *Itt;
			if (!bCheckForInterpFlag || Property->HasAnyPropertyFlags(CPF_Interp))
			{
				if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					for (int32 DimIndex = 0; DimIndex < ArrayProperty->ArrayDim; ++DimIndex)
					{
						const void* Data0 = ArrayProperty->ContainerPtrToValuePtr<const void>(InFrameDataA, DimIndex);
						const void* Data1 = ArrayProperty->ContainerPtrToValuePtr<const void>(InFrameDataB, DimIndex);
						void* DataResult = ArrayProperty->ContainerPtrToValuePtr<void>(OutFrameData, DimIndex);

						FScriptArrayHelper ArrayHelperA(ArrayProperty, Data0);
						FScriptArrayHelper ArrayHelperB(ArrayProperty, Data1);
						FScriptArrayHelper ArrayHelperResult(ArrayProperty, DataResult);

						int32 MinValue = FMath::Min(ArrayHelperA.Num(), FMath::Min(ArrayHelperB.Num(), ArrayHelperResult.Num()));
						for (int32 ArrayIndex = 0; ArrayIndex < MinValue; ++ArrayIndex)
						{
							InterpolateProperty(ArrayProperty->Inner, InBlendWeight, ArrayHelperA.GetRawPtr(ArrayIndex), ArrayHelperB.GetRawPtr(ArrayIndex), ArrayHelperResult.GetRawPtr(ArrayIndex));
						}
					}
				}
				else
				{
					InterpolateProperty(Property, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
				}
			}
		}
	}

	void InterpolateProperty(FProperty* Property, float InBlendWeight, const void* InFrameDataA, const void* InFrameDataB, void* OutFrameData)
	{
		if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct->GetFName() == NAME_Vector)
			{
				Interpolate<FVector>(StructProperty, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Vector4)
			{
				Interpolate<FVector4>(StructProperty, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Rotator)
			{
				Interpolate<FRotator>(StructProperty, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Quat)
			{
				Interpolate<FQuat>(StructProperty, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Transform)
			{
				Interpolate<FTransform>(StructProperty, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_LinearColor)
			{
				Interpolate<FLinearColor>(StructProperty, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
			}
			else
			{
				for (int32 ArrayIndex = 0; ArrayIndex < StructProperty->ArrayDim; ++ArrayIndex)
				{
					const void* Data0 = StructProperty->ContainerPtrToValuePtr<const void>(InFrameDataA, ArrayIndex);
					const void* Data1 = StructProperty->ContainerPtrToValuePtr<const void>(InFrameDataB, ArrayIndex);
					void* DataResult = StructProperty->ContainerPtrToValuePtr<void>(OutFrameData, ArrayIndex);
					Interpolate(StructProperty->Struct, false, InBlendWeight, Data0, Data1, DataResult);
				}
			}
		}
		else if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
		{
			if (NumericProperty->IsFloatingPoint())
			{
				for (int32 ArrayIndex = 0; ArrayIndex < NumericProperty->ArrayDim; ++ArrayIndex)
				{
					const void* Data0 = NumericProperty->ContainerPtrToValuePtr<const void>(InFrameDataA, ArrayIndex);
					double Value0 = NumericProperty->GetFloatingPointPropertyValue(Data0);
					const void* Data1 = NumericProperty->ContainerPtrToValuePtr<const void>(InFrameDataB, ArrayIndex);
					double Value1 = NumericProperty->GetFloatingPointPropertyValue(Data1);

					double ValueResult = FMath::Lerp(Value0, Value1, InBlendWeight);

					void* DataResult = NumericProperty->ContainerPtrToValuePtr<void>(OutFrameData, ArrayIndex);
					NumericProperty->SetFloatingPointPropertyValue(DataResult, ValueResult);
				}
			}
			else if (NumericProperty->IsInteger() && !NumericProperty->IsEnum())
			{
				for (int32 ArrayIndex = 0; ArrayIndex < NumericProperty->ArrayDim; ++ArrayIndex)
				{
					const void* Data0 = NumericProperty->ContainerPtrToValuePtr<const void>(InFrameDataA, ArrayIndex);
					int64 Value0 = NumericProperty->GetSignedIntPropertyValue(Data0);
					const void* Data1 = NumericProperty->ContainerPtrToValuePtr<const void>(InFrameDataB, ArrayIndex);
					int64 Value1 = NumericProperty->GetSignedIntPropertyValue(Data1);

					int64 ValueResult = FMath::Lerp(Value0, Value1, InBlendWeight);

					void* DataResult = NumericProperty->ContainerPtrToValuePtr<void>(OutFrameData, ArrayIndex);
					NumericProperty->SetIntPropertyValue(DataResult, ValueResult);
				}
			}
		}
	}

	template<class TTimeType>
	void Interpolate(TTimeType InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, bool bInInterpolatePropertyValues, FLiveLinkInterpolationInfo& OutInterpolationInfo)
	{
		int32 FrameDataIndexA = INDEX_NONE;
		int32 FrameDataIndexB = INDEX_NONE;
		if (ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::FindInterpolateIndex(InTime, InSourceFrames, FrameDataIndexA, FrameDataIndexB, OutInterpolationInfo))
		{
			OutInterpolationInfo.FrameIndexA = FrameDataIndexA;
			OutInterpolationInfo.FrameIndexB = FrameDataIndexB;

			if (FrameDataIndexA == FrameDataIndexB)
			{
				// Copy over the frame directly
				OutBlendedFrame.FrameData.InitializeWith(InSourceFrames[FrameDataIndexA]);
			}
			else
			{
				const FLiveLinkFrameDataStruct& FrameDataA = InSourceFrames[FrameDataIndexA];
				const FLiveLinkFrameDataStruct& FrameDataB = InSourceFrames[FrameDataIndexB];

				const double BlendFactor = ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::GetBlendFactor(InTime, FrameDataA, FrameDataB);
				if (FMath::IsNearlyZero(BlendFactor))
				{
					OutBlendedFrame.FrameData.InitializeWith(FrameDataA);
				}
				else if (FMath::IsNearlyEqual(1.0, BlendFactor))
				{
					OutBlendedFrame.FrameData.InitializeWith(FrameDataB);
				}
				else
				{
					ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::FGenericInterpolateOptions InterpolationOptions;
					InterpolationOptions.bInterpolatePropertyValues = bInInterpolatePropertyValues;

					ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::GenericInterpolate(BlendFactor, InterpolationOptions, FrameDataA, FrameDataB, OutBlendedFrame.FrameData);
				}
			}
		}
		else
		{
			//If we could not find a sample, tag it as an overflow. i.e Asking for the future
			OutInterpolationInfo.bOverflowDetected = true;
		}
	}
}

void ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::Interpolate(double InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo)
{
	LiveLinkInterpolation::Interpolate(InTime, InStaticData, InSourceFrames, OutBlendedFrame, bInterpolatePropertyValues, OutInterpolationInfo);
}

void ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::Interpolate(const FQualifiedFrameTime& InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo)
{
	LiveLinkInterpolation::Interpolate(InTime, InStaticData, InSourceFrames, OutBlendedFrame, bInterpolatePropertyValues, OutInterpolationInfo);
}

void ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::GenericInterpolate(double InBlendWeight, const FGenericInterpolateOptions& Options, const FLiveLinkFrameDataStruct& FrameDataA, const FLiveLinkFrameDataStruct& FrameDataB, FLiveLinkFrameDataStruct& OutBlendedFrameData)
{
	check(FrameDataA.GetStruct() == FrameDataB.GetStruct());

	const FLiveLinkFrameDataStruct& FrameWhenCanNotBlend = (InBlendWeight > 0.5f) ? FrameDataB : FrameDataA;

	// Initialize the data (copy all properties, including those that will be interpolate later).
	if (Options.bCopyClosestFrame)
	{
		OutBlendedFrameData.InitializeWith(FrameDataA.GetStruct(), FrameWhenCanNotBlend.GetBaseData());
	}
	else
	{
		OutBlendedFrameData.InitializeWith(FrameDataA.GetStruct(), nullptr);
		if (Options.bCopyClosestMetaData)
		{
			OutBlendedFrameData.GetBaseData()->MetaData = FrameWhenCanNotBlend.GetBaseData()->MetaData;
		}
	}

	// Interpolate Time data
	OutBlendedFrameData.GetBaseData()->WorldTime = FLiveLinkWorldTime(FMath::Lerp(FrameDataA.GetBaseData()->WorldTime.GetSourceTime(), FrameDataB.GetBaseData()->WorldTime.GetSourceTime(), InBlendWeight), 0.0);
	OutBlendedFrameData.GetBaseData()->MetaData.SceneTime.Time = FFrameTime(FMath::Lerp(FrameDataA.GetBaseData()->MetaData.SceneTime.Time, FrameDataB.GetBaseData()->MetaData.SceneTime.Time, InBlendWeight));
	OutBlendedFrameData.GetBaseData()->ArrivalTime.WorldTime = FMath::Lerp(FrameDataA.GetBaseData()->ArrivalTime.WorldTime, FrameDataB.GetBaseData()->ArrivalTime.WorldTime, InBlendWeight);
	OutBlendedFrameData.GetBaseData()->ArrivalTime.SceneTime.Time = FFrameTime(FMath::Lerp(FrameDataA.GetBaseData()->ArrivalTime.SceneTime.Time, FrameDataB.GetBaseData()->ArrivalTime.SceneTime.Time, InBlendWeight));

	// Interpolate Property Values
	if (Options.bInterpolatePropertyValues)
	{
		const TArray<float>& PropertiesA = FrameDataA.GetBaseData()->PropertyValues;
		const TArray<float>& PropertiesB = FrameDataB.GetBaseData()->PropertyValues;
		TArray<float>& PropertiesResult = OutBlendedFrameData.GetBaseData()->PropertyValues;

		int32 NumOfProperties = FMath::Min(PropertiesA.Num(), PropertiesB.Num());
		PropertiesResult.SetNum(NumOfProperties);

		for (int32 PropertyIndex = 0; PropertyIndex < NumOfProperties; ++PropertyIndex)
		{
			PropertiesResult[PropertyIndex] = FMath::Lerp(PropertiesA[PropertyIndex], PropertiesB[PropertyIndex], InBlendWeight);
		}
	}
	// Copy the Property Values if they were not copied previously
	else if (!Options.bCopyClosestFrame)
	{
		OutBlendedFrameData.GetBaseData()->PropertyValues = FrameWhenCanNotBlend.GetBaseData()->PropertyValues;
	}

	// Interpolate all properties with the tag "interp"
	if (Options.bInterpolateInterpProperties)
	{
		LiveLinkInterpolation::Interpolate(FrameDataA.GetStruct(), true, InBlendWeight, FrameDataA.GetBaseData(), FrameDataB.GetBaseData(), OutBlendedFrameData.GetBaseData());
	}
}

double ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::GetBlendFactor(double InTime, const FLiveLinkFrameDataStruct& FrameDataA, const FLiveLinkFrameDataStruct& FrameDataB)
{
	const double FrameATime = FrameDataA.GetBaseData()->WorldTime.GetSourceTime();
	const double FrameBTime = FrameDataB.GetBaseData()->WorldTime.GetSourceTime();
	
	const double Divider = FrameBTime - FrameATime;
	if (!FMath::IsNearlyZero(Divider))
	{
		return (InTime - FrameATime) / Divider;
	}
	else
	{
		return 1.0;
	}
}

double ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::GetBlendFactor(FQualifiedFrameTime InTime, const FLiveLinkFrameDataStruct& FrameDataA, const FLiveLinkFrameDataStruct& FrameDataB)
{
	const double FrameASeconds = FrameDataA.GetBaseData()->MetaData.SceneTime.AsSeconds();
	const double FrameBSeconds = FrameDataB.GetBaseData()->MetaData.SceneTime.AsSeconds();

	const double Divider = FrameBSeconds - FrameASeconds;
	if (!FMath::IsNearlyZero(Divider))
	{
		return (InTime.AsSeconds() - FrameASeconds) / Divider;
	}
	else
	{
		return 1.0;
	}
}

 bool ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::FindInterpolateIndex(double InTime, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, int32& OutFrameIndexA, int32& OutFrameIndexB, FLiveLinkInterpolationInfo& OutInterpolationInfo)
 {
	if (InSourceFrames.Num() == 0)
	{
		return false;
	}

	OutInterpolationInfo.ExpectedEvaluationDistanceFromNewestSeconds = InSourceFrames.Last().GetBaseData()->WorldTime.GetSourceTime() - InTime;
	OutInterpolationInfo.ExpectedEvaluationDistanceFromOldestSeconds = InTime - InSourceFrames[0].GetBaseData()->WorldTime.GetSourceTime();

	for (int32 FrameIndex = InSourceFrames.Num() - 1; FrameIndex >= 0; --FrameIndex)
	{
		const FLiveLinkFrameDataStruct& SourceFrameData = InSourceFrames[FrameIndex];
		if (SourceFrameData.GetBaseData()->WorldTime.GetSourceTime() <= InTime)
		{
			if (FrameIndex == InSourceFrames.Num() - 1)
			{
				OutFrameIndexA = FrameIndex;
				OutFrameIndexB = FrameIndex;
				OutInterpolationInfo.bOverflowDetected = !FMath::IsNearlyEqual(InTime, SourceFrameData.GetBaseData()->WorldTime.GetSourceTime());
				return true;
			}
			else
			{
				OutFrameIndexA = FrameIndex;
				OutFrameIndexB = FrameIndex + 1;
				return true;
			}
		}
	}

	OutFrameIndexA = 0;
	OutFrameIndexB = 0;
	OutInterpolationInfo.bUnderflowDetected = true;
	return true;
}

bool ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::FindInterpolateIndex(FQualifiedFrameTime InTime, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, int32& OutFrameIndexA, int32& OutFrameIndexB, FLiveLinkInterpolationInfo& OutInterpolationInfo)
{
	if (InSourceFrames.Num() == 0)
	{
		return false;
	}
	
	OutInterpolationInfo.ExpectedEvaluationDistanceFromNewestSeconds = InSourceFrames.Last().GetBaseData()->MetaData.SceneTime.AsSeconds() - InTime.AsSeconds();
	OutInterpolationInfo.ExpectedEvaluationDistanceFromOldestSeconds = InTime.AsSeconds() - InSourceFrames[0].GetBaseData()->MetaData.SceneTime.AsSeconds();

	const double InTimeInSeconds = InTime.AsSeconds();
	for (int32 FrameIndex = InSourceFrames.Num() - 1; FrameIndex >= 0; --FrameIndex)
	{
		const FLiveLinkFrameDataStruct& SourceFrameData = InSourceFrames[FrameIndex];
		if (SourceFrameData.GetBaseData()->MetaData.SceneTime.AsSeconds() <= InTimeInSeconds)
		{
			if (FrameIndex == InSourceFrames.Num() - 1)
			{
				OutFrameIndexA = FrameIndex;
				OutFrameIndexB = FrameIndex;

				OutInterpolationInfo.bOverflowDetected = !FMath::IsNearlyEqual(InTimeInSeconds, SourceFrameData.GetBaseData()->MetaData.SceneTime.AsSeconds());
				return true;
			}
			else
			{
				OutFrameIndexA = FrameIndex;
				OutFrameIndexB = FrameIndex + 1;
				return true;
			}
		}
	}

	OutFrameIndexA = 0;
	OutFrameIndexB = 0;
	OutInterpolationInfo.bUnderflowDetected = true;
	return true;
}

