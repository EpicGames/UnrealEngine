// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkAnimationVirtualSubject.h"

#include "Algo/TopologicalSort.h"
#include "ILiveLinkClient.h"
#include "Misc/App.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkAnimationVirtualSubject)


namespace LiveLinkAnimationVirtualSubjectUtils
{
	void AddToBoneNames(TArray<FName>& BoneNames, const TArray<FName>& NewBoneNames, const FName Prefix, TMap<int32, FName>& NamesToOverride)
	{
		FString NameFormat;
		if (Prefix != NAME_None)
		{
			NameFormat = Prefix.ToString() + TEXT("_");
		}

		BoneNames.Reserve(BoneNames.Num() + NewBoneNames.Num());

		for (const FName& NewBoneName : NewBoneNames)
		{
			int32 Index = BoneNames.IndexOfByKey(NewBoneName);
			if (Index != INDEX_NONE)
			{
				FString OverridenName = TEXT("__REPLACED_BONE_") + NewBoneName.ToString();
				NamesToOverride.Add(Index, *OverridenName);
			}

			BoneNames.Add(*(NameFormat + NewBoneName.ToString()));
		}
	}

	void AddToBoneParents(TArray<int32>& BoneParents, const TArray<int32>& NewBoneParents)
	{
		const int32 Offset = BoneParents.Num();

		BoneParents.Reserve(BoneParents.Num() + NewBoneParents.Num());

		for (int32 BoneParent : NewBoneParents)
		{
			// Here we are combining multiple bone hierarchies under one root bone
			// Each hierarchy is complete self contained so we have a simple calculation to perform
			// 1) Bones with out a parent get parented to root (-1 => 0 )
			// 2) Bones with a parent need and offset based on the current size of the buffer
			if (BoneParent == INDEX_NONE)
			{
				if (BoneParents.Num())
				{
					BoneParents.Add(0);
				}
				else
				{
					BoneParents.Add(INDEX_NONE);
				}
			}
			else
			{
				BoneParents.Add(BoneParent + Offset);
			}
		}
	}
}


ULiveLinkAnimationVirtualSubject::ULiveLinkAnimationVirtualSubject()
{
	Role = ULiveLinkAnimationRole::StaticClass();
	bInvalidate = true;
}

void ULiveLinkAnimationVirtualSubject::Update()
{
	if (!IsPaused())
	{
		// Invalid the snapshot
		InvalidateFrameData();
	}

	UpdateTranslatorsForThisFrame();

	TArray<FLiveLinkSubjectKey> ActiveSubjects = LiveLinkClient->GetSubjects(false, false);

	TArray<FLiveLinkSubjectFrameData> SubjectSnapshot;

	if (bSubjectsNeedSorting)
	{
		SortSubjects();
	}

	if (!IsPaused())
	{
		bLastSnapshotIsComplete = BuildSubjectSnapshot(SubjectSnapshot);

		// BuildSkeleton/BuildFrame assert that every entry's StaticData/FrameData is valid. Skip the rebuild when
		// BuildSubjectSnapshot bailed out because a source had no data and no cached fallback was available.
		if (SubjectSnapshot.Num() == Subjects.Num())
		{
			BuildSkeleton(SubjectSnapshot);
			BuildFrame(SubjectSnapshot);

			LastFrameSnapshots = MoveTemp(SubjectSnapshot);
		}
	}

	if (!AreSubjectsValid(ActiveSubjects))
	{
		for (const FLiveLinkVirtualSubjectBoneAttachment& Attachment : Attachments)
		{
			// Update error messages on the attachments.
			Attachment.IsValid(Subjects);
		}
	}
}

bool ULiveLinkAnimationVirtualSubject::AreSubjectsValid(const TArray<FLiveLinkSubjectKey>& InActiveSubjects) const
{
	if (Subjects.Num() <= 0)
	{
		return false;
	}

	bool bValid = true;

	for (const FName& SubjectName : Subjects)
	{
		const FLiveLinkSubjectKey* FoundPtr = InActiveSubjects.FindByPredicate(
			[SubjectName](const FLiveLinkSubjectKey& SubjectData)
			{
				return (SubjectData.SubjectName == SubjectName);
			});


		bValid = false;

		if (FoundPtr)
		{
			TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient->GetSubjectRole_AnyThread(*FoundPtr);
			bValid = SubjectRole && (SubjectRole->IsChildOf(ULiveLinkAnimationRole::StaticClass()) || SubjectRole->IsChildOf(ULiveLinkBasicRole::StaticClass()));
		}

		if (!bValid)
		{
			break;
		}
	}

	return bValid;
}

bool ULiveLinkAnimationVirtualSubject::BuildSubjectSnapshot(TArray<FLiveLinkSubjectFrameData>& OutSnapshot)
{
	OutSnapshot.Reset(Subjects.Num());

	bool bAllSnapshotsSucceeded = true;

	uint32 SubjectIndex = 0;
	for (const FName& SubjectName : Subjects)
	{
		FLiveLinkSubjectFrameData& NextSnapshot = OutSnapshot.AddDefaulted_GetRef();
		TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient->GetSubjectRole_AnyThread(SubjectName);
		
		// Evaluate for basic role if that's the subject's role.
		TSubclassOf<ULiveLinkRole> DesiredRole = SubjectRole && SubjectRole->IsChildOf(ULiveLinkBasicRole::StaticClass()) ? SubjectRole : GetRole();
		if (!LiveLinkClient->EvaluateFrame_AnyThread(SubjectName, DesiredRole, NextSnapshot))
		{
			bAllSnapshotsSucceeded = false;
			if (LastFrameSnapshots.IsValidIndex(SubjectIndex))
			{
				// Snapshots are non-copyable, so this should be fine since the resulting snapshot array will be used
				// to build the next LastFrameSnapshots list.
				NextSnapshot = MoveTemp(LastFrameSnapshots[SubjectIndex]);
			}
			else
			{
				// First evaluation after session load: no cached snapshot exists yet. Bail out — leaving NextSnapshot
				// default-initialised would trip the StaticData/FrameData IsValid checks in BuildSkeleton/BuildFrame.
				OutSnapshot.Reset();
				return false;
			}
		}

		SubjectIndex++;
	}

	return bAllSnapshotsSucceeded;
}

void ULiveLinkAnimationVirtualSubject::BuildSkeleton(const TArray<FLiveLinkSubjectFrameData>& InSubjectSnapshots)
{
	if (DoesSkeletonNeedRebuilding())
	{
		ChildBonesInfo.Reset();
		BoneNameToIndex.Reset();

		FLiveLinkStaticDataStruct StaticData(FLiveLinkSkeletonStaticData::StaticStruct());
		FLiveLinkSkeletonStaticData* SkeletonData = StaticData.Cast<FLiveLinkSkeletonStaticData>();

		check(InSubjectSnapshots.Num() == Subjects.Num());

		TArray<FName> BoneNames{ };
		TArray<int32> BoneParents{ };

		TMap<int32, FName> NamesToOverride;

		TArray<FName> CombinedPropertyNames;

		for (int32 Index = 0; Index < InSubjectSnapshots.Num(); ++Index)
		{
			const FLiveLinkSubjectFrameData& SubjectSnapShotData = InSubjectSnapshots[Index];
			check(SubjectSnapShotData.StaticData.IsValid());

			if (const FLiveLinkSkeletonStaticData* SubjectSkeletonData = SubjectSnapShotData.StaticData.Cast<FLiveLinkSkeletonStaticData>())
			{
				const FName BonePrefix = bAppendSubjectNameToBones ? Subjects[Index] : NAME_None;

				LiveLinkAnimationVirtualSubjectUtils::AddToBoneNames(BoneNames, SubjectSkeletonData->GetBoneNames(), BonePrefix, NamesToOverride);
				LiveLinkAnimationVirtualSubjectUtils::AddToBoneParents(BoneParents, SubjectSkeletonData->GetBoneParents());

				// Cache bone names to bone index, we need to use both the subject and bone names to make sure
				// that we can find a parent bone even if it's in a name conflict.
				for (int32 BoneIndex = 0; BoneIndex < BoneNames.Num(); BoneIndex++)
				{
					BoneNameToIndex.FindOrAdd({ Subjects[Index], BoneNames[BoneIndex] }) = BoneIndex;
				}
			}

			CombinedPropertyNames.Append(SubjectSnapShotData.StaticData.GetBaseData()->PropertyNames);
		}

		ProcessAttachmentsForStaticData(BoneParents);

		for (const TPair<int32, FName>& Names : NamesToOverride)
		{
			BoneNames[Names.Key] = Names.Value;
		}

		SkeletonData->SetBoneNames(BoneNames);
		SkeletonData->SetBoneParents(BoneParents);
		SkeletonData->PropertyNames = MoveTemp(CombinedPropertyNames);

		UpdateStaticDataSnapshot(MoveTemp(StaticData));

		bInvalidate = false;

		PostSkeletonRebuild();
	}
	else
	{
		TArray<int32> BoneParents;
		ProcessAttachmentsForStaticData(BoneParents);
	}
}

void ULiveLinkAnimationVirtualSubject::BuildFrame(const TArray<FLiveLinkSubjectFrameData>& InSubjectSnapshots)
{
	const FLiveLinkSkeletonStaticData* SnapshotSkeletonData = GetFrameSnapshot().StaticData.Cast<FLiveLinkSkeletonStaticData>();
	FLiveLinkFrameDataStruct NewFrameData(FLiveLinkAnimationFrameData::StaticStruct());
	FLiveLinkAnimationFrameData* NewSnapshotFrameData = NewFrameData.Cast<FLiveLinkAnimationFrameData>();

	NewSnapshotFrameData->Transforms.Reset(SnapshotSkeletonData->GetBoneNames().Num());
	NewSnapshotFrameData->MetaData.StringMetaData.Empty();
	NewSnapshotFrameData->PropertyValues.Empty();

	//Go over each subject snapshot and take transforms and curves
	check(InSubjectSnapshots.Num() == Subjects.Num());
	for (int32 Index = 0; Index < InSubjectSnapshots.Num(); ++Index)
	{
		const FLiveLinkSubjectFrameData& SubjectSnapShotData = InSubjectSnapshots[Index];

		check(SubjectSnapShotData.FrameData.IsValid());
		if (const FLiveLinkAnimationFrameData* SubjectAnimationData = SubjectSnapShotData.FrameData.Cast<FLiveLinkAnimationFrameData>())
		{
			NewSnapshotFrameData->Transforms.Append(SubjectAnimationData->Transforms);
		}

		NewSnapshotFrameData->PropertyValues.Append(SubjectSnapShotData.FrameData.GetBaseData()->PropertyValues);
		for (const auto& MetaDatum : SubjectSnapShotData.FrameData.GetBaseData()->MetaData.StringMetaData)
		{
			NewSnapshotFrameData->MetaData.StringMetaData.Emplace(Subjects[Index], MetaDatum.Value);
		}

		if (SyncSubject == Subjects[Index])
		{
			// Assign timecode from the sync subject.
			NewSnapshotFrameData->MetaData.SceneTime = SubjectSnapShotData.FrameData.GetBaseData()->MetaData.SceneTime;
		}
	}

	if (NewSnapshotFrameData->MetaData.SceneTime.Time == 0)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FAppTime AppTime = FAppTime::Get_UnsafeNoCheck();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (TOptional<FQualifiedFrameTime> FrameTime = AppTime.CurrentFrameTime)
		{
			NewSnapshotFrameData->MetaData.SceneTime = *FrameTime;
		}
	}

	ProcessAttachmentsForFrameData(NewSnapshotFrameData);

	UpdateFrameDataSnapshot(MoveTemp(NewFrameData));
}


void ULiveLinkAnimationVirtualSubject::SortSubjects()
{
	if (Attachments.Num() == 0)
	{
		return;
	}

	bool bOneValidAttachment = false;
	for (const FLiveLinkVirtualSubjectBoneAttachment& Attachment : Attachments)
	{
		if (Attachment.IsValid(Subjects))
		{
			bOneValidAttachment = true;
			break;
		}
	}

	if (!bOneValidAttachment)
	{
		return;
	}

	TMap<FLiveLinkSubjectName, TArray<FLiveLinkSubjectName>> ParentToChildren;
	for (const FLiveLinkVirtualSubjectBoneAttachment& Attachment : Attachments)
	{
		if (Attachment.ChildSubject != FName(NAME_None))
		{
			ParentToChildren.FindOrAdd(Attachment.ParentSubject).Add(Attachment.ChildSubject);
		}
	}

	auto FindDependencies = [&ParentToChildren](FLiveLinkSubjectName SubjectName) -> TArray<FLiveLinkSubjectName>
	{
		if (TArray<FLiveLinkSubjectName>* Children = ParentToChildren.Find(SubjectName))
		{
			return *Children;
		}

		return {};
	};

	if (!Algo::TopologicalSort(Subjects, FindDependencies))
	{
		UE_LOGF(LogTemp, Warning, "Circular dependency present in attachments.");
	}

	Algo::Reverse(Subjects);

	bSubjectsNeedSorting = false;
}

void ULiveLinkAnimationVirtualSubject::ProcessAttachmentsForStaticData(TArray<int32>& InOutBoneParents)
{
	// Compute the bone parents from the attachments and modify the bone array.
	int32 GlobalParentIndex = INDEX_NONE; // The global index of the parent bone.
	int32 GlobalChildIndex = INDEX_NONE; // The global index of the bone that will be attached to the parent bone.
	TSet<FLiveLinkSubjectName> SubjectsRequiredByAttachments;
	for (const FLiveLinkVirtualSubjectBoneAttachment& Attachment : Attachments)
	{
		if (Attachment.IsValid(Subjects))
		{
			SubjectsRequiredByAttachments.Add(Attachment.ChildSubject);
			SubjectsRequiredByAttachments.Add(Attachment.ParentSubject);

			// 1. Find global bone index for parent and child bone
			if (int32* ParentIndex = BoneNameToIndex.Find({ Attachment.ParentSubject, Attachment.ParentBone }))
			{
				GlobalParentIndex = *ParentIndex;
			}

			if (int32* ChildIndex = BoneNameToIndex.Find({ Attachment.ChildSubject, Attachment.ChildBone }))
			{
				GlobalChildIndex = *ChildIndex;
			}

			if (GlobalParentIndex == INDEX_NONE || GlobalChildIndex == INDEX_NONE)
			{
				// Skip this attachment if we couldn't find either the parent or child bone.
				continue;
			}

			// 2. Override the bone parents according to the attachments
			if (InOutBoneParents.IsValidIndex(GlobalChildIndex))
			{
				InOutBoneParents[GlobalChildIndex] = GlobalParentIndex;
			}

			// 3. Store the info for the attachment child.
			FTransform Offset = FTransform::Identity;
			Offset.SetTranslation(Attachment.LocationOffset);
			Offset.SetRotation(Attachment.RotationOffset.Quaternion());

			FChildBoneInfo ChildBoneInfo;
			ChildBoneInfo.Offset = Offset;
			ChildBoneInfo.ParentBone = GlobalParentIndex;

			ChildBonesInfo.Add(GlobalChildIndex, MoveTemp(ChildBoneInfo));
		}
	}
}

void ULiveLinkAnimationVirtualSubject::ProcessAttachmentsForFrameData(FLiveLinkAnimationFrameData* SnapshotFrameData)
{
	// Apply transformations specified by the attachments.
	for (const TPair<int32, FChildBoneInfo>& ChildInfo : ChildBonesInfo)
	{
		// Apply offsets specified by the attachments.
		FTransform Offset = ChildInfo.Value.Offset;

		const FChildBoneInfo& ChildBoneInfo = ChildInfo.Value;

		FTransform& ParentBoneTransform = SnapshotFrameData->Transforms[ChildInfo.Value.ParentBone];
		FTransform& ChildBoneTransform = SnapshotFrameData->Transforms[ChildInfo.Key];

		FTransform FinalBoneTransform;
		switch(LocationBehavior)
		{
			using enum EBoneTransformResolution;
			case KeepParent:
				FinalBoneTransform.SetLocation(ParentBoneTransform.GetLocation());
				break;
			case KeepChild:
			{
				FinalBoneTransform.SetLocation(ChildBoneTransform.GetLocation());
				break;
			}
			case Combine:
			{
				FinalBoneTransform.SetLocation(ChildBoneTransform.TransformPosition(ParentBoneTransform.GetLocation()));
				break;
			}
		}

		switch(RotationBehavior)
		{
			using enum EBoneTransformResolution;
			case KeepParent:
				FinalBoneTransform.SetRotation(ParentBoneTransform.GetRotation());
				break;
			case KeepChild:
				FinalBoneTransform.SetRotation(ChildBoneTransform.GetRotation());
				break;
			case Combine:
				FinalBoneTransform.SetRotation(ChildBoneTransform.TransformRotation(ParentBoneTransform.GetRotation()));
				break;
		}

		ChildBoneTransform = FinalBoneTransform * Offset;
	}
}

bool ULiveLinkAnimationVirtualSubject::DoesSkeletonNeedRebuilding() const
{
	return !HasValidStaticData() || bInvalidate;
}

#if WITH_EDITOR
void ULiveLinkAnimationVirtualSubject::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// When modifying an attachment, we only want to invalidate the static data if we modify a parent/child bone or subject.
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkAnimationVirtualSubject, Attachments))
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ParentBone)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ChildBone)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ParentSubject)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ChildSubject))
		{
			bSubjectsNeedSorting = true;
			bInvalidate = true;
			InvalidateStaticData();
		}
	}
	else
	{
		bInvalidate = true;
		InvalidateStaticData();
	}
}
#endif //WITH_EDITOR
