// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkBlueprintLibrary.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkSource.h"
#include "Misc/App.h"
#include "LiveLinkRole.h"
#include "Roles/LiveLinkAnimationBlueprintStructs.h"
#include "Roles/LiveLinkBasicTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkBlueprintLibrary)

bool ULiveLinkBlueprintLibrary::GetPropertyValue(UPARAM(ref) FLiveLinkBasicBlueprintData& BasicData, FName PropertyName, float& Value)
{
	return BasicData.StaticData.FindPropertyValue(BasicData.FrameData, PropertyName, Value);
};

void ULiveLinkBlueprintLibrary::GetCurves(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, TMap<FName, float>& Curves)
{
	SubjectFrameHandle.GetCurves(Curves);
};

int ULiveLinkBlueprintLibrary::NumberOfTransforms(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle)
{
	return SubjectFrameHandle.GetNumberOfTransforms();
};

void ULiveLinkBlueprintLibrary::TransformNames(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, TArray<FName>& TransformNames)
{
	return SubjectFrameHandle.GetTransformNames(TransformNames);
}

void ULiveLinkBlueprintLibrary::GetRootTransform(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkTransform& LiveLinkTransform)
{
	SubjectFrameHandle.GetRootTransform(LiveLinkTransform);
};

void ULiveLinkBlueprintLibrary::GetTransformByIndex(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, int TransformIndex, FLiveLinkTransform& LiveLinkTransform)
{
	SubjectFrameHandle.GetTransformByIndex(TransformIndex, LiveLinkTransform);
};

void ULiveLinkBlueprintLibrary::GetTransformByName(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FName TransformName, FLiveLinkTransform& LiveLinkTransform)
{
	SubjectFrameHandle.GetTransformByName(TransformName, LiveLinkTransform);
};

void ULiveLinkBlueprintLibrary::GetMetadata(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FSubjectMetadata& Metadata)
{
	SubjectFrameHandle.GetSubjectMetadata(Metadata);
};

void ULiveLinkBlueprintLibrary::GetBasicData(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkBasicBlueprintData& BasicBlueprintData)
{
	if (const FLiveLinkSkeletonStaticData* StaticData = SubjectFrameHandle.GetSourceSkeletonStaticData())
	{
		BasicBlueprintData.StaticData = *StaticData;
	}
	if (const FLiveLinkAnimationFrameData* FrameData = SubjectFrameHandle.GetSourceAnimationFrameData())
	{
		BasicBlueprintData.FrameData = *FrameData;
	}
};

bool ULiveLinkBlueprintLibrary::GetAnimationStaticData(FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkSkeletonStaticData& AnimationStaticData)
{
	if (const FLiveLinkSkeletonStaticData* StaticData = SubjectFrameHandle.GetSourceSkeletonStaticData())
	{
		AnimationStaticData = *StaticData;
		return true;
	}

	return false;
}

bool ULiveLinkBlueprintLibrary::GetAnimationFrameData(FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkAnimationFrameData& AnimationFrameData)
{
	if (const FLiveLinkAnimationFrameData* FrameData = SubjectFrameHandle.GetSourceAnimationFrameData())
	{
		AnimationFrameData = *FrameData;
		return true;
	}

	return false;
}

void ULiveLinkBlueprintLibrary::TransformName(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FName& Name)
{
	return LiveLinkTransform.GetName(Name);
};

void ULiveLinkBlueprintLibrary::ParentBoneSpaceTransform(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FTransform& Transform)
{
	LiveLinkTransform.GetTransformParentSpace(Transform);
};

void ULiveLinkBlueprintLibrary::ComponentSpaceTransform(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FTransform& Transform)
{
	LiveLinkTransform.GetTransformRootSpace(Transform);
};

bool ULiveLinkBlueprintLibrary::HasParent(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform)
{
	return LiveLinkTransform.HasParent();
};

void ULiveLinkBlueprintLibrary::GetParent(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FLiveLinkTransform& Parent)
{
	LiveLinkTransform.GetParent(Parent);
};

int ULiveLinkBlueprintLibrary::ChildCount(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform)
{
	return LiveLinkTransform.GetChildCount();
};

void ULiveLinkBlueprintLibrary::GetChildren(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, TArray<FLiveLinkTransform>& Children)
{
	LiveLinkTransform.GetChildren(Children);
};

bool ULiveLinkBlueprintLibrary::IsSourceStillValid(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle)
{
	return SourceHandle.SourcePointer.IsValid() && SourceHandle.SourcePointer->IsSourceStillValid();
};

bool ULiveLinkBlueprintLibrary::RemoveSource(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle)
{
	if (SourceHandle.SourcePointer.IsValid())
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
			LiveLinkClient->RemoveSource(SourceHandle.SourcePointer);
			return true;
		}
	}

	return false;
}

FText ULiveLinkBlueprintLibrary::GetSourceStatus(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle)
{
	if (SourceHandle.SourcePointer.IsValid())
	{
		return SourceHandle.SourcePointer->GetSourceStatus();
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText ULiveLinkBlueprintLibrary::GetSourceType(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle)
{
	if (SourceHandle.SourcePointer.IsValid())
	{
		return SourceHandle.SourcePointer->GetSourceType();
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText ULiveLinkBlueprintLibrary::GetSourceMachineName(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle)
{
	if (SourceHandle.SourcePointer.IsValid())
	{
		return SourceHandle.SourcePointer->GetSourceMachineName();
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText ULiveLinkBlueprintLibrary::GetSourceTypeFromGuid(FGuid SourceGuid)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		const ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		return LiveLinkClient.GetSourceType(SourceGuid);
	}
	return FText::GetEmpty();
}

TArray<FLiveLinkSubjectName> ULiveLinkBlueprintLibrary::GetLiveLinkEnabledSubjectNames(bool bIncludeVirtualSubject)
{
	TArray<FLiveLinkSubjectName> Result;
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		TArray<FLiveLinkSubjectKey> SubjectKeys = LiveLinkClient.GetSubjects(false, bIncludeVirtualSubject);
		Result.Reset(SubjectKeys.Num());
		for (const FLiveLinkSubjectKey& SubjectKey : SubjectKeys)
		{
			Result.Add(SubjectKey.SubjectName);
		}
	}
	return Result;
}

TArray<FLiveLinkSubjectKey> ULiveLinkBlueprintLibrary::GetLiveLinkSubjects(bool bIncludeDisabledSubject, bool bIncludeVirtualSubject)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		return LiveLinkClient.GetSubjects(bIncludeDisabledSubject, bIncludeVirtualSubject);
	}
	return TArray<FLiveLinkSubjectKey>();
}

bool ULiveLinkBlueprintLibrary::IsSpecificLiveLinkSubjectEnabled(const FLiveLinkSubjectKey SubjectKey, bool bForThisFrame)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		return LiveLinkClient.IsSubjectEnabled(SubjectKey, bForThisFrame);
	}
	return false;
}

bool ULiveLinkBlueprintLibrary::IsLiveLinkSubjectEnabled(const FLiveLinkSubjectName SubjectName)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		return LiveLinkClient.IsSubjectEnabled(SubjectName);
	}
	return false;
}

ELiveLinkSubjectState ULiveLinkBlueprintLibrary::GetLiveLinkSubjectState(const FLiveLinkSubjectName SubjectName)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (!ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		return ELiveLinkSubjectState::Unknown;
	}

	ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	
	return LiveLinkClient.GetSubjectState(SubjectName);
}

void ULiveLinkBlueprintLibrary::SetLiveLinkSubjectEnabled(const FLiveLinkSubjectKey SubjectKey, bool bEnabled)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		LiveLinkClient.SetSubjectEnabled(SubjectKey, bEnabled);
	}
}

TSubclassOf<ULiveLinkRole> ULiveLinkBlueprintLibrary::GetSpecificLiveLinkSubjectRole(const FLiveLinkSubjectKey SubjectKey)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		return LiveLinkClient.GetSubjectRole_AnyThread(SubjectKey);
	}
	return TSubclassOf<ULiveLinkRole>();
}

TSubclassOf<ULiveLinkRole> ULiveLinkBlueprintLibrary::GetLiveLinkSubjectRole(const FLiveLinkSubjectName SubjectName)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		return LiveLinkClient.GetSubjectRole_AnyThread(SubjectName);
	}
	return TSubclassOf<ULiveLinkRole>();
}

void ULiveLinkBlueprintLibrary::PauseSubject(const FLiveLinkSubjectName SubjectName)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		LiveLinkClient.PauseSubject_AnyThread(SubjectName);
	}
}

void ULiveLinkBlueprintLibrary::UnpauseSubject(const FLiveLinkSubjectName SubjectName)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		LiveLinkClient.UnpauseSubject_AnyThread(SubjectName);
	}
}

bool ULiveLinkBlueprintLibrary::EvaluateLiveLinkFrame(FLiveLinkSubjectRepresentation InSubjectRepresentation, FLiveLinkBaseBlueprintData& OutBlueprintData)
{
	// We should never hit this!  stubs to avoid NoExport on the class.
	check(0);
	return false;
}

bool ULiveLinkBlueprintLibrary::EvaluateLiveLinkFrameWithSpecificRole(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkBaseBlueprintData& OutBlueprintData)
{
	// We should never hit this!  stubs to avoid NoExport on the class.
	check(0);
	return false;
}

bool ULiveLinkBlueprintLibrary::EvaluateLiveLinkFrameAtWorldTimeOffset(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, float WorldTimeOffset, FLiveLinkBaseBlueprintData& OutBlueprintData)
{
	// We should never hit this!  stubs to avoid NoExport on the class.
	check(0);
	return false;
}

bool ULiveLinkBlueprintLibrary::EvaluateLiveLinkFrameAtSceneTime(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, FTimecode SceneTime, FLiveLinkBaseBlueprintData& OutBlueprintData)
{
	// We should never hit this!  stubs to avoid NoExport on the class.
	check(0);
	return false;
}

namespace EvaluateLiveLinkFrame
{
	enum class EEvaluateType
	{
		Snapshot,
		WorldTime,
		SceneTime,
	};

	bool Generic_Evaluate(const ULiveLinkBlueprintLibrary* InSelf, struct FFrame& InStack, FStructProperty* InBlueprintDataStructProp
		, FLiveLinkSubjectRepresentation InSubjectRepresentation, EEvaluateType InEvaluationType, double InWorldTime, const FQualifiedFrameTime& InQualifiedFrameTime, void* OutBlueprintDataPtr)
	{
		bool bSuccess = false;

		if (!InSubjectRepresentation.Role || InSubjectRepresentation.Subject.IsNone())
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AccessViolation,
				NSLOCTEXT("EvaluateLiveLinkFrame", "MissingRoleInput", "Failed to resolve the subject. Be sure the subject name and role are valid.")
			);
			FBlueprintCoreDelegates::ThrowScriptException(InSelf, InStack, ExceptionInfo);
		}
		else if (InBlueprintDataStructProp && OutBlueprintDataPtr)
		{
			if (ULiveLinkRole* LiveLinkRole = Cast<ULiveLinkRole>(InSubjectRepresentation.Role->GetDefaultObject()))
			{
				UScriptStruct* BlueprintDataType = InBlueprintDataStructProp->Struct;
				const UScriptStruct* RoleBlueprintDataType = LiveLinkRole->GetBlueprintDataStruct();

				const bool bBlueprintDataCompatible = (BlueprintDataType == RoleBlueprintDataType) ||
					(BlueprintDataType->IsChildOf(RoleBlueprintDataType) && FStructUtils::TheSameLayout(BlueprintDataType, RoleBlueprintDataType));
				if (bBlueprintDataCompatible)
				{
					P_NATIVE_BEGIN;
					//Create the struct holder and make it point to the output data
					FLiveLinkBlueprintDataStruct BlueprintDataWrapper(BlueprintDataType, StaticCast<FLiveLinkBaseBlueprintData*>(OutBlueprintDataPtr));
					IModularFeatures& ModularFeatures = IModularFeatures::Get();
					if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
					{
						ILiveLinkClient& LiveLinkClient = ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
						FLiveLinkSubjectFrameData FrameData;
						switch(InEvaluationType)
						{
						case EEvaluateType::WorldTime:
							bSuccess = LiveLinkClient.EvaluateFrameAtWorldTime_AnyThread(InSubjectRepresentation.Subject, InWorldTime, InSubjectRepresentation.Role, FrameData);
							break;
						case EEvaluateType::SceneTime:
							bSuccess = LiveLinkClient.EvaluateFrameAtSceneTime_AnyThread(InSubjectRepresentation.Subject, InQualifiedFrameTime, InSubjectRepresentation.Role, FrameData);
							break;
						case EEvaluateType::Snapshot:
						default:
							bSuccess = LiveLinkClient.EvaluateFrame_AnyThread(InSubjectRepresentation.Subject, InSubjectRepresentation.Role, FrameData);
							break;
						}

						if (bSuccess)
						{

							bSuccess = InSubjectRepresentation.Role.GetDefaultObject()->InitializeBlueprintData(FrameData, BlueprintDataWrapper);
						}
					}
					P_NATIVE_END;
				}
				else
				{
					FBlueprintExceptionInfo ExceptionInfo(
						EBlueprintExceptionType::AccessViolation,
						NSLOCTEXT("EvaluateLiveLinkFrame", "IncompatibleProperty", "Incompatible output blueprint data; the role blueprint's data type is not the same as the return type.")
					);
					FBlueprintCoreDelegates::ThrowScriptException(InSelf, InStack, ExceptionInfo);
				}
			}
		}
		else
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AccessViolation,
				NSLOCTEXT("EvaluateLiveLinkFrame", "MissingOutputProperty", "Failed to resolve the output parameter for EvaluateLiveLinkFrame.")
			);
			FBlueprintCoreDelegates::ThrowScriptException(InSelf, InStack, ExceptionInfo);
		}

		return bSuccess;
	}
}

DEFINE_FUNCTION(ULiveLinkBlueprintLibrary::execEvaluateLiveLinkFrame)
{
	P_GET_STRUCT(FLiveLinkSubjectRepresentation, SubjectRepresentation);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(NULL);
	void* OutBlueprintDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* BlueprintDataStructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;

	double WorldTime = 0.0;
	FQualifiedFrameTime SceneTime;
	bool bSuccess = EvaluateLiveLinkFrame::Generic_Evaluate(P_THIS, Stack, BlueprintDataStructProp, SubjectRepresentation, EvaluateLiveLinkFrame::EEvaluateType::Snapshot, WorldTime, SceneTime, OutBlueprintDataPtr);
	*(bool*)RESULT_PARAM = bSuccess;
}

DEFINE_FUNCTION(ULiveLinkBlueprintLibrary::execEvaluateLiveLinkFrameWithSpecificRole)
{
	P_GET_STRUCT(FLiveLinkSubjectName, SubjectName);
	UClass* RoleClass;
	Stack.StepCompiledIn<FClassProperty>(&RoleClass);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(NULL);
	void* OutBlueprintDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* BlueprintDataStructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;


	double WorldTime = 0.0;
	FQualifiedFrameTime SceneTime;
	bool bSuccess = EvaluateLiveLinkFrame::Generic_Evaluate(P_THIS, Stack, BlueprintDataStructProp, {SubjectName, RoleClass}, EvaluateLiveLinkFrame::EEvaluateType::Snapshot, WorldTime, SceneTime, OutBlueprintDataPtr);
	*(bool*)RESULT_PARAM = bSuccess;
}

DEFINE_FUNCTION(ULiveLinkBlueprintLibrary::execEvaluateLiveLinkFrameAtWorldTimeOffset)
{

	P_GET_STRUCT(FLiveLinkSubjectName, SubjectName);
	UClass* RoleClass;
	Stack.StepCompiledIn<FClassProperty>(&RoleClass);
	float WorldTimeOffset = 0.f;
	Stack.StepCompiledIn<FFloatProperty>(&WorldTimeOffset);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(NULL);
	void* OutBlueprintDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* BlueprintDataStructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;

	double WorldTime = FApp::GetCurrentTime() + WorldTimeOffset;
	FQualifiedFrameTime SceneTime;
	bool bSuccess = EvaluateLiveLinkFrame::Generic_Evaluate(P_THIS, Stack, BlueprintDataStructProp, {SubjectName, RoleClass}, EvaluateLiveLinkFrame::EEvaluateType::WorldTime, WorldTime, SceneTime, OutBlueprintDataPtr);
	*(bool*)RESULT_PARAM = bSuccess;
}

DEFINE_FUNCTION(ULiveLinkBlueprintLibrary::execEvaluateLiveLinkFrameAtSceneTime)
{
	P_GET_STRUCT(FLiveLinkSubjectName, SubjectName);
	UClass* RoleClass;
	Stack.StepCompiledIn<FClassProperty>(&RoleClass);
	P_GET_STRUCT(FTimecode, SceneTime);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(NULL);
	void* OutBlueprintDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* BlueprintDataStructProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;

	double WorldTime = 0.0;
	bool bSuccess = EvaluateLiveLinkFrame::Generic_Evaluate(P_THIS, Stack, BlueprintDataStructProp, {SubjectName, RoleClass}, EvaluateLiveLinkFrame::EEvaluateType::SceneTime, WorldTime, FQualifiedFrameTime(SceneTime, FApp::GetTimecodeFrameRate()), OutBlueprintDataPtr);
	*(bool*)RESULT_PARAM = bSuccess;
}
