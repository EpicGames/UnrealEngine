// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkBroadcastComponent.h"

#include "Animation/AnimInstance.h"
#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkBroadcastSubsystem.h"
#include "Roles/LiveLinkBasicRole.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkCameraRole.h"
#include "Roles/LiveLinkCameraTypes.h"
#include "Roles/LiveLinkCharacterRole.h"
#include "Roles/LiveLinkCharacterTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkBroadcastComponent)

ULiveLinkBroadcastComponent::ULiveLinkBroadcastComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	bTickInEditor = true;
	bIsDirty = true;
}

void ULiveLinkBroadcastComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	ULiveLinkBroadcastSubsystem* BroadcastSubsystem = GEngine->GetEngineSubsystem<ULiveLinkBroadcastSubsystem>();

	BroadcastSubsystem->RemoveSubject(SubjectKey);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
#endif
	
	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

#if WITH_EDITOR
void ULiveLinkBroadcastComponent::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bIsDirty = PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, SubjectName)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, Role)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, AllowedCurveNames)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, AllowedBoneNames);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, bEnable))
	{
		if (!bEnable && bRemoveSubjectWhenDisabled)
		{
			if (ULiveLinkBroadcastSubsystem* BroadcastSubsystem = GEngine->GetEngineSubsystem<ULiveLinkBroadcastSubsystem>())
			{
				BroadcastSubsystem->RemoveSubject(SubjectKey);
			}
		}
		else
		{
			MarkDirty();
		}
	}
}

void ULiveLinkBroadcastComponent::OnObjectModified(UObject* ModifiedObject)
{
	if (ModifiedObject == CachedResolvedObject)
	{
		bIsDirty = true;
	}
}
#endif

void ULiveLinkBroadcastComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	USkeletalMeshComponent* SkeletalMeshComponent = GetOwner()->GetComponentByClass<USkeletalMeshComponent>();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &ULiveLinkBroadcastComponent::OnObjectModified);
#endif

	if (!Role)
	{
		// Try to infer the role from the components on the actor.
		if (SkeletalMeshComponent)
		{
			Role = ULiveLinkAnimationRole::StaticClass();

			// Set the mesh to the first skel mesh component we find if it's not set.
			if (SourceMesh.PathToComponent.IsEmpty())
			{
				SourceMesh.PathToComponent = SkeletalMeshComponent->GetPathName(GetOwner());
			}

			SkeletalMeshComponent->SetUpdateAnimationInEditor(true);
		}
		else if (UCameraComponent* CameraComponent = GetOwner()->GetComponentByClass<UCameraComponent>())
		{
			Role = ULiveLinkCameraRole::StaticClass();

			if (SourceMesh.PathToComponent.IsEmpty())
			{
				SourceMesh.PathToComponent = CameraComponent->GetPathName(GetOwner());
			}
		}
		else
		{
			Role = ULiveLinkTransformRole::StaticClass();

			if (SourceMesh.PathToComponent.IsEmpty())
			{
				SourceMesh.PathToComponent = GetOwner()->GetRootComponent()->GetPathName(GetOwner());
			}
		}
	}
}

void ULiveLinkBroadcastComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (!bEnable)
	{
		return;
	}

	if (bIsDirty)
	{
		if (SubjectName == NAME_None)
		{
			SubjectName = *GetTypedOuter<AActor>()->GetActorNameOrLabel();
		}

		CachedResolvedObject = SourceMesh.GetComponent(GetOwner());

		InitiateLiveLink();
		
		BroadcastStaticData();

		bIsDirty = false;
	}

	BroadcastFrameData();
}

void ULiveLinkBroadcastComponent::SetEnabled(bool bInEnabled)
{
	bEnable = bInEnabled;

	if (bRemoveSubjectWhenDisabled && !bEnable)
	{
		ULiveLinkBroadcastSubsystem* BroadcastSubsystem = GEngine->GetEngineSubsystem<ULiveLinkBroadcastSubsystem>();
		BroadcastSubsystem->RemoveSubject(SubjectKey);
	}
	else
	{
		MarkDirty();
	}
}

void ULiveLinkBroadcastComponent::InitiateLiveLink()
{
	if (Role && (!SourceMesh.PathToComponent.IsEmpty() || !SourceMesh.ComponentProperty.IsNone()))
	{
		SubjectKey = GEngine->GetEngineSubsystem<ULiveLinkBroadcastSubsystem>()->CreateSubject(SubjectName, Role);
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "LiveLinkBroadcastComponent role or source mesh isn't set on %ls", *GetOwner()->GetName());
	}
}

TArray<FName> ULiveLinkBroadcastComponent::GetSkeletalBoneNames() const
{
	TArray<FName> BoneNames;
		
	if (USkeletalMeshComponent* SourceSkelMesh = Cast<USkeletalMeshComponent>(SourceMesh.GetComponent(GetOwner())))
	{
		// If there are any entries in the allowed bones list, just use those bones names.
		if (AllowedBoneNames.Num() > 0)
		{
			BoneNames = AllowedBoneNames;
		}
		else
		{
			for (int32 Index = 0; Index < SourceSkelMesh->GetNumBones(); Index++)
			{
				FName BoneName = SourceSkelMesh->GetBoneName(Index);
				BoneNames.Emplace(BoneName);
			}
		}
	}

	return BoneNames;
}

TArray<int32> ULiveLinkBroadcastComponent::GetSkeletalBoneParents() const
{
	TArray<int32> BoneParents;
	TArray<FName> BoneNames;
	
	if (USkeletalMeshComponent* SourceSkelMesh = Cast<USkeletalMeshComponent>(SourceMesh.GetComponent(GetOwner())))
	{
		if (AllowedBoneNames.Num() > 0) //If there are any entries in the allowed bones list, just use those bones names.
		{
			for (FName BoneName : AllowedBoneNames)
			{
				FName ParentBoneName = SourceSkelMesh->GetParentBone(BoneName);
				int32 BoneParentIdx = AllowedBoneNames.Find(ParentBoneName);
				BoneParents.Add(BoneParentIdx);
			}
		}
		else
		{
			for (int32 Index = 0; Index < SourceSkelMesh->GetNumBones(); Index++)
			{
				FName BoneName = SourceSkelMesh->GetBoneName(Index);
				FName ParentBoneName = SourceSkelMesh->GetParentBone(BoneName);
				int32 BoneParentIndex = SourceSkelMesh->GetBoneIndex(ParentBoneName);
				BoneParents.Add(BoneParentIndex);
			}
		}
	}

	return BoneParents;
}

TArray<FTransform> ULiveLinkBroadcastComponent::GetBoneTransforms() const
{
	TArray<FTransform> BoneTransforms;

	if (USkeletalMeshComponent* SourceSkelMesh = Cast<USkeletalMeshComponent>(SourceMesh.GetComponent(GetOwner())))
	{
		auto GetBoneTransform = [SourceSkelMesh](int32 BoneIndex, TOptional<FName> BoneName = {})
		{
			if (!BoneName)
			{
				check(BoneIndex != INDEX_NONE);
				BoneName = SourceSkelMesh->GetBoneName(BoneIndex);
			}
			else
			{
				check(BoneName);
				BoneIndex = SourceSkelMesh->GetBoneIndex(*BoneName);
			}

			// Note: we purposefully don't call the SourceSkelMesh::GetBoneTransform override that takes a bone name, since it calls down into GetSocketTransform,
			// which is problematic when a socket has the same name as a bone.
			FTransform BoneTransform = SourceSkelMesh->GetBoneTransform(BoneIndex);

			FName ParentBone = SourceSkelMesh->GetParentBone(*BoneName);
			int32 ParentIndex = SourceSkelMesh->GetBoneIndex(ParentBone);
			if (ParentIndex != INDEX_NONE)
			{
				return BoneTransform.GetRelativeTransform(SourceSkelMesh->GetBoneTransform(ParentIndex));
			}
			else
			{
				return BoneTransform.GetRelativeTransform(SourceSkelMesh->GetComponentTransform());
			}
		};

		if (AllowedBoneNames.Num() > 0) //If there are any entries in the allowed bones list, just use those bones names.
		{
			for (FName BoneName : AllowedBoneNames)
			{
				BoneTransforms.Add(GetBoneTransform(INDEX_NONE, BoneName));
			}
		}
		else
		{
			for (int32 BoneIndex = 0; BoneIndex < SourceSkelMesh->GetNumBones(); BoneIndex++)
			{
				BoneTransforms.Add(GetBoneTransform(BoneIndex));
			}
		}
	}

	return BoneTransforms;
}

TArray<FName> ULiveLinkBroadcastComponent::GetCurveNames() const
{
	TArray<FName> CurveNames;

	USkeletalMeshComponent* SourceSkelMesh = Cast<USkeletalMeshComponent>(SourceMesh.GetComponent(GetOwner()));

	if (AllowedCurveNames.Num() > 0)
	{
		CurveNames = AllowedCurveNames;
	}

	return CurveNames;
}

TArray<float> ULiveLinkBroadcastComponent::GetCurveValues() const
{
	TArray<float> Curves;

	USkeletalMeshComponent* SourceSkelMesh = Cast<USkeletalMeshComponent>(SourceMesh.GetComponent(GetOwner()));

	TArray<FName> CurveNames = GetCurveNames();

	//TODO filter just for Metahuman control curves. Or allow user to filter on prefix? 
	for (FName Curve : CurveNames)
	{
		if (SourceSkelMesh->GetAnimInstance())
		{
			Curves.Add(SourceSkelMesh->GetAnimInstance()->GetCurveValue(Curve));
		}
	}

	return Curves;
}

void ULiveLinkBroadcastComponent::BroadcastStaticData() const
{
	if (!Role)
	{
		return;
	}

	FLiveLinkStaticDataStruct StaticDataStruct;

	if (Role->IsChildOf<ULiveLinkCharacterRole>())
	{
		StaticDataStruct = GetCharacterStaticData();
	}
	else if (Role->IsChildOf<ULiveLinkAnimationRole>())
	{
		StaticDataStruct = GetSkeletonStaticData();
	}
	else if (Role->IsChildOf<ULiveLinkCameraRole>())
	{
		StaticDataStruct = GetCameraStaticData();
	}
	else if (Role->IsChildOf<ULiveLinkTransformRole>())
	{
		StaticDataStruct = GetTransformStaticData();
	}

	ULiveLinkBroadcastSubsystem* BroadcastSubsystem = GEngine->GetEngineSubsystem<ULiveLinkBroadcastSubsystem>();
	BroadcastSubsystem->BroadcastStaticData(SubjectKey, Role, MoveTemp(StaticDataStruct));
}

FLiveLinkStaticDataStruct ULiveLinkBroadcastComponent::GetSkeletonStaticData() const
{
	FLiveLinkStaticDataStruct StaticDataStruct(FLiveLinkSkeletonStaticData::StaticStruct());
	FLiveLinkSkeletonStaticData& StaticData = *StaticDataStruct.Cast<FLiveLinkSkeletonStaticData>();

	StaticData.BoneNames = GetSkeletalBoneNames();
	StaticData.BoneParents = GetSkeletalBoneParents();
	StaticData.PropertyNames = GetCurveNames();

	return StaticDataStruct;
}

FLiveLinkFrameDataStruct ULiveLinkBroadcastComponent::GetSkeletonFrameData() const
{
	FLiveLinkFrameDataStruct FrameDataStruct(FLiveLinkAnimationFrameData::StaticStruct());
	FLiveLinkAnimationFrameData& FrameData = *FrameDataStruct.Cast<FLiveLinkAnimationFrameData>();
	FrameData.Transforms = GetBoneTransforms();
	FrameData.PropertyValues = GetCurveValues();

	return FrameDataStruct;
}


FLiveLinkStaticDataStruct ULiveLinkBroadcastComponent::GetCharacterStaticData() const
{
	FLiveLinkStaticDataStruct StaticDataStruct(FLiveLinkCharacterStaticData::StaticStruct());
	FLiveLinkCharacterStaticData& StaticData = *StaticDataStruct.Cast<FLiveLinkCharacterStaticData>();

	StaticData.BoneNames = GetSkeletalBoneNames();
	StaticData.BoneParents = GetSkeletalBoneParents();
	StaticData.PropertyNames = GetCurveNames();

	StaticData.bIsLocationSupported = true;
	StaticData.bIsRotationSupported = true;
	StaticData.bIsScaleSupported = true;

	return StaticDataStruct;
}

FLiveLinkFrameDataStruct ULiveLinkBroadcastComponent::GetCharacterFrameData() const
{

	FLiveLinkFrameDataStruct FrameDataStruct(FLiveLinkCharacterFrameData::StaticStruct());
	FLiveLinkCharacterFrameData& FrameData = *FrameDataStruct.Cast<FLiveLinkCharacterFrameData>();

	FrameData.Transforms = GetBoneTransforms();
	FrameData.PropertyValues = GetCurveValues();

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(SourceMesh.GetComponent(GetOwner())))
	{
		FrameData.Transform = SceneComponent->GetComponentTransform();
	}

	return FrameDataStruct;
}

FLiveLinkStaticDataStruct ULiveLinkBroadcastComponent::GetCameraStaticData() const
{
	FLiveLinkStaticDataStruct StaticDataStruct(FLiveLinkCameraStaticData::StaticStruct());
	FLiveLinkCameraStaticData& StaticData = *StaticDataStruct.Cast<FLiveLinkCameraStaticData>();

	StaticData.bIsLocationSupported = true;
	StaticData.bIsRotationSupported = true;
	StaticData.bIsScaleSupported = true;

	if (UCameraComponent* CameraComponent = Cast<UCameraComponent>(SourceMesh.GetComponent(GetOwner())))
	{
		if (UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(CameraComponent))
		{
			// Todo: Detect changes on the camera component to automatically update the static data when it's modified.
			StaticData.FilmBackWidth = CineCameraComponent->Filmback.SensorWidth;
			StaticData.FilmBackHeight = CineCameraComponent->Filmback.SensorHeight;

			StaticData.bIsFocalLengthSupported = true;
			StaticData.bIsDepthOfFieldSupported = true;
			StaticData.bIsApertureSupported = true;
			StaticData.bIsFocusDistanceSupported = true;
			StaticData.bIsDynamicFilmbackSupported = true;
		}

		StaticData.bIsFieldOfViewSupported = true;
		StaticData.bIsAspectRatioSupported = true;
		StaticData.bIsProjectionModeSupported = true;
	}

	return StaticDataStruct;
}

FLiveLinkFrameDataStruct ULiveLinkBroadcastComponent::GetCameraFrameData() const
{
	FLiveLinkFrameDataStruct FrameDataStruct(FLiveLinkCameraFrameData::StaticStruct());
	FLiveLinkCameraFrameData& FrameData = *FrameDataStruct.Cast<FLiveLinkCameraFrameData>();

	if (UCameraComponent* CameraComponent = Cast<UCameraComponent>(SourceMesh.GetComponent(GetOwner())))
	{
		FrameData.Transform = CameraComponent->GetComponentTransform();

		FrameData.FieldOfView = CameraComponent->FieldOfView;
		FrameData.AspectRatio = CameraComponent->AspectRatio;
		FrameData.ProjectionMode = CameraComponent->ProjectionMode == ECameraProjectionMode::Perspective ? ELiveLinkCameraProjectionMode::Perspective  : ELiveLinkCameraProjectionMode::Orthographic;

		if (UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(CameraComponent))
		{
			FrameData.FocalLength = CineCameraComponent->CurrentFocalLength;
			FrameData.Aperture = CineCameraComponent->CurrentAperture;
			FrameData.FocusDistance = CineCameraComponent->CurrentFocusDistance;
		}
	}

	return FrameDataStruct;
}

FLiveLinkStaticDataStruct ULiveLinkBroadcastComponent::GetTransformStaticData() const
{
	FLiveLinkStaticDataStruct StaticDataStruct(FLiveLinkTransformStaticData::StaticStruct());
	FLiveLinkTransformStaticData& StaticData = *StaticDataStruct.Cast<FLiveLinkTransformStaticData>();

	StaticData.bIsLocationSupported = true;
	StaticData.bIsRotationSupported = true;
	StaticData.bIsScaleSupported = true;

	return StaticDataStruct;
}

FLiveLinkFrameDataStruct ULiveLinkBroadcastComponent::GetTransformFrameData() const
{
	FLiveLinkFrameDataStruct FrameDataStruct(FLiveLinkTransformFrameData::StaticStruct());
	FLiveLinkTransformFrameData& FrameData = *FrameDataStruct.Cast<FLiveLinkTransformFrameData>();

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(SourceMesh.GetComponent(GetOwner())))
	{
		// Todo: Flag for relative vs absolute transform?
		FrameData.Transform = SceneComponent->GetComponentTransform();
	}

	return FrameDataStruct;
}

void ULiveLinkBroadcastComponent::BroadcastFrameData() const
{
	if (!Role)
	{
		return;
	}

	FLiveLinkFrameDataStruct FrameDataStruct;

	if (Role->IsChildOf<ULiveLinkCharacterRole>())
	{
		FrameDataStruct = GetCharacterFrameData();
	}
	else if (Role->IsChildOf<ULiveLinkAnimationRole>())
	{
		FrameDataStruct = GetSkeletonFrameData();
	}
	else if (Role->IsChildOf<ULiveLinkCameraRole>())
	{
		FrameDataStruct = GetCameraFrameData();
	}
	else if (Role->IsChildOf<ULiveLinkTransformRole>())
	{
		FrameDataStruct = GetTransformFrameData();
	}

	ULiveLinkBroadcastSubsystem* BroadcastSubsystem = GEngine->GetEngineSubsystem<ULiveLinkBroadcastSubsystem>();
	BroadcastSubsystem->BroadcastFrameData(SubjectKey, MoveTemp(FrameDataStruct));
}
