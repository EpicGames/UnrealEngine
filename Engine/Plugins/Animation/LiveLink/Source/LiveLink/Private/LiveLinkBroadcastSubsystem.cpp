// Copyright Epic Games, Inc. All Rights Reserved.


#include "LiveLinkBroadcastSubsystem.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkBroadcastComponent.h"
#include "LiveLinkPresetTypes.h"
#include "LiveLinkRole.h"
#include "LiveLinkSubjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkBroadcastSubsystem)


class FLiveLinkBroadcastSource : public ILiveLinkSource
{
public:
	FLiveLinkBroadcastSource(const FString& SubjectName)
	{
		SourceName = *FString::Printf(TEXT("%s"), *SubjectName);
	}

	virtual ~FLiveLinkBroadcastSource() = default;

	//~ Begin ILiveLinkSource interface
	virtual bool CanBeDisplayedInUI() const override
	{
		return true;
	}

	virtual bool IsSourceStillValid() const override
	{
		return true;
	}

	virtual bool RequestSourceShutdown() override
	{
		return true;
	}

	virtual FText GetSourceType() const override
	{
		return FText::FromName(SourceName);
	}

	virtual FText GetSourceMachineName() const override
	{
		return FText().FromString(FPlatformProcess::ComputerName());
	}

	virtual FText GetSourceStatus() const override
	{
		return NSLOCTEXT("LiveLinkBroadcastSource", "BroadcastSourceStatus", "Active");
	}

	void ReceiveClient(ILiveLinkClient*, FGuid)
	{
	}

protected:
	/** Source name. */
	FName SourceName;
};

FLiveLinkSubjectKey ULiveLinkBroadcastSubsystem::CreateSubject(FName SubjectName, TSubclassOf<ULiveLinkRole> Role)
{
	FLiveLinkSubjectKey SubjectKey;

	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		if (!LiveLinkClient->IsSourceStillValid(VirtualSourceGUID))
		{
			TSharedRef<FLiveLinkBroadcastSource> NewSource = MakeShared<FLiveLinkBroadcastSource>(TEXT("Live Link Actor Broadcast"));
			VirtualSourceGUID = LiveLinkClient->AddSource(NewSource);
		}

		SubjectKey = FLiveLinkSubjectKey{ VirtualSourceGUID, SubjectName };

		FLiveLinkSubjectPreset Preset;
		Preset.bEnabled = true;
		Preset.Key = SubjectKey;
		Preset.Role = Role;

		// CreateSubject handles the pending-kill case (removes and recreates),
		// and returns false if the subject already exists and is alive.
		if (LiveLinkClient->CreateSubject(Preset))
		{
			ULiveLinkSubjectSettings* Settings = CastChecked<ULiveLinkSubjectSettings>(LiveLinkClient->GetSubjectSettings(SubjectKey));
			Settings->bRebroadcastSubject = true;
		}
	}

	return SubjectKey;
}

void ULiveLinkBroadcastSubsystem::RemoveSubject(const FLiveLinkSubjectKey& SubjectKey)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		LiveLinkClient->RemoveSubject_AnyThread(SubjectKey);
	}
}

void ULiveLinkBroadcastSubsystem::BroadcastStaticData(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData) const
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		// Don't recreate the source if it's been explicitly deleted.
		if (LiveLinkClient->IsSourceStillValid(VirtualSourceGUID))
		{
			LiveLinkClient->PushSubjectStaticData_AnyThread(SubjectKey, Role, MoveTemp(StaticData));
		}
	}
}

void ULiveLinkBroadcastSubsystem::BroadcastFrameData(const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct&& FrameData) const
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		if (LiveLinkClient->IsSourceStillValid(VirtualSourceGUID))
		{
			LiveLinkClient->PushSubjectFrameData_AnyThread(SubjectKey, MoveTemp(FrameData));
		}
	}
}
