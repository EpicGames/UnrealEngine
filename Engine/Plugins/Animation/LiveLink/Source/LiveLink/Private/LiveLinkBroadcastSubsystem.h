// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "LiveLinkTypes.h"

#include "LiveLinkBroadcastSubsystem.generated.h"

class FLiveLinkBroadcastSource;
class ULiveLinkRole;

/**
 * Subsystem used primarily by the LiveLinkBroadcastComponent to rebroadcast a skeletal mesh on the network.
 */
UCLASS()
class ULiveLinkBroadcastSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
	
public:
	/** Create a broadcast subject with a given subject name. */
	FLiveLinkSubjectKey CreateSubject(FName SubjectName, TSubclassOf<ULiveLinkRole> Role);

	/** Remove a LiveLink s */
	void RemoveSubject(const FLiveLinkSubjectKey& SubjectKey);

	/** Broadcast static data for a subject. */
	void BroadcastStaticData(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData) const;

	/** Broadcast frame data for a subject. */
	void BroadcastFrameData(const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct&& FrameData) const;

private:
	/** ID of the source used to rebroadcast the data. */
	FGuid VirtualSourceGUID;
};
