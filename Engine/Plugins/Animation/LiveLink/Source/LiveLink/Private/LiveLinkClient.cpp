// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkClient.h"

#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "LiveLinkAnimationVirtualSubject.h"
#include "LiveLinkLog.h"
#include "LiveLinkMessages.h"
#include "LiveLinkPreset.h"
#include "LiveLinkPresetTypes.h"
#include "LiveLinkRoleTrait.h"
#include "LiveLinkProvider.h"
#include "LiveLinkSettings.h"
#include "LiveLinkSubjectRemapper.h"
#include "LiveLinkSourceCollection.h"
#include "LiveLinkSourceFactory.h"
#include "LiveLinkTimedDataInput.h"
#include "LiveLinkVirtualSource.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "VirtualSubjects/LiveLinkBlueprintVirtualSubject.h"
#endif

LLM_DEFINE_TAG(LiveLink_LiveLinkClient);

/**
 * Declare stats to see what takes up time in LiveLink
 */
DECLARE_CYCLE_STAT(TEXT("LiveLink - Push StaticData"), STAT_LiveLink_PushStaticData, STATGROUP_LiveLink);
DECLARE_CYCLE_STAT(TEXT("LiveLink - Push FrameData"), STAT_LiveLink_PushFrameData, STATGROUP_LiveLink);
DECLARE_CYCLE_STAT(TEXT("LiveLink - Client - Tick"), STAT_LiveLink_Client_Tick, STATGROUP_LiveLink);
DECLARE_CYCLE_STAT(TEXT("LiveLink - EvaluateFrame"), STAT_LiveLink_EvaluateFrame, STATGROUP_LiveLink);

DEFINE_LOG_CATEGORY(LogLiveLink);

static TAutoConsoleVariable<int32> CVarMaxNewStaticDataPerUpdate(
	TEXT("LiveLink.Client.MaxNewStaticDataPerUpdate"),
	256,
	TEXT("Maximun number of new static data that can be added in a single UE frame."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMaxNewFrameDataPerUpdate(
	TEXT("LiveLink.Client.MaxNewFrameDataPerUpdate"),
	2048,
	TEXT("Maximun number of new frame data that can be added in a single UE frame."),
	ECVF_Default);

FLiveLinkClient::FLiveLinkClient()
	: FLiveLinkClient(FCoreDelegates::OnSamplingInput)
{
	// Use OnSamplingInput as the ticking delegate for now since it's as close as the previous PreEngineCompleted callback we were hooked before it was changed
	// OnBeginFrame is too early since Timecode hasn't been updated for the frame
	// OnSamplingInput is right before ticking the engine so we can build our snapshots and be consistent throughout the frame
}


FLiveLinkClient::FLiveLinkClient(FSimpleMulticastDelegate& InTickingDelegate)
	: Collection(MakeUnique<FLiveLinkSourceCollection>())
{
	InTickingDelegate.AddRaw(this, &FLiveLinkClient::Tick, ELiveLinkTickType::Default);
	Initialize();
}

FLiveLinkClient::FLiveLinkClient(FTSSimpleMulticastDelegate& InTickingDelegate)
	: Collection(MakeUnique<FLiveLinkSourceCollection>())
{
	InTickingDelegate.AddRaw(this, &FLiveLinkClient::Tick, ELiveLinkTickType::Default);
	Initialize();
}

FLiveLinkClient::FLiveLinkClient(FLiveLinkTickDelegate& InTickingDelegate)
	: Collection(MakeUnique<FLiveLinkSourceCollection>())
{
	InTickingDelegate.AddRaw(this, &FLiveLinkClient::Tick);
	Initialize();
}

FLiveLinkClient::~FLiveLinkClient()
{
	FCoreDelegates::OnPreExit.RemoveAll(this);
	Shutdown();
}

void FLiveLinkClient::Tick(ELiveLinkTickType TickType)
{
	SCOPE_CYCLE_COUNTER(STAT_LiveLink_Client_Tick);

	TSet<FLiveLinkSubjectKey> ReceivedSubjects;

	DoPendingWork(ReceivedSubjects);

	if (TickType == ELiveLinkTickType::Scheduled || TickType == ELiveLinkTickType::Default)
	{
		CacheValues();
		UpdateSources();
	}

	TSet<FLiveLinkSubjectKey> UpdatedSubjects = BuildThisTicksSubjectSnapshot(ReceivedSubjects);
	RebroadcastSubjects(UpdatedSubjects);

	OnLiveLinkTickedDelegate.Broadcast();
}

void FLiveLinkClient::Initialize()
{
	LLM_SCOPE_BYTAG(LiveLink_LiveLinkClient);

	OnLiveLinkSubjectRemoved().AddRaw(this, &FLiveLinkClient::OnSubjectRemovedCallback);
	FCoreDelegates::OnPreExit.AddRaw(this, &FLiveLinkClient::Shutdown);

	// Setup rebroadcaster name in case we need it later
	RebroadcastLiveLinkProviderName = TEXT("LiveLink Rebroadcast");

	bPreProcessRebroadcastFrames = GetDefault<ULiveLinkSettings>()->bPreProcessRebroadcastFrames;
	bTranslateRebroadcastFrames = GetDefault<ULiveLinkSettings>()->bTranslateRebroadcastFrames;
	bEnableParentSubjects = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bEnableParentSubjects"), false, GEngineIni);
}

void FLiveLinkClient::DoPendingWork(TSet<FLiveLinkSubjectKey>& OutUpdatedSubjects)
{
	check(Collection);

	// Remove Sources and Subjects
	// Note MH-15807: This needs to happen outside of the PendingFramesCriticalSection, since deleting a subject could cause it to way on its thread,
	// which could already have locked this critical section, causing a deadlock.
	Collection->RemovePendingKill();

	FScopeLock PendingFramesLock(&PendingFramesCriticalSection);

	{
		// Add new Subject static data
		for (FPendingSubjectStatic& SubjectStatic : SubjectStaticToPush)
		{
			OutUpdatedSubjects.Add(SubjectStatic.SubjectKey);
			PushSubjectStaticData_Internal(MoveTemp(SubjectStatic));
		}
		SubjectStaticToPush.Reset();

		// Add new Subject frame data
		for (FPendingSubjectFrame& SubjectFrame : SubjectFrameToPush)
		{
			OutUpdatedSubjects.Add(SubjectFrame.SubjectKey);
			PushSubjectFrameData_Internal(MoveTemp(SubjectFrame));
		}
		SubjectFrameToPush.Reset();
	}
}

void FLiveLinkClient::UpdateSources()
{
	Collection->ForEachSource([](const FLiveLinkCollectionSourceItem& SourceItem)
	{
		SourceItem.Source->Update();
	});
}

bool FLiveLinkClient::HandleSubjectRebroadcast(ILiveLinkSubject* InSubject, FLiveLinkSubjectFrameData&& SubjectFrameData)
{
	// Check the rebroadcast flag and act accordingly, creating the LiveLinkProvider and/or sending the static data if needed
	if (InSubject->IsRebroadcasted())
	{
		if (InSubject->GetStaticData().IsValid() && SubjectFrameData.FrameData.IsValid())
		{
			// Setup rebroadcast provider
			if (!RebroadcastLiveLinkProvider.IsValid())
			{
				RebroadcastLiveLinkProvider = GetRebroadcastLiveLinkProvider();
			}
				
			if (RebroadcastLiveLinkProvider.IsValid())
			{
				TSubclassOf<ULiveLinkRole> SubjectRole = InSubject->GetRole();

				const FName RebroadcastName = GetRebroadcastName(InSubject->GetSubjectKey());

				const FText OriginalSourceType = GetSourceType(InSubject->GetSubjectKey().Source);

				TMap<FName, FString> ExtraAnnotations;
				ExtraAnnotations.Add(FLiveLinkMessageAnnotation::OriginalSourceAnnotation, OriginalSourceType.ToString());

				if (!InSubject->HasStaticDataBeenRebroadcasted())
				{
					RebroadcastLiveLinkProvider->UpdateSubjectStaticData(RebroadcastName, SubjectRole, MoveTemp(SubjectFrameData.StaticData), ExtraAnnotations);
					InSubject->SetStaticDataAsRebroadcasted(true);
					RebroadcastedSubjects.Add(InSubject->GetSubjectKey());
				}

				RebroadcastLiveLinkProvider->UpdateSubjectFrameData(RebroadcastName, MoveTemp(SubjectFrameData.FrameData), ExtraAnnotations);

				return true;
			}
			else
			{
				UE_LOGF(LogLiveLink, Warning, "Rebroadcaster doesn't exist, but was requested and failed");
			}
		}
	}
	else if (InSubject->HasStaticDataBeenRebroadcasted())
	{
		RemoveRebroadcastedSubject(InSubject->GetSubjectKey());
		InSubject->SetStaticDataAsRebroadcasted(false);
	}

	return false;
}

TSet<FLiveLinkSubjectKey> FLiveLinkClient::BuildThisTicksSubjectSnapshot(const TSet<FLiveLinkSubjectKey>& ReceivedSubjects)
{
	EnabledSubjects.Reset();

	TSet<FGuid> TaggedSources;
	TSet<FLiveLinkSubjectKey> UpdatedSubjects;

	// Update the Live Subject before the Virtual Subject
	ForEachLiveSubject([this, &TaggedSources, &UpdatedSubjects](FLiveLinkSubject* LiveSubject, FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			if (SubjectItem.bEnabled)
			{
				ULiveLinkSourceSettings* SourceSettings = SourceItem.Setting.Get();
				ULiveLinkSubjectSettings* SubjectSettings = SubjectItem.GetLinkSettings();

				LiveSubject->CacheSettings(SourceSettings, SubjectSettings);
				LiveSubject->Update();
				UpdatedSubjects.Add(SubjectItem.Key);
				EnabledSubjects.Add(SubjectItem.Key.SubjectName, SubjectItem.Key);

				// Update Source FrameRate from first enabled subject with valid data.
				if (LiveSubject->HasValidFrameSnapshot() && SubjectSettings->FrameRate.IsValid() && !TaggedSources.Contains(SubjectItem.Key.Source))
				{
					SourceSettings->BufferSettings.DetectedFrameRate = SubjectSettings->FrameRate;
					TaggedSources.Add(SubjectItem.Key.Source);
				}
			}
			else
			{
				if (!LiveSubject->IsPaused())
				{
					LiveSubject->ClearFrames();
				}
			}
		});

	// Update Virtual Subjects
	ForEachVirtualSubject([this, &ReceivedSubjects, &UpdatedSubjects](ULiveLinkVirtualSubject* VSubject, FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			if (SubjectItem.bEnabled)
			{
				const bool bSubjectIsSynchronized = SourceItem.Setting->ParentSubject != FLiveLinkSubjectName() || VSubject->SyncSubject != FLiveLinkSubjectName();

				bool bSubjectNeedsUpdate = false;

				if (bSubjectIsSynchronized)
				{
					if (SubjectItem.bEnabled && !VSubject->IsPaused())
					{
						for (const FLiveLinkSubjectKey& ReceivedSubject : ReceivedSubjects)
						{
							// We can either synchronize at the source or subject level to another subject.
							if (SourceItem.Setting->ParentSubject == ReceivedSubject.SubjectName
								|| VSubject->SyncSubject == ReceivedSubject.SubjectName)
							{
								bSubjectNeedsUpdate = true;
								break;
							}
						}
					}
				}
				else
				{
					bSubjectNeedsUpdate = true;
				}

				if (bSubjectNeedsUpdate)
				{
					VSubject->Update();
					UpdatedSubjects.Add(SubjectItem.Key);
				
					// If we want the LiveLinkTimecodeProvider to pick up on the timecode from the Virtual Subject, then we 
					// need to broadcast this delegate as if it were any other subject that received data.
					if (VSubject->HasValidFrameSnapshot())
					{
						BroadcastFrameDataUpdate(VSubject->GetSubjectKey(), VSubject->GetFrameData());
					}
				}

				EnabledSubjects.Add(SubjectItem.Key.SubjectName, SubjectItem.Key);
			}
			else
			{
				VSubject->ClearFrames();
			}
		});

	return UpdatedSubjects;
}

void FLiveLinkClient::RebroadcastSubjects(const TSet<FLiveLinkSubjectKey>& UpdatedSubjects)
{
	// Rebroadcast paused live subjects
	ForEachLiveSubject([this](FLiveLinkSubject* LiveSubject, FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			if (SubjectItem.bEnabled && LiveSubject->IsPaused())
			{
				HandleSubjectRebroadcast(LiveSubject, LiveSubject->GetFrameSnapshot().FrameData);
			}
		});

	// Rebroadcast Live Subjects (TransmitEvaluatedData path)
	ForEachLiveSubject([this, UpdatedSubjects](FLiveLinkSubject* LiveSubject, FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			if (SourceItem.Setting->bTransmitEvaluatedData)
			{
				// Evaluated path acts like a VSubject with a single dependant as far as rebroadcast goes.
				// So make sure to only rebroadcast it when we received an update for it.
				if (SubjectItem.bEnabled
					&& !LiveSubject->IsPaused()
					&& UpdatedSubjects.Contains(SubjectItem.Key))
				{
					FLiveLinkFrameIdentifier SnapshotFrameId = LiveSubject->GetSnapshotFrameId();

					if (LiveSubject->GetSnapshotFrameId() != LiveSubject->GetLastRebroadcastedFrameId() || SnapshotFrameId == -1)
					{
						FLiveLinkSubjectFrameData FrameData;

						// todo: Evaluate at time
						if (LiveSubject->EvaluateFrame(LiveSubject->GetRole(), FrameData))
						{
							if (HandleSubjectRebroadcast(SubjectItem.GetLiveSubject(), MoveTemp(FrameData)))
							{
								LiveSubject->SetLastRebroadcastedFrameId(SnapshotFrameId);
							}
						}
					}
				}
			}
		});

	// Rebroadcast Virtual Subjects
	ForEachVirtualSubject([this, UpdatedSubjects](ULiveLinkVirtualSubject* VSubject, FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			if (UpdatedSubjects.Contains(SubjectItem.Key))
			{
				HandleSubjectRebroadcast(VSubject, VSubject->GetFrameData());
			}
		});
}

void FLiveLinkClient::CacheValues()
{
#if WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAppTime Time = FAppTime::Get_UnsafeNoCheck();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (Time.CurrentFrameTime.IsSet())
	{
		CachedEngineFrameTime = Time.CurrentFrameTime;
	}
#endif
}

void FLiveLinkClient::SetLastSavedPreset(ULiveLinkPreset* InPreset)
{
	LastSavedPreset = InPreset;
}

ULiveLinkPreset* FLiveLinkClient::GetLastSavedPreset() const
{
	return LastSavedPreset.Get();
}

void FLiveLinkClient::Shutdown()
{
	LLM_SCOPE_BYTAG(LiveLink_LiveLinkClient);

	FCoreDelegates::OnSamplingInput.RemoveAll(this);

	// Shut down the rebroadcaster if active
	if (RebroadcastLiveLinkProvider.IsValid())
	{
		RebroadcastLiveLinkProvider.Reset();
	}

	if (Collection)
	{
		OnLiveLinkSubjectRemoved().RemoveAll(this);

		double Timeout = 2.0;
		GConfig->GetDouble(TEXT("LiveLink"), TEXT("ClientShutdownTimeout"), Timeout, GGameIni);

		const double StartShutdownSeconds = FPlatformTime::Seconds();
		bool bContinue = true;
		while(bContinue)
		{
			bContinue = !Collection->RequestShutdown();

			if (FPlatformTime::Seconds() - StartShutdownSeconds > Timeout)
			{
				bContinue = false;
				UE_LOGF(LogLiveLink, Warning, "Force shutdown LiveLink after %f seconds. One or more sources refused to shutdown.", Timeout);
			}
		}
	}
}

bool FLiveLinkClient::HandleSubjectRebroadcast(ILiveLinkSubject* InSubject, const FLiveLinkFrameDataStruct& InFrameData)
{
	check(InSubject);

	// Check the rebroadcast flag and act accordingly, creating the LiveLinkProvider and/or sending the static data if needed
	if (InSubject->IsRebroadcasted())
	{
		if (InSubject->GetStaticData().IsValid() && InFrameData.IsValid())
		{
			// Setup rebroadcast provider
			if (!RebroadcastLiveLinkProvider.IsValid())
			{
				RebroadcastLiveLinkProvider = GetRebroadcastLiveLinkProvider();
			}

			if (RebroadcastLiveLinkProvider.IsValid())
			{
				// Make a copy of the data for use by the rebroadcaster
				FLiveLinkFrameDataStruct FrameDataCopy;
				FrameDataCopy.InitializeWith(InFrameData);

				FLiveLinkStaticDataStruct StaticDataCopy;
				StaticDataCopy.InitializeWith(InSubject->GetStaticData());

				if (bPreProcessRebroadcastFrames)
				{
					InSubject->PreprocessFrame(StaticDataCopy, FrameDataCopy);
				}

				TSubclassOf<ULiveLinkRole> SubjectRole = InSubject->GetRole();

				if (bTranslateRebroadcastFrames)
				{
					TArray<ULiveLinkFrameTranslator::FWorkerSharedPtr> Translators = InSubject->GetFrameTranslators();
					if (Translators.Num() && Translators[0].IsValid())
					{
						FLiveLinkSubjectFrameData TranslatedFrameData;
						if (Translators[0]->Translate(InSubject->GetStaticData(), FrameDataCopy, TranslatedFrameData))
						{
							SubjectRole = Translators[0]->GetToRole();
							StaticDataCopy = MoveTemp(TranslatedFrameData.StaticData);
							FrameDataCopy = MoveTemp(TranslatedFrameData.FrameData);
						}
					}
				}

				const FName RebroadcastName = GetRebroadcastName(InSubject->GetSubjectKey());

				const FText OriginalSourceType = GetSourceType(InSubject->GetSubjectKey().Source);

				TMap<FName, FString> ExtraAnnotations;
				ExtraAnnotations.Add(FLiveLinkMessageAnnotation::OriginalSourceAnnotation, OriginalSourceType.ToString());

				if (!InSubject->HasStaticDataBeenRebroadcasted())
				{
					RebroadcastLiveLinkProvider->UpdateSubjectStaticData(RebroadcastName, SubjectRole, MoveTemp(StaticDataCopy), ExtraAnnotations);
					InSubject->SetStaticDataAsRebroadcasted(true);
					RebroadcastedSubjects.Add(InSubject->GetSubjectKey());
				}

				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FLiveLinkClient::HandleSubjectRebroadcast %s"), *InSubject->GetSubjectKey().SubjectName.Name.ToString()));
				RebroadcastLiveLinkProvider->UpdateSubjectFrameData(RebroadcastName, MoveTemp(FrameDataCopy), ExtraAnnotations);

				return true;
			}
			else
			{
				UE_LOGF(LogLiveLink, Warning, "Rebroadcaster doesn't exist, but was requested and failed");
			}
		}
	}
	else if (InSubject->HasStaticDataBeenRebroadcasted())
	{
		RemoveRebroadcastedSubject(InSubject->GetSubjectKey());
		InSubject->SetStaticDataAsRebroadcasted(false);
	}

	return false;
}

void FLiveLinkClient::OnSubjectRemovedCallback(FLiveLinkSubjectKey InSubjectKey)
{
	RemoveRebroadcastedSubject(InSubjectKey);

	{
		FScopeLock Lock(&SubjectFrameCountersLock);
		SubjectFrameCounters.Remove(InSubjectKey);
	}
}

void FLiveLinkClient::RemoveRebroadcastedSubject(FLiveLinkSubjectKey InSubjectKey)
{
	if (RebroadcastLiveLinkProvider.IsValid())
	{
		const FName SubjectName = GetRebroadcastName(InSubjectKey);

		if (RebroadcastedSubjects.Contains(InSubjectKey))
		{
			RebroadcastLiveLinkProvider->RemoveSubject(SubjectName);
			RebroadcastedSubjects.Remove(InSubjectKey);

			if (RebroadcastedSubjects.Num() <= 0)
			{
				RebroadcastLiveLinkProvider.Reset();
			}
		}
	}
}

void FLiveLinkClient::ForEachLiveSubject(TFunctionRef<void(FLiveLinkSubject*, FLiveLinkCollectionSourceItem&, FLiveLinkCollectionSubjectItem&)> VisitorFunc)
{
	check(Collection);

	Collection->ForEachSubject([VisitorFunc](FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			if (SubjectItem.GetLiveSubject())
			{
				VisitorFunc(SubjectItem.GetLiveSubject(), SourceItem, SubjectItem);
			}
		});
}

void FLiveLinkClient::ForEachVirtualSubject(TFunctionRef<void(ULiveLinkVirtualSubject*, FLiveLinkCollectionSourceItem&, FLiveLinkCollectionSubjectItem&)> VisitorFunc)
{
	check(Collection);

	Collection->ForEachSubject([VisitorFunc](FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			if (SubjectItem.GetVirtualSubject())
			{
				VisitorFunc(SubjectItem.GetVirtualSubject(), SourceItem, SubjectItem);
			}
		});
}

ELiveLinkSubjectState FLiveLinkClient::GetSubjectState(FLiveLinkSubjectName InSubjectName) const
{
	const FLiveLinkSubjectKey* SubjectKey = EnabledSubjects.Find(InSubjectName);

	if (!SubjectKey)
	{
		return ELiveLinkSubjectState::InvalidOrDisabled;
	}

	const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(*SubjectKey);

	if (!SubjectItem)
	{
		return ELiveLinkSubjectState::InvalidOrDisabled;
	}

	const FLiveLinkSubject* LiveSubject = SubjectItem->GetLiveSubject();

	if (!LiveSubject)
	{
		if (ULiveLinkVirtualSubject* VirtualSubject = SubjectItem->GetVirtualSubject())
		{
			if (!SubjectItem->bEnabled)
			{
				return ELiveLinkSubjectState::InvalidOrDisabled;
			}
			else
			{
				return VirtualSubject->GetState();
			}
		}
		else
		{
			return ELiveLinkSubjectState::InvalidOrDisabled;
		}
	}

	if (LiveSubject->IsPaused())
	{
		return ELiveLinkSubjectState::Paused;
	}

	switch (const ETimedDataInputState InputState = LiveSubject->GetState())
	{
	case ETimedDataInputState::Connected: return ELiveLinkSubjectState::Connected;
	case ETimedDataInputState::Unresponsive: return ELiveLinkSubjectState::Unresponsive;
	case ETimedDataInputState::Disconnected: return ELiveLinkSubjectState::Disconnected;
	default:
		ensureMsgf(false, TEXT("Unhandled ETimedDataInputState::%d"), EnumToUnderlyingType(InputState));
		return ELiveLinkSubjectState::Unknown;
	}
}

FGuid FLiveLinkClient::AddSource(TSharedPtr<ILiveLinkSource> InSource)
{
	check(Collection);

	FGuid Guid;
	if (FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InSource))
	{
		SourceItem->bPendingKill = false;
		Guid = SourceItem->Guid;
	}
	else
	{
		ULiveLinkSourceSettings* Settings = nullptr;
		Guid = FGuid::NewGuid();

		FLiveLinkCollectionSourceItem Data;
		Data.Guid = Guid;
		Data.Source = InSource;
		Data.TimedData = MakeShared<FLiveLinkTimedDataInput>(this, Guid);
		{
			UClass* SourceSettingsClass = InSource->GetSettingsClass().Get();
			UClass* SettingsClass = SourceSettingsClass ? SourceSettingsClass : ULiveLinkSourceSettings::StaticClass();
			Data.Setting = TStrongObjectPtr(NewObject<ULiveLinkSourceSettings>(GetTransientPackage(), SettingsClass));
			Settings = Data.Setting.Get();
		}
		Collection->AddSource(MoveTemp(Data));

		InSource->ReceiveClient(this, Guid);
		InSource->InitializeSettings(Settings);
	}

	return Guid;
}

FGuid FLiveLinkClient::AddVirtualSubjectSource(FName SourceName)
{
	check(Collection);

	FGuid Guid;

	if (Collection->FindVirtualSource(SourceName) == nullptr)
	{
		TSharedPtr<FLiveLinkVirtualSubjectSource> Source = MakeShared<FLiveLinkVirtualSubjectSource>();
		Guid = FGuid::NewGuid();

		FLiveLinkCollectionSourceItem Data;
		Data.Guid = Guid;
		Data.Source = Source;

		ULiveLinkVirtualSubjectSourceSettings* NewSettings = NewObject<ULiveLinkVirtualSubjectSourceSettings>(GetTransientPackage(), ULiveLinkVirtualSubjectSourceSettings::StaticClass());
		NewSettings->SourceName = SourceName;
		Data.Setting = TStrongObjectPtr(NewSettings);
		Data.bIsVirtualSource = true;
		Data.TimedData = MakeShared<FLiveLinkTimedDataInput>(this, Guid);
		Collection->AddSource(MoveTemp(Data));

		Source->ReceiveClient(this, Guid);
		Source->InitializeSettings(NewSettings);
	}
	else
	{
		FLiveLinkLog::Warning(TEXT("The virtual subject Source '%s' could not be created. It already exists."), *SourceName.ToString());
	}

	return Guid;
}

bool FLiveLinkClient::CreateSource(const FLiveLinkSourcePreset& InSourcePreset)
{
	check(Collection);

	if (InSourcePreset.Settings == nullptr)
	{
		FLiveLinkLog::Warning(TEXT("Create Source Failure: The settings are not defined."));
		return false;
	}

	if (InSourcePreset.Guid == FLiveLinkSourceCollection::DefaultVirtualSubjectGuid)
	{
		FLiveLinkLog::Warning(TEXT("Create Source Failure: Can't create default virtual subject source. It will be created automatically."));
		return false;
	}

	if (!InSourcePreset.Guid.IsValid())
	{
		FLiveLinkLog::Warning(TEXT("Create Source Failure: The guid is invalid."));
		return false;
	}

	if (Collection->FindSource(InSourcePreset.Guid) != nullptr)
	{
		FLiveLinkLog::Warning(TEXT("Create Source Failure: The guid already exist."));
		return false;
	}

	ULiveLinkSourceSettings* Setting = nullptr;
	TSharedPtr<ILiveLinkSource> Source;
	FLiveLinkCollectionSourceItem Data;
	Data.Guid = InSourcePreset.Guid;

	// Virtual subject source have a special settings class. We can differentiate them using this
	if (InSourcePreset.Settings->GetClass() == ULiveLinkVirtualSubjectSourceSettings::StaticClass())
	{
		Source = MakeShared<FLiveLinkVirtualSubjectSource>();
		Data.bIsVirtualSource = true;
	}
	else
	{
		if (InSourcePreset.Settings->Factory.Get() == nullptr || InSourcePreset.Settings->Factory.Get() == ULiveLinkSourceFactory::StaticClass())
		{
			FLiveLinkLog::Warning(TEXT("Create Source Failure: The factory is not defined."));
			return false;
		}

		Source = InSourcePreset.Settings->Factory.Get()->GetDefaultObject<ULiveLinkSourceFactory>()->CreateSource(InSourcePreset);
		if (!Source.IsValid())
		{
			FLiveLinkLog::Warning(TEXT("Create Source Failure: The source couldn't be created by the factory."));
			return false;
		}

		Data.TimedData = MakeShared<FLiveLinkTimedDataInput>(this, InSourcePreset.Guid);
	}

	Data.Source = Source;

	//In case a source has changed its source settings class, instead of duplicating, create the right one and copy previous properties
	UClass* SourceSettingsClass = Source->GetSettingsClass().Get();
	if (SourceSettingsClass && SourceSettingsClass != InSourcePreset.Settings->GetClass())
	{
		FLiveLinkLog::Info(TEXT("Creating Source '%s' from Preset: Settings class '%s' is not what is expected ('%s'). Updating to new class."), *InSourcePreset.SourceType.ToString(), *InSourcePreset.Settings->GetClass()->GetName(), *SourceSettingsClass->GetName());
		Setting = NewObject<ULiveLinkSourceSettings>(GetTransientPackage(), SourceSettingsClass);
		UEngine::CopyPropertiesForUnrelatedObjects(InSourcePreset.Settings, Setting);
		Data.Setting = TStrongObjectPtr(Setting);
	}
	else
	{
		Data.Setting = TStrongObjectPtr(DuplicateObject<ULiveLinkSourceSettings>(InSourcePreset.Settings, GetTransientPackage()));
		Setting = Data.Setting.Get();
	}

	Collection->AddSource(MoveTemp(Data));
	Source->ReceiveClient(this, InSourcePreset.Guid);
	Source->InitializeSettings(Setting);

	return true;
}

void FLiveLinkClient::RemoveSource(TSharedPtr<ILiveLinkSource> InSource)
{
	check(Collection);
	if (FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InSource))
	{
		SourceItem->bPendingKill = true;
	}
}

void FLiveLinkClient::RemoveSource(FGuid InEntryGuid)
{
	check(Collection);
	if (FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		SourceItem->bPendingKill = true;
	}
}

void FLiveLinkClient::RemoveAllSources()
{
	CachedPresetSubjectSettings.Reset();

	check(Collection);

	Collection->ForEachSource([](FLiveLinkCollectionSourceItem& SourceItem)
	{
		// Don't remove virtual subject sources, or restoring Virtual Subjects will fail because their source doesn't exist anymore.
		if (!SourceItem.bIsVirtualSource)
		{
			SourceItem.bPendingKill = true;
		}
	});
}

bool FLiveLinkClient::RemoveAllSourcesWithTimeout(float InTimeout)
{
	RemoveAllSources();

	const double MaxTime = FPlatformTime::Seconds() + InTimeout;

	auto GetNumNonDefaultSources = [this]()
	{
		int32 NumNonDefaultSources = 0;
		Collection->ForEachSource([&NumNonDefaultSources] (const FLiveLinkCollectionSourceItem& SourceItem)
		{
			if (SourceItem.Guid != FLiveLinkSourceCollection::DefaultVirtualSubjectGuid)
			{
				NumNonDefaultSources++;
			}
		});

		return NumNonDefaultSources;
	};
	
	while (GetNumNonDefaultSources() > 0)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime >= MaxTime)
		{
			return false;
		}

		FPlatformProcess::Sleep(0.002f);
	}

	return true;
}

bool FLiveLinkClient::HasSourceBeenAdded(TSharedPtr<ILiveLinkSource> InSource) const
{
	check(Collection);
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InSource))
	{
		return !SourceItem->bPendingKill;
	}
	return false;
}

TArray<FGuid> FLiveLinkClient::GetSources(bool bEvenIfPendingKill) const
{
	check(Collection);

	TArray<FGuid> Result;
	Collection->ForEachSource([&Result, bEvenIfPendingKill](const FLiveLinkCollectionSourceItem& SourceItem)
	{
		if ((!SourceItem.bPendingKill || bEvenIfPendingKill) && !SourceItem.IsVirtualSource())
		{
			Result.Add(SourceItem.Guid);
		}
	});

	return Result;
}

TArray<FGuid> FLiveLinkClient::GetVirtualSources(bool bEvenIfPendingKill) const
{
	check(Collection);

	TArray<FGuid> Result;

	Collection->ForEachSource([&Result, bEvenIfPendingKill](const FLiveLinkCollectionSourceItem& SourceItem)
	{
		if ((!SourceItem.bPendingKill || bEvenIfPendingKill) && SourceItem.IsVirtualSource())
		{
			Result.Add(SourceItem.Guid);
		}
	});

	return Result;
}

FLiveLinkSourcePreset FLiveLinkClient::GetSourcePreset(FGuid InSourceGuid, UObject* InDuplicatedObjectOuter) const
{
	check(Collection);

	UObject* DuplicatedObjectOuter = InDuplicatedObjectOuter ? InDuplicatedObjectOuter : GetTransientPackage();

	FLiveLinkSourcePreset SourcePreset;
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InSourceGuid))
	{
		if (SourceItem->Guid != FLiveLinkSourceCollection::DefaultVirtualSubjectGuid && SourceItem->Setting && SourceItem->Source)
		{
			SourcePreset.Guid = SourceItem->Guid;
			SourcePreset.SourceType = SourceItem->Source->CanBeDisplayedInUI() ? SourceItem->Source->GetSourceType() : FText::GetEmpty();
			SourcePreset.Settings = DuplicateObject<ULiveLinkSourceSettings>(SourceItem->Setting.Get(), DuplicatedObjectOuter);
		}
	}
	return SourcePreset;
}

void FLiveLinkClient::PushSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, FLiveLinkStaticDataStruct&& InStaticData)
{
	FPendingSubjectStatic SubjectStatic{ InSubjectKey, InRole, MoveTemp(InStaticData) };
	PushPendingSubject_AnyThread(MoveTemp(SubjectStatic));
}

void FLiveLinkClient::PushSubjectStaticData_Internal(FPendingSubjectStatic&& SubjectStaticData)
{
	SCOPE_CYCLE_COUNTER(STAT_LiveLink_PushStaticData);

	check(Collection);

	if (!FLiveLinkRoleTrait::Validate(SubjectStaticData.Role, SubjectStaticData.StaticData))
	{
		if (SubjectStaticData.Role == nullptr)
		{
			FLiveLinkLog::Error(TEXT("Trying to add unsupported static data type with subject '%s'."), *SubjectStaticData.SubjectKey.SubjectName.ToString());
		}
		else
		{
			FLiveLinkLog::Error(TEXT("Trying to add unsupported static data type to role '%s' with subject '%s'."), *SubjectStaticData.Role->GetName(), *SubjectStaticData.SubjectKey.SubjectName.ToString());
		}
		return;
	}

	bool bShouldLogIfInvalidStaticData = true;
	if (!SubjectStaticData.Role.GetDefaultObject()->IsStaticDataValid(SubjectStaticData.StaticData, bShouldLogIfInvalidStaticData))
	{
		if (bShouldLogIfInvalidStaticData)
		{
			FLiveLinkLog::Error(TEXT("Trying to add static data that is not formatted properly to role '%s' with subject '%s'."), *SubjectStaticData.Role->GetName(), *SubjectStaticData.SubjectKey.SubjectName.ToString());
		}
		return;
	}
	
	const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(SubjectStaticData.SubjectKey.Source);
	if (SourceItem == nullptr || SourceItem->bPendingKill)
	{
		return;
	}

	// We use a strong object ptr to prevent GC for this object in case this was created outside the game thread.
	TStrongObjectPtr<ULiveLinkSubjectSettings> SubjectSettings;

	FLiveLinkSubject* LiveLinkSubject = nullptr;
	{
		if (FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(SubjectStaticData.SubjectKey))
		{
			if (!SubjectItem->bPendingKill)
			{
				LiveLinkSubject = SubjectItem->GetLiveSubject();

				if (LiveLinkSubject->GetRole() != SubjectStaticData.Role)
				{
					FLiveLinkLog::Warning(TEXT("Subject '%s' of role '%s' is changing its role to '%s'. Current subject will be removed and a new one will be created"), *SubjectStaticData.SubjectKey.SubjectName.ToString(), *LiveLinkSubject->GetRole().GetDefaultObject()->GetDisplayName().ToString(), *SubjectStaticData.Role.GetDefaultObject()->GetDisplayName().ToString());

					// Make sure to keep the previous settings even if we're changing the role.
					SubjectSettings = TStrongObjectPtr{ Cast<ULiveLinkSubjectSettings>(GetSubjectSettings(SubjectItem->Key)) };
					Collection->RemoveSubject(SubjectStaticData.SubjectKey);
					LiveLinkSubject = nullptr;
				}
				else
				{
					LiveLinkSubject->ClearFrames();
				}
			}
		}
	}


	// Prevent GC while we're creating UObjects since LL Client can potentially be ticked outside of the game thread.
	FGCScopeGuard Guard;

	if (LiveLinkSubject)
	{
		// If new static data was received, make sure it's rebroadcast
		LiveLinkSubject->SetStaticDataAsRebroadcasted(false);
	}
	else
	{
		const ULiveLinkSettings* LiveLinkSettings = GetDefault<ULiveLinkSettings>();
		const FLiveLinkRoleProjectSetting DefaultSetting = LiveLinkSettings->GetDefaultSettingForRole(SubjectStaticData.Role.Get());

		{
			// By default, try to reuse the previous subject settings when possible. The role and interpolation will be replaced but the other settings (such as bRebroadcastSubject) will be kept.
			bool bRestoredFromPresetCache = false;
			if (!SubjectSettings)
			{
				// Check if we have cached preset settings for this subject.
				// This allows preprocessors and other settings to survive source disconnection/reconnection.
				if (TStrongObjectPtr<ULiveLinkSubjectSettings>* CachedSettings = CachedPresetSubjectSettings.Find(SubjectStaticData.SubjectKey.SubjectName))
				{
					FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(CachedSettings->Get(), GetTransientPackage());
					Parameters.bSkipPostLoad = true;
				
					// Cannot call ordinary DuplicateObject() because it calls PostLoad for legacy reasons,
					// which is not allowed anywhere except the GameThread.
					SubjectSettings = TStrongObjectPtr{ Cast<ULiveLinkSubjectSettings>(StaticDuplicateObjectEx(Parameters)) };

					// PATCH until UE-250259 bug is fixed
					SubjectSettings->ClearFlags(EObjectFlags::RF_NeedPostLoad | EObjectFlags::RF_NeedPostLoadSubobjects);

					bRestoredFromPresetCache = true;
				}
				else
				{
					// Setting class should always be valid

					UClass* SettingClass = DefaultSetting.SettingClass.Get();

					if (SettingClass == nullptr)
					{
						SettingClass = ULiveLinkSubjectSettings::StaticClass();
					}

					SubjectSettings = TStrongObjectPtr{ NewObject<ULiveLinkSubjectSettings>(GetTransientPackage(), SettingClass) };
				}
			}

			SubjectSettings->Initialize(SubjectStaticData.SubjectKey);

			SubjectSettings->Role = SubjectStaticData.Role;

			if (FString* OriginalSourceName = SubjectStaticData.ExtraMetadata.Find(FLiveLinkMessageAnnotation::OriginalSourceAnnotation))
			{
				SubjectSettings->OriginalSourceName = **OriginalSourceName;
			}

			// Skip default interpolation/preprocessor setup if we restored settings from a preset cache,
			// since the cached settings already have the correct processors configured.
			if (!bRestoredFromPresetCache)
			{
				UClass* FrameInterpolationProcessorClass = DefaultSetting.FrameInterpolationProcessor.Get();
				if (FrameInterpolationProcessorClass != nullptr)
				{
					UClass* InterpolationRole = FrameInterpolationProcessorClass->GetDefaultObject<ULiveLinkFrameInterpolationProcessor>()->GetRole();
					if (SubjectStaticData.Role->IsChildOf(InterpolationRole))
					{
						SubjectSettings->InterpolationProcessor = NewObject<ULiveLinkFrameInterpolationProcessor>(SubjectSettings.Get(), FrameInterpolationProcessorClass);
						// Clear async flag since this might've been created outside the game thread.
						SubjectSettings->InterpolationProcessor->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
					}
					else
					{
						FLiveLinkLog::Warning(TEXT("The interpolator '%s' is not valid for the Role '%s'"), *FrameInterpolationProcessorClass->GetName(), *SubjectStaticData.Role->GetName());
					}
				}
				else
				{
					// If no settings were found for a specific role, check if the default interpolator is compatible with the role
					UClass* FallbackInterpolationProcessorClass = LiveLinkSettings->FrameInterpolationProcessor.Get();
					if (FallbackInterpolationProcessorClass != nullptr)
					{
						UClass* InterpolationRole = FallbackInterpolationProcessorClass->GetDefaultObject<ULiveLinkFrameInterpolationProcessor>()->GetRole();
						if (SubjectStaticData.Role->IsChildOf(InterpolationRole))
						{
							SubjectSettings->InterpolationProcessor = NewObject<ULiveLinkFrameInterpolationProcessor>(SubjectSettings.Get(), FallbackInterpolationProcessorClass);
							// Clear async flag since this might've been created outside the game thread.
							SubjectSettings->InterpolationProcessor->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
						}
					}
				}

				for (TSubclassOf<ULiveLinkFramePreProcessor> PreProcessor : DefaultSetting.FramePreProcessors)
				{
					if (PreProcessor != nullptr)
					{
						UClass* PreProcessorRole = PreProcessor->GetDefaultObject<ULiveLinkFramePreProcessor>()->GetRole();
						if (SubjectStaticData.Role->IsChildOf(PreProcessorRole))
						{
							TObjectPtr<ULiveLinkFramePreProcessor>& FramePreprocessor = SubjectSettings->PreProcessors.Add_GetRef(NewObject<ULiveLinkFramePreProcessor>(SubjectSettings.Get(), PreProcessor.Get()));
							// Clear async flag since this might've been created outside the game thread.
							FramePreprocessor->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
						}
						else
						{
							FLiveLinkLog::Warning(TEXT("The pre processor '%s' is not valid for the Role '%s'"), *PreProcessor->GetName(), *SubjectStaticData.Role->GetName());
						}
					}
				}
			}
		}

		bool bEnabled = Collection->FindEnabledSubject(SubjectStaticData.SubjectKey.SubjectName) == nullptr;
		FLiveLinkCollectionSubjectItem CollectionSubjectItem(SubjectStaticData.SubjectKey, MakeUnique<FLiveLinkSubject>(SourceItem->TimedData), SubjectSettings.Get(), bEnabled);
		CollectionSubjectItem.GetLiveSubject()->Initialize(SubjectStaticData.SubjectKey, SubjectStaticData.Role.Get(), this);

		// Clear the async flag since we've passed  the SubjectSettings to the subject collection item.
		SubjectSettings->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);

		LiveLinkSubject = CollectionSubjectItem.GetLiveSubject();

		Collection->AddSubject(MoveTemp(CollectionSubjectItem));
	}

	const FLiveLinkStaticDataStruct* UnmappedStaticData = &SubjectStaticData.StaticData;
	
	if (LiveLinkSubject)
	{
		if (ULiveLinkSubjectRemapper::FWorkerSharedPtr Remapper = LiveLinkSubject->GetFrameRemapper())
		{
			// ATM we will assume vsubjects can't have remappers.
			if (ULiveLinkSubjectSettings* Settings = Cast<ULiveLinkSubjectSettings>(GetSubjectSettings(SubjectStaticData.SubjectKey)))
			{
				// Make sure we have a valid settings object to not remap the static data while we're resetting the remapper.
				if (Settings->Remapper)
				{
					// Make sure to rebroadcast the new static data.
					LiveLinkSubject->SetStaticDataAsRebroadcasted(false);
				}
			}
			UnmappedStaticData = &LiveLinkSubject->GetStaticData(/*bGetOverrideData*/ false);
		}

		if (const FSubjectFramesAddedHandles* Handles = SubjectFrameAddedHandles.Find(SubjectStaticData.SubjectKey.SubjectName))
		{
			Handles->OnStaticDataAdded.Broadcast(SubjectStaticData.SubjectKey, SubjectStaticData.Role, SubjectStaticData.StaticData);
			Handles->OnUnmappedStaticDataAdded.Broadcast(SubjectStaticData.SubjectKey, SubjectStaticData.Role, *UnmappedStaticData);
		}
		else if (const FSubjectFramesAddedHandles* AllSubjectsHandler = SubjectFrameAddedHandles.Find(UE::LiveLink::Private::ALL_SUBJECTS_DELEGATE_TOKEN))
		{
			AllSubjectsHandler->OnStaticDataAdded.Broadcast(SubjectStaticData.SubjectKey, SubjectStaticData.Role, SubjectStaticData.StaticData);
			AllSubjectsHandler->OnUnmappedStaticDataAdded.Broadcast(SubjectStaticData.SubjectKey, SubjectStaticData.Role, *UnmappedStaticData);
		}

		LiveLinkSubject->SetStaticData(SubjectStaticData.Role, MoveTemp(SubjectStaticData.StaticData));
	}
}


void FLiveLinkClient::BroadcastFrameDataUpdate(const FLiveLinkSubjectKey& InSubjectKey, const FLiveLinkFrameDataStruct& InFrameData)
{
	FScopeLock BroadcastLock(&SubjectFrameReceivedHandleseCriticalSection);
	if (const FSubjectFramesReceivedHandles* Handles = SubjectFrameReceivedHandles.Find(InSubjectKey))
	{
		Handles->OnFrameDataReceived.Broadcast(InFrameData);
	}

	if (const FSubjectFramesReceivedHandles* Handles = SubjectFrameReceivedHandles.Find(FLiveLinkSubjectKey{ FGuid(), UE::LiveLink::Private::ALL_SUBJECTS_DELEGATE_TOKEN}))
	{
		Handles->OnFrameDataReceived.Broadcast(InFrameData);
	}
}

TSharedPtr<ILiveLinkProvider> FLiveLinkClient::GetRebroadcastLiveLinkProvider() const
{
	return ILiveLinkProvider::CreateLiveLinkProvider(RebroadcastLiveLinkProviderName);
}

void FLiveLinkClient::PushSubjectFrameData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, FLiveLinkFrameDataStruct&& InFrameData)
{
	FPendingSubjectFrame SubjectFrame{ InSubjectKey, MoveTemp(InFrameData) };
	const int32 MaxNumBufferToCached = CVarMaxNewFrameDataPerUpdate.GetValueOnAnyThread();
	bool bLogError = false;

	bool bCanPushFrame = true;

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FLiveLinkClient::PushSubjectFrameData_AnyThread %s"), *InSubjectKey.SubjectName.Name.ToString()));

	// Assign FrameId using uint32 wraparound arithmetic.
	if (FLiveLinkBaseFrameData* BaseData = SubjectFrame.FrameData.GetBaseData())
	{
		if (BaseData->FrameId == INDEX_NONE)
		{
			FScopeLock Lock(&SubjectFrameCountersLock);
			FLiveLinkFrameIdentifier& Counter = SubjectFrameCounters.FindOrAdd(InSubjectKey, 0);
			BaseData->FrameId = Counter;
			Counter = int32(uint32(Counter) + 1u);
		}
	}

	{
		FScopeLock Lock(&PendingFramesCriticalSection);

		if (SubjectFrameToPush.Num() > MaxNumBufferToCached) // Something is wrong somewhere. Warn the user and discard the new Frame Data.
		{
			bLogError = true;
			SubjectFrameToPush.RemoveAt(0, SubjectFrameToPush.Num() - MaxNumBufferToCached, EAllowShrinking::No);
			bCanPushFrame = false;
		}
	}

	if (bCanPushFrame)
	{
		BroadcastFrameDataUpdate(InSubjectKey, SubjectFrame.FrameData);
			
		// Since the lock was released between setting bCanPushFrame and adding to the array, it is possible that
		// we exceed MaxNumBufferToCached. But this should be rare and also harmless.
		// The lock was released so that OnFrameDataReceived doesn't need to be called with the lock on,
		// which can hang the game thread when it calls EvaluateFrame if the broadcast takes longer than usual.

		FScopeLock Lock(&PendingFramesCriticalSection);
		SubjectFrameToPush.Add(MoveTemp(SubjectFrame));
	}

	if (bLogError)
	{
		static const FName NAME_TooManyFrame = "LiveLinkClient_TooManyFrame";
		FLiveLinkLog::InfoOnce(NAME_TooManyFrame, FLiveLinkSubjectKey(), TEXT("Trying to add more than %d frames in the same frame. Oldest frames will be discarded."), MaxNumBufferToCached);
	}
}

void FLiveLinkClient::PushSubjectFrameData_Internal(FPendingSubjectFrame&& SubjectFrameData)
{
	SCOPE_CYCLE_COUNTER(STAT_LiveLink_PushFrameData);

	check(Collection);

	FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(SubjectFrameData.SubjectKey.Source);
	if (SourceItem == nullptr || SourceItem->bPendingKill)
	{
		return;
	}

	//To add a frame data, we need to find our subject but also have a static data associated to it.
	//With presets, the subject could exist but no static data received yet.
	FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(SubjectFrameData.SubjectKey);
	if (SubjectItem == nullptr)
	{
		return;
	}

	if (!SubjectItem->bEnabled || SubjectItem->bPendingKill)
	{
		return;
	}

	FLiveLinkSubject* LinkSubject = SubjectItem->GetLiveSubject();
	if (LinkSubject == nullptr)
	{
		FLiveLinkLog::Error(TEXT("The Subject is not allowed to push to a virtual subject."));
		return;
	}


	if (!LinkSubject->HasStaticData())
	{
		return;
	}

	const TSubclassOf<ULiveLinkRole> Role = LinkSubject->GetRole();
	if (Role == nullptr)
	{
		return;
	}

	FLiveLinkFrameDataStruct UnmappedFrameData;
	
	if (ULiveLinkSubjectRemapper::FWorkerSharedPtr Remapper = LinkSubject->GetFrameRemapper())
	{
		UnmappedFrameData.InitializeWith(SubjectFrameData.FrameData);
		Remapper->RemapFrameData(LinkSubject->GetStaticData(), SubjectFrameData.FrameData);
	}

	bool bShouldLogWarning = true;
	if (!Role.GetDefaultObject()->IsFrameDataValid(LinkSubject->GetStaticData(), SubjectFrameData.FrameData, bShouldLogWarning))
	{
		if (bShouldLogWarning)
		{
			static const FName NAME_InvalidFrameData = "LiveLinkClient_InvalidFrameData";
			FLiveLinkLog::ErrorOnce(NAME_InvalidFrameData, SubjectFrameData.SubjectKey, TEXT("Trying to add frame data that is not formatted properly to role '%s' with subject '%s'."), *Role->GetName(), *SubjectFrameData.SubjectKey.SubjectName.ToString());
		}
		return;
	}

	if (UnmappedFrameData.IsValid() && !Role.GetDefaultObject()->IsFrameDataValid(LinkSubject->GetStaticData(/*bGetOverrideData*/ false), UnmappedFrameData, bShouldLogWarning))
	{
		if (bShouldLogWarning)
		{
			static const FName NAME_InvalidFrameData = "LiveLinkClient_InvalidUnmappedFrameData";
			FLiveLinkLog::ErrorOnce(NAME_InvalidFrameData, SubjectFrameData.SubjectKey, TEXT("Trying to add unmapped frame data that is not formatted properly to role '%s' with subject '%s'."), *Role->GetName(), *SubjectFrameData.SubjectKey.SubjectName.ToString());
		}
		return;
	}

	//Stamp arrival time of each packet to track clock difference when it is effectively added to the stash.
	//Doing it in the Add_AnyThread would mean that we stamp it up to 1 frame time behind, causing the offset to always be 1 frame behind
	//and requiring 2.5 frames or so to have a valid smooth offset
	if (SubjectFrameData.FrameData.GetBaseData())
	{
		SubjectFrameData.FrameData.GetBaseData()->ArrivalTime.WorldTime = FPlatformTime::Seconds();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Have to use the unsafe method since we're calling this outside the game thread.
		FAppTime Time = FAppTime::Get_UnsafeNoCheck();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const TOptional<FQualifiedFrameTime>& CurrentTime = Time.CurrentFrameTime;
		if (CurrentTime.IsSet())
		{
			SubjectFrameData.FrameData.GetBaseData()->ArrivalTime.SceneTime = *CurrentTime;
			if (UnmappedFrameData.IsValid() && UnmappedFrameData.GetBaseData())
			{
				UnmappedFrameData.GetBaseData()->ArrivalTime.SceneTime = *CurrentTime;
			}
		}
	}

	//Let source data know about this new frame to get latest clock offset
	SourceItem->TimedData->ProcessNewFrameTimingInfo(*SubjectFrameData.FrameData.GetBaseData());

	if (const FSubjectFramesAddedHandles* Handles = SubjectFrameAddedHandles.Find(SubjectFrameData.SubjectKey.SubjectName))
	{
		Handles->OnFrameDataAdded.Broadcast(SubjectItem->Key, Role, SubjectFrameData.FrameData);
		Handles->OnUnmappedFrameDataAdded.Broadcast(SubjectItem->Key, Role, UnmappedFrameData.IsValid() ? UnmappedFrameData : SubjectFrameData.FrameData);
	}
	else if (const FSubjectFramesAddedHandles* AllSubjectsHandler = SubjectFrameAddedHandles.Find(UE::LiveLink::Private::ALL_SUBJECTS_DELEGATE_TOKEN))
	{
		// NAME_None means we registered for all subjects update.
		AllSubjectsHandler->OnFrameDataAdded.Broadcast(SubjectItem->Key, Role, SubjectFrameData.FrameData);
		AllSubjectsHandler->OnUnmappedFrameDataAdded.Broadcast(SubjectItem->Key, Role, UnmappedFrameData.IsValid() ? UnmappedFrameData : SubjectFrameData.FrameData);
	}

	const bool bHasParentSubject = SourceItem->Setting->ParentSubject != FLiveLinkSubjectName();
	if (!bHasParentSubject)
	{
		// If it's paused, rebroadcast will be handled in FLiveLinkClient::BuildThisTicksSubjectSnapshot
		if (!LinkSubject->IsPaused())
		{
			/** Only rebroadcast here if we're transmitted non-evaluated data. */
			if (!SourceItem->Setting->bTransmitEvaluatedData)
			{
				HandleSubjectRebroadcast(LinkSubject, SubjectFrameData.FrameData);
			}
		}

		if (bEnableParentSubjects)
		{
			Collection->ForEachSubject([this, &SubjectFrameData](const FLiveLinkCollectionSourceItem& SourceItem, const FLiveLinkCollectionSubjectItem& SubjectItem)
			{
				const FLiveLinkSubjectKey SubjectKey = SubjectFrameData.SubjectKey;

				// Handle virtual subjects synchronized rebroadcast.
				if (SourceItem.Setting->ParentSubject.Name == SubjectKey.SubjectName)
				{
					const FLiveLinkFrameDataStruct& FrameData = SubjectFrameData.FrameData;

					// todo: Time offset evaluation
					if (ILiveLinkSubject* LiveSubject = SubjectItem.GetLiveSubject())
					{
						FLiveLinkSubjectFrameData ChildData;

						if (SubjectItem.GetLiveSubject()->EvaluateFrameAtWorldTime(SubjectFrameData.FrameData.GetBaseData()->WorldTime.GetSourceTime(), SubjectItem.GetLinkSettings()->Role, ChildData))
						{
							const FTimecode FrameTC = FTimecode::FromFrameNumber(FrameData.GetBaseData()->MetaData.SceneTime.Time.GetFrame(), FrameData.GetBaseData()->MetaData.SceneTime.Rate);
							UE_LOGF(LogLiveLink, Verbose, "LiveLinkHub Parent (%ls) - Child '%ls' adding frame with Timecode:[%ls.%0.3f] - SourceTime: %0.4f, Offset: %0.6f, CorrectedTime: %0.4f", *SubjectKey.SubjectName.ToString(), *SubjectItem.Key.SubjectName.ToString(), *FrameTC.ToString(), FrameData.GetBaseData()->MetaData.SceneTime.Time.GetSubFrame(), FrameData.GetBaseData()->WorldTime.GetSourceTime(), FrameData.GetBaseData()->WorldTime.GetOffset(), FrameData.GetBaseData()->WorldTime.GetOffsettedTime());

							ChildData.FrameData.GetBaseData()->MetaData.SceneTime = FrameData.GetBaseData()->MetaData.SceneTime;
							ChildData.FrameData.GetBaseData()->MetaData.SceneTime.Rate = FrameData.GetBaseData()->MetaData.SceneTime.Rate;

							HandleSubjectRebroadcast(LiveSubject, ChildData.FrameData);
						}
						else
						{
							FLiveLinkLog::Warning(TEXT("Child subjects %s could not be evaluated for data resampling."), *SubjectKey.SubjectName.Name.ToString());
						}
					}
				}
			});
		}
	}

	//Finally, add the new frame to the subject. After this point, the frame data is unusable, it has been moved!
	LinkSubject->AddFrameData(MoveTemp(SubjectFrameData.FrameData));
}

bool FLiveLinkClient::CreateSubject(const FLiveLinkSubjectPreset& InSubjectPreset)
{
	check(Collection);

	if (InSubjectPreset.Role.Get() == nullptr || InSubjectPreset.Role.Get() == ULiveLinkRole::StaticClass())
	{
		FLiveLinkLog::Warning(TEXT("Create Subject Failure: The role is not defined."));
		return false;
	}

	if (InSubjectPreset.Key.Source == FLiveLinkSourceCollection::DefaultVirtualSubjectGuid && InSubjectPreset.VirtualSubject == nullptr)
	{
		FLiveLinkLog::Warning(TEXT("Create Source Failure: Can't create an empty virtual subject."));
		return false;
	}

	if (InSubjectPreset.Key.SubjectName.IsNone())
	{
		FLiveLinkLog::Warning(TEXT("Create Subject Failure: The subject name is invalid."));
		return false;
	}

	// Cache the preset settings so they can be restored if the subject is recreated later
	// (e.g., after source disconnection/reconnection).
	if (InSubjectPreset.Settings)
	{
		CachedPresetSubjectSettings.Add(InSubjectPreset.Key.SubjectName,
			TStrongObjectPtr{DuplicateObject<ULiveLinkSubjectSettings>(InSubjectPreset.Settings, GetTransientPackage())});
	}

	FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InSubjectPreset.Key.Source);
	if (SourceItem == nullptr || SourceItem->bPendingKill)
	{
		FLiveLinkLog::Warning(TEXT("Create Subject Failure: The source doesn't exist."));
		return false;
	}

	FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectPreset.Key);
	if (SubjectItem != nullptr)
	{
		if (SubjectItem->bPendingKill)
		{
			Collection->RemoveSubject(InSubjectPreset.Key);
		}
		else
		{
			FLiveLinkLog::Warning(TEXT("Create Subject Failure: The subject already exist."));
			return false;
		}
	}

	if (InSubjectPreset.VirtualSubject)
	{
		bool bEnabled = false;
		ULiveLinkVirtualSubject* VSubject = DuplicateObject<ULiveLinkVirtualSubject>(InSubjectPreset.VirtualSubject, GetTransientPackage());
		FLiveLinkCollectionSubjectItem VSubjectData(InSubjectPreset.Key, VSubject, bEnabled);
		VSubject->Initialize(VSubjectData.Key, VSubject->GetRole(), this);

		Collection->AddSubject(MoveTemp(VSubjectData));
		Collection->SetSubjectEnabled(InSubjectPreset.Key, InSubjectPreset.bEnabled);
	}
	else
	{
		ULiveLinkSubjectSettings* SubjectSettings = nullptr;
		if (InSubjectPreset.Settings)
		{
			SubjectSettings = DuplicateObject<ULiveLinkSubjectSettings>(InSubjectPreset.Settings, GetTransientPackage());
		}
		else
		{
			SubjectSettings = NewObject<ULiveLinkSubjectSettings>();
		}

		SubjectSettings->Initialize(InSubjectPreset.Key);

		bool bEnabled = false;

		FLiveLinkCollectionSubjectItem CollectionSubjectItem(InSubjectPreset.Key, MakeUnique<FLiveLinkSubject>(SourceItem->TimedData), SubjectSettings, bEnabled);
		CollectionSubjectItem.GetLiveSubject()->Initialize(InSubjectPreset.Key, InSubjectPreset.Role.Get(), this);

		Collection->AddSubject(MoveTemp(CollectionSubjectItem));
		Collection->SetSubjectEnabled(InSubjectPreset.Key, InSubjectPreset.bEnabled);
	}
	return true;
}

void FLiveLinkClient::RemoveSubject_AnyThread(const FLiveLinkSubjectKey& InSubjectKey)
{
	if (Collection)
	{
		if (FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
		{
			SubjectItem->bPendingKill = true;
		}
	}
}

void FLiveLinkClient::PauseSubject_AnyThread(FLiveLinkSubjectName SubjectName)
{
	if (Collection)
	{
		if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(SubjectName))
		{
			bool bPauseResult = false;

			if (!IsSubjectValid(SubjectName))
			{
				UE_LOGF(LogLiveLink, Warning, "Could not pause subject %ls since it's not in a valid state.", *SubjectName.ToString());
				return;
			}

			if (FLiveLinkSubject* LiveSubject = SubjectItem->GetLiveSubject())
			{
				LiveSubject->PauseSubject();
			}
			else if (ULiveLinkVirtualSubject* VirtualSubject = SubjectItem->GetVirtualSubject())
			{
				VirtualSubject->PauseSubject();
			}

			OnLiveLinkSubjectStateChanged().Broadcast(SubjectItem->Key, ELiveLinkSubjectState::Paused);
		}
	}
}

void FLiveLinkClient::UnpauseSubject_AnyThread(FLiveLinkSubjectName SubjectName)
{
	if (Collection)
	{
		if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(SubjectName))
		{
			if (FLiveLinkSubject* LiveSubject = SubjectItem->GetLiveSubject())
			{
				LiveSubject->UnpauseSubject();
				OnLiveLinkSubjectStateChanged().Broadcast(SubjectItem->Key, LiveSubject->State);
			}
			else if (ULiveLinkVirtualSubject* VirtualSubject = SubjectItem->GetVirtualSubject())
			{
				VirtualSubject->UnpauseSubject();
				ELiveLinkSubjectState State = ELiveLinkSubjectState::Connected;
				if (!VirtualSubject->HasValidFrameSnapshot())
				{
					State = ELiveLinkSubjectState::InvalidOrDisabled;
				}

				OnLiveLinkSubjectStateChanged().Broadcast(SubjectItem->Key, State);
			}
		}
	}
}

bool FLiveLinkClient::AddVirtualSubject(const FLiveLinkSubjectKey& InVirtualSubjectKey, TSubclassOf<ULiveLinkVirtualSubject> InVirtualSubjectClass)
{
	bool bResult = false;

	if (Collection && !InVirtualSubjectKey.SubjectName.IsNone() && InVirtualSubjectClass != nullptr)
	{
		FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InVirtualSubjectKey.Source);
		if (SourceItem == nullptr || SourceItem->bPendingKill)
		{
			FLiveLinkLog::Warning(TEXT("Create Virtual Subject Failure: The source doesn't exist."));
		}
		else
		{

			const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InVirtualSubjectKey);
			const bool bFoundVirtualSubject = SubjectItem && SubjectItem->GetVirtualSubject();

			if (!bFoundVirtualSubject)
			{
				ULiveLinkVirtualSubject* VSubject = NewObject<ULiveLinkVirtualSubject>(GetTransientPackage(), InVirtualSubjectClass.Get());
				const bool bDoEnableSubject = Collection->FindEnabledSubject(InVirtualSubjectKey.SubjectName) == nullptr;
				FLiveLinkCollectionSubjectItem VSubjectData(InVirtualSubjectKey, VSubject, bDoEnableSubject);

				VSubject->Initialize(VSubjectData.Key, VSubject->GetRole(), this);

#if WITH_EDITOR
				// Add a callback to reinitialize the blueprint virtual subject if it is compiled
				if (ULiveLinkBlueprintVirtualSubject* BlueprintVirtualSubject = Cast<ULiveLinkBlueprintVirtualSubject>(VSubject))
				{
					UBlueprint* Blueprint = Cast<UBlueprint>(BlueprintVirtualSubject->GetClass()->ClassGeneratedBy);
					if (Blueprint)
					{
						Blueprint->OnCompiled().AddLambda([this, SubjectKey = VSubjectData.Key](UBlueprint* BP) {
							this->ReinitializeVirtualSubject(SubjectKey);
							});
					}
				}
#endif

				Collection->AddSubject(MoveTemp(VSubjectData));

				bResult = true;
			}
			else
			{
				FLiveLinkLog::Warning(TEXT("The virtual subject '%s' could not be created."), *InVirtualSubjectKey.SubjectName.Name.ToString());
			}
		}
	}

	return bResult;
}

void FLiveLinkClient::RemoveVirtualSubject(const FLiveLinkSubjectKey& InVirtualSubjectKey)
{
	if (Collection)
	{
		Collection->RemoveSubject(InVirtualSubjectKey);
	}
}

void FLiveLinkClient::ClearSubjectsFrames_AnyThread(FLiveLinkSubjectName InSubjectName)
{
	// Use the subject enabled for at this frame
	if (FLiveLinkSubjectKey* SubjectKey = EnabledSubjects.Find(InSubjectName))
	{
		ClearSubjectsFrames_AnyThread(*SubjectKey);
	}
}

void FLiveLinkClient::ClearSubjectsFrames_AnyThread(const FLiveLinkSubjectKey& InSubjectKey)
{
	if (Collection)
	{
		Collection->ApplyToSubject(InSubjectKey, [](FLiveLinkCollectionSubjectItem& SubjectItem) 
		{
			SubjectItem.GetSubject()->ClearFrames();
		});
	}
}

void FLiveLinkClient::ClearAllSubjectsFrames_AnyThread()
{
	if (Collection)
	{
		Collection->ForEachSubject([](FLiveLinkCollectionSourceItem& SourceItem, FLiveLinkCollectionSubjectItem& SubjectItem)
		{
			SubjectItem.GetSubject()->ClearFrames();
		});
	}
}

#if WITH_EDITOR
void FLiveLinkClient::ReinitializeVirtualSubject(const FLiveLinkSubjectKey& SubjectKey)
{
	if (Collection)
	{
		if (FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(SubjectKey))
		{
			if (ULiveLinkVirtualSubject* VSubject = SubjectItem->GetVirtualSubject())
			{
				VSubject->Initialize(SubjectKey, VSubject->GetRole(), this);
			}
		}
	}
}
#endif

FLiveLinkSubjectPreset FLiveLinkClient::GetSubjectPreset(const FLiveLinkSubjectKey& InSubjectKey, UObject* InDuplicatedObjectOuter) const
{
	UObject* DuplicatedObjectOuter = InDuplicatedObjectOuter ? InDuplicatedObjectOuter : GetTransientPackage();

	FLiveLinkSubjectPreset SubjectPreset;

	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		SubjectPreset.Key = SubjectItem->Key;
		SubjectPreset.Role = SubjectItem->GetSubject()->GetRole();
		SubjectPreset.bEnabled = SubjectItem->bEnabled;
		if (SubjectItem->GetVirtualSubject() != nullptr)
		{
			SubjectPreset.VirtualSubject = DuplicateObject<ULiveLinkVirtualSubject>(SubjectItem->GetVirtualSubject(), DuplicatedObjectOuter);
		}
		else
		{
			SubjectPreset.Settings = DuplicateObject<ULiveLinkSubjectSettings>(SubjectItem->GetLinkSettings(), DuplicatedObjectOuter);
		}
	}


	return SubjectPreset;
}

TArray<FLiveLinkSubjectKey> FLiveLinkClient::GetSubjects(bool bIncludeDisabledSubject, bool bIncludeVirtualSubject) const
{
	TArray<FLiveLinkSubjectKey> SubjectEntries;

	SubjectEntries.Reserve(Collection->NumSubjects());

	Collection->ForEachSubject([this, bIncludeDisabledSubject, bIncludeVirtualSubject, &SubjectEntries](const FLiveLinkCollectionSourceItem& SourceItem, const FLiveLinkCollectionSubjectItem& SubjectItem)
	{
		if ((SubjectItem.bEnabled || bIncludeDisabledSubject) && (bIncludeVirtualSubject || SubjectItem.GetVirtualSubject() == nullptr))
		{
			SubjectEntries.Add(SubjectItem.Key);
		}
	});

	return SubjectEntries;
}

bool FLiveLinkClient::IsSubjectValid(const FLiveLinkSubjectKey& InSubjectKey) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		if (SubjectItem->GetSubject()->HasValidFrameSnapshot())
		{
			if (ULiveLinkVirtualSubject* VSubject = SubjectItem->GetVirtualSubject())
			{
				return VSubject->GetState() == ELiveLinkSubjectState::Connected;
			}

			if (FLiveLinkSubject* LiveSubject = SubjectItem->GetLiveSubject())
			{
				return LiveSubject->GetState() == ETimedDataInputState::Connected;
			}
		}
	}
	return false;
}

bool FLiveLinkClient::IsSubjectValid(FLiveLinkSubjectName InSubjectName) const
{
	if (const FLiveLinkSubjectKey* FoundSubjectKey = EnabledSubjects.Find(InSubjectName))
	{
		return IsSubjectValid(*FoundSubjectKey);
	}
	return false;
}

bool FLiveLinkClient::IsSubjectEnabled(const FLiveLinkSubjectKey& InSubjectKey, bool bForThisFrame) const
{
	if (bForThisFrame)
	{
		if (const FLiveLinkSubjectKey* FoundSubjectKey = EnabledSubjects.Find(InSubjectKey.SubjectName))
		{
			return *FoundSubjectKey == InSubjectKey;
		}
		return false;
	}

	return Collection->IsSubjectEnabled(InSubjectKey);
}

bool FLiveLinkClient::IsSubjectEnabled(FLiveLinkSubjectName InSubjectName) const
{
	return EnabledSubjects.Find(InSubjectName) != nullptr;
}

void FLiveLinkClient::SetSubjectEnabled(const FLiveLinkSubjectKey& InSubjectKey, bool bInEnabled)
{
	Collection->SetSubjectEnabled(InSubjectKey, bInEnabled);
}

void FLiveLinkClient::SetSubjectEnabled(FLiveLinkSubjectName SubjectName, bool bInEnabled)
{
	if (const FLiveLinkSubjectKey* FoundSubjectKey = EnabledSubjects.Find(SubjectName))
	{
		Collection->SetSubjectEnabled(*FoundSubjectKey, bInEnabled);
	}
}

bool FLiveLinkClient::IsSubjectTimeSynchronized(const FLiveLinkSubjectKey& InSubjectKey) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		if (FLiveLinkSubject* LinkSubject = SubjectItem->GetLiveSubject())
		{
			return LinkSubject->IsTimeSynchronized();
		}
	}
	return false;
}

bool FLiveLinkClient::IsSubjectTimeSynchronized(FLiveLinkSubjectName InSubjectName) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindEnabledSubject(InSubjectName))
	{
		if (FLiveLinkSubject* LinkSubject = SubjectItem->GetLiveSubject())
		{
			return LinkSubject->IsTimeSynchronized();
		}
	}
	return false;
}

TSubclassOf<ULiveLinkRole> FLiveLinkClient::GetSubjectRole_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		return SubjectItem->GetSubject()->GetRole();
	}

	return TSubclassOf<ULiveLinkRole>();
}

TSubclassOf<ULiveLinkRole> FLiveLinkClient::GetSubjectRole_AnyThread(FLiveLinkSubjectName InSubjectName) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindEnabledSubject(InSubjectName))
	{
		return SubjectItem->GetSubject()->GetRole();
	}

	return TSubclassOf<ULiveLinkRole>();
}

TSubclassOf<ULiveLinkRole> FLiveLinkClient::GetSubjectTranslatedRole_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) const
{
	if (GetDefault<ULiveLinkSettings>()->bTranslateRebroadcastFrames)
	{
		if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
		{
			if (FLiveLinkSubject* Subject = SubjectItem->GetLiveSubject())
			{
				TArray<ULiveLinkFrameTranslator::FWorkerSharedPtr> Translators = Subject->GetFrameTranslators();
				if (Translators.Num() && Translators[0])
				{
					return Translators[0]->GetToRole();
				}
			}
		}
	}

	return nullptr;
}

bool FLiveLinkClient::DoesSubjectSupportsRole_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, TSubclassOf<ULiveLinkRole> InSupportedRole) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		return SubjectItem->GetSubject()->SupportsRole(InSupportedRole);
	}

	return false;
}

bool FLiveLinkClient::DoesSubjectSupportsRole_AnyThread(FLiveLinkSubjectName InSubjectName, TSubclassOf<ULiveLinkRole> InSupportedRole) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindEnabledSubject(InSubjectName))
	{
		return SubjectItem->GetSubject()->SupportsRole(InSupportedRole);
	}

	return false;
}

TArray<FLiveLinkTime> FLiveLinkClient::GetSubjectFrameTimes(const FLiveLinkSubjectKey& InSubjectKey) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		return SubjectItem->GetSubject()->GetFrameTimes();
	}
	return TArray<FLiveLinkTime>();
}

TArray<FLiveLinkTime> FLiveLinkClient::GetSubjectFrameTimes(FLiveLinkSubjectName InSubjectName) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindEnabledSubject(InSubjectName))
	{
		return SubjectItem->GetSubject()->GetFrameTimes();
	}

	return TArray<FLiveLinkTime>();
}

FText FLiveLinkClient::GetSourceNameOverride(const FLiveLinkSubjectKey& SubjectKey) const
{
	FText SourceType = GetSourceType(SubjectKey.Source);
	FText SourceNameOverride = SourceType;

	ULiveLinkSubjectSettings* SubjectSettings = Cast<ULiveLinkSubjectSettings>(GetSubjectSettings(SubjectKey));
	ULiveLinkSourceSettings* SourceSettings = GetSourceSettings(SubjectKey.Source);

	if (SubjectSettings && SourceSettings)
	{
		SourceNameOverride = SourceSettings->GetSourceNameOverride(SubjectSettings, SourceType);
	}

	return SourceNameOverride;
}

FText FLiveLinkClient::GetSubjectDisplayName(const FLiveLinkSubjectKey& InSubjectKey) const
{
	FText DisplayName;
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		UObject* Settings = SubjectItem->GetSettings();
		if (ULiveLinkSubjectSettings* SubjectSettings = Cast<ULiveLinkSubjectSettings>(Settings))
		{
			DisplayName = SubjectSettings->GetDisplayName();
		}
		else if (ULiveLinkVirtualSubject* VirtualSubject = Cast<ULiveLinkVirtualSubject>(Settings))
		{
			DisplayName = VirtualSubject->GetDisplayName();
		}
	}

	return DisplayName;
}

TArray<FLiveLinkSubjectKey> FLiveLinkClient::GetSubjectsSupportingRole(TSubclassOf<ULiveLinkRole> InSupportedRole, bool bIncludeDisabledSubject, bool bIncludeVirtualSubject) const
{
	TArray<FLiveLinkSubjectKey> SubjectKeys;

	Collection->ForEachSubject([this, &SubjectKeys, InSupportedRole, bIncludeDisabledSubject, bIncludeVirtualSubject](const FLiveLinkCollectionSourceItem& SourceItem, const FLiveLinkCollectionSubjectItem& SubjectItem)
	{
		if (SubjectItem.GetSubject()->SupportsRole(InSupportedRole))
		{
			if ((SubjectItem.bEnabled || bIncludeDisabledSubject) && (bIncludeVirtualSubject || SubjectItem.GetVirtualSubject() == nullptr))
			{
				SubjectKeys.Add(SubjectItem.Key);
			}
		}
	});

	return SubjectKeys;
}

bool FLiveLinkClient::EvaluateFrameFromSource_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_LiveLink_EvaluateFrame);

	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		return SubjectItem->GetSubject()->EvaluateFrame(InDesiredRole, OutFrame);
	}

	return false;
}

//just call our tick
void FLiveLinkClient::ForceTick()
{
	Tick();
}

bool FLiveLinkClient::HasPendingSubjectFrames()
{
	FScopeLock PendingFramesLock(&PendingFramesCriticalSection);
	return !SubjectFrameToPush.IsEmpty();
}

void FLiveLinkClient::ClearOverrideStaticData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey)
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		SubjectItem->GetLiveSubject()->ClearOverrideStaticData_AnyThread();
	}
}

bool FLiveLinkClient::EvaluateFrame_AnyThread(FLiveLinkSubjectName InSubjectName, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_LiveLink_EvaluateFrame);

	bool bResult = false;

	// Used the cached enabled list
	if (FLiveLinkSubjectKey* FoundSubjectKey = EnabledSubjects.Find(InSubjectName))
	{
		if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(*FoundSubjectKey))
		{
			bResult = SubjectItem->GetSubject()->EvaluateFrame(InDesiredRole, OutFrame);
		}

#if WITH_EDITOR
		if (OnLiveLinkSubjectEvaluated().IsBound())
		{
			FLiveLinkTime RequestedTime = FLiveLinkTime(FPlatformTime::Seconds(), CachedEngineFrameTime.Get(FQualifiedFrameTime()));
			FLiveLinkTime ResultTime;
			if (bResult)
			{
				ResultTime = OutFrame.FrameData.GetBaseData()->GetLiveLinkTime();
			}
			OnLiveLinkSubjectEvaluated().Broadcast(*FoundSubjectKey, InDesiredRole, RequestedTime, bResult, ResultTime);
		}
#endif //WITH_EDITOR
	}
	else
	{
		UE_LOGF(LogLiveLink, Verbose, "Subject '%ls' is not enabled or doesn't exist", *InSubjectName.ToString());
	}

	return bResult;
}

bool FLiveLinkClient::EvaluateFrameAtWorldTime_AnyThread(FLiveLinkSubjectName InSubjectName, double InWorldTime, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_LiveLink_EvaluateFrame);

	bool bResult = false;

	// Used the cached enabled list
	if (FLiveLinkSubjectKey* FoundSubjectKey = EnabledSubjects.Find(InSubjectName))
	{
		if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(*FoundSubjectKey))
		{
			if (FLiveLinkSubject* LinkSubject = SubjectItem->GetLiveSubject())
			{
				bResult = LinkSubject->EvaluateFrameAtWorldTime(InWorldTime, InDesiredRole, OutFrame);
			}
			else
			{
				bResult = SubjectItem->GetSubject()->EvaluateFrame(InDesiredRole, OutFrame);
			}

#if WITH_EDITOR
			if (OnLiveLinkSubjectEvaluated().IsBound())
			{
				FLiveLinkTime RequestedTime = FLiveLinkTime(InWorldTime, FQualifiedFrameTime());
				FLiveLinkTime ResultTime;
				if (bResult)
				{
					ResultTime = OutFrame.FrameData.GetBaseData()->GetLiveLinkTime();
				}
				OnLiveLinkSubjectEvaluated().Broadcast(*FoundSubjectKey, InDesiredRole, RequestedTime, bResult, ResultTime);
			}
#endif //WITH_EDITOR
		}
	}
	else
	{
		UE_LOGF(LogLiveLink, Verbose, "Subject '%ls' is not enabled or doesn't exist", *InSubjectName.ToString());
	}

	return bResult;
}

bool FLiveLinkClient::EvaluateFrameAtSceneTime_AnyThread(FLiveLinkSubjectName InSubjectName, const FQualifiedFrameTime& InSceneTime, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_LiveLink_EvaluateFrame);



	bool bResult = false;

	// Used the cached enabled list
	if (FLiveLinkSubjectKey* FoundSubjectKey = EnabledSubjects.Find(InSubjectName))
	{
		if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(*FoundSubjectKey))
		{
			if (FLiveLinkSubject* LinkSubject = SubjectItem->GetLiveSubject())
			{
				bResult = LinkSubject->EvaluateFrameAtSceneTime(InSceneTime, InDesiredRole, OutFrame);
			}
			else
			{
				bResult = SubjectItem->GetSubject()->EvaluateFrame(InDesiredRole, OutFrame);
			}

#if WITH_EDITOR
			if (OnLiveLinkSubjectEvaluated().IsBound())
			{
				FLiveLinkTime RequestedTime = FLiveLinkTime(0.0, InSceneTime);
				FLiveLinkTime ResultTime;
				if (bResult)
				{
					ResultTime = OutFrame.FrameData.GetBaseData()->GetLiveLinkTime();
				}
				OnLiveLinkSubjectEvaluated().Broadcast(*FoundSubjectKey, InDesiredRole, RequestedTime, bResult, ResultTime);
			}
#endif //WITH_EDITOR
		}
	}
	else
	{
		UE_LOGF(LogLiveLink, Verbose, "Subject '%ls' is not enabled or doesn't exist", *InSubjectName.ToString());
	}

	return bResult;
}

FSimpleMulticastDelegate& FLiveLinkClient::OnLiveLinkTicked()
{
	return OnLiveLinkTickedDelegate;
}

TArray<FGuid> FLiveLinkClient::GetDisplayableSources(bool bIncludeVirtualSources) const
{
	TArray<FGuid> Results;

	Collection->ForEachSource([&Results, bIncludeVirtualSources](const FLiveLinkCollectionSourceItem& SourceItem)
	{
		if (SourceItem.Source->CanBeDisplayedInUI() || (bIncludeVirtualSources && SourceItem.IsVirtualSource()))
		{
			Results.Add(SourceItem.Guid);
		}
	});

	return Results;
}

FLiveLinkSubjectTimeSyncData FLiveLinkClient::GetTimeSyncData(FLiveLinkSubjectName InSubjectName)
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindEnabledSubject(InSubjectName))
	{
		if (FLiveLinkSubject* LinkSubject = SubjectItem->GetLiveSubject())
		{
			return LinkSubject->GetTimeSyncData();
		}
	}

	return FLiveLinkSubjectTimeSyncData();
}

FName FLiveLinkClient::GetRebroadcastName(const FLiveLinkSubjectKey& InSubjectKey) const
{
	FName RebroadcastName = InSubjectKey.SubjectName.Name;

	if (UObject* Settings = GetSubjectSettings(InSubjectKey))
	{
		if (ULiveLinkSubjectSettings* SubjectSettings = Cast<ULiveLinkSubjectSettings>(Settings))
		{
			RebroadcastName = SubjectSettings->GetRebroadcastName();
		}
		else if (ULiveLinkVirtualSubject* VSubject = Cast<ULiveLinkVirtualSubject>(Settings))
		{
			RebroadcastName = VSubject->GetRebroadcastName();
		}
	}

	return RebroadcastName;
}

void FLiveLinkClient::PushPendingSubject_AnyThread(FPendingSubjectStatic&& PendingSubject)
{
	const int32 MaxNumBufferToCached = CVarMaxNewStaticDataPerUpdate.GetValueOnAnyThread();
	bool bLogError = true;
	{
		FScopeLock Lock(&PendingFramesCriticalSection);
		if (SubjectStaticToPush.Num() <= MaxNumBufferToCached) 
		{
			bLogError = false;

			{
				FScopeLock BroadcastLock(&SubjectFrameReceivedHandleseCriticalSection);
				if (const FSubjectFramesReceivedHandles* Handles = SubjectFrameReceivedHandles.Find(PendingSubject.SubjectKey))
				{
					Handles->OnStaticDataReceived.Broadcast(PendingSubject.StaticData);
				}
			}
			SubjectStaticToPush.Add(MoveTemp(PendingSubject));
		}
	}

	if (bLogError)
	{
		// Something is wrong somewhere. Warn the user and discard the new Static Data.
		static const FName NAME_TooManyStatic = "LiveLinkClient_TooManyStatic";
		FLiveLinkLog::ErrorOnce(NAME_TooManyStatic, FLiveLinkSubjectKey(), TEXT("Trying to add more than %d static subjects in the same frame. New Subjects will be discarded."), MaxNumBufferToCached);
	}
}

FText FLiveLinkClient::GetSourceType(FGuid InEntryGuid) const
{
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		return SourceItem->Source->GetSourceType();
	}
	return FText(NSLOCTEXT("TempLocTextLiveLink", "InvalidSourceType", "Invalid Source Type"));
}

FText FLiveLinkClient::GetSourceMachineName(FGuid InEntryGuid) const
{
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		return SourceItem->Source->GetSourceMachineName();
	}
	return FText(NSLOCTEXT("TempLocTextLiveLink", "InvalidSourceMachineName", "Invalid Source Machine Name"));
}

FText FLiveLinkClient::GetSourceStatus(FGuid InEntryGuid) const
{
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		return SourceItem->Source->GetSourceStatus();
	}
	return FText(NSLOCTEXT("TempLocTextLiveLink", "InvalidSourceStatus", "Invalid Source Status"));
}

FText FLiveLinkClient::GetSourceToolTip(FGuid InEntryGuid) const
{
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		return SourceItem->Source->GetSourceToolTip();
	}
	return FText(NSLOCTEXT("TempLocTextLiveLink", "InvalidSourceToolTip", "Invalid Source ToolTip"));
}

bool FLiveLinkClient::IsSourceStillValid(FGuid InEntryGuid) const
{
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		return SourceItem->Source->IsSourceStillValid();
	}
	return false;
}

bool FLiveLinkClient::IsVirtualSubject(const FLiveLinkSubjectKey& InSubjectKey) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		return SubjectItem->GetVirtualSubject() != nullptr;
	}
	return false;
}

void FLiveLinkClient::OnPropertyChanged(FGuid InEntryGuid, const FPropertyChangedEvent& InPropertyChangedEvent)
{

	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		SourceItem->Source->OnSettingsChanged(SourceItem->Setting.Get(), InPropertyChangedEvent);
	}
}

ULiveLinkSourceSettings* FLiveLinkClient::GetSourceSettings(const FGuid& InEntryGuid) const
{
	if (const FLiveLinkCollectionSourceItem* SourceItem = Collection->FindSource(InEntryGuid))
	{
		return SourceItem->Setting.Get();
	}
	return nullptr;
}

UObject* FLiveLinkClient::GetSubjectSettings(const FLiveLinkSubjectKey& InSubjectKey) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		return SubjectItem->GetSettings();
	}
	return nullptr;
}	

const FLiveLinkStaticDataStruct* FLiveLinkClient::GetSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, bool bGetOverrideData) const
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindSubject(InSubjectKey))
	{
		if (FLiveLinkSubject* LiveLinkSubject = SubjectItem->GetLiveSubject())
		{
			return &LiveLinkSubject->GetStaticData(bGetOverrideData);
		}
	}

	return nullptr;
}

void FLiveLinkClient::RegisterForFrameDataReceived(const FLiveLinkSubjectKey& InSubjectKey, const FOnLiveLinkSubjectStaticDataReceived::FDelegate& OnStaticDataReceived_AnyThread, const FOnLiveLinkSubjectFrameDataReceived::FDelegate& OnFrameDataReceived_AnyThread, FDelegateHandle& OutStaticDataReceivedHandle, FDelegateHandle& OutFrameDataReceivedHandle)
{
	OutStaticDataReceivedHandle.Reset();
	OutFrameDataReceivedHandle.Reset();

	FScopeLock Lock(&SubjectFrameReceivedHandleseCriticalSection);

	FSubjectFramesReceivedHandles& Handles = SubjectFrameReceivedHandles.FindOrAdd(InSubjectKey);
	if (OnStaticDataReceived_AnyThread.IsBound())
	{
		OutStaticDataReceivedHandle = Handles.OnStaticDataReceived.Add(OnStaticDataReceived_AnyThread);
	}
	if (OnFrameDataReceived_AnyThread.IsBound())
	{
		OutFrameDataReceivedHandle = Handles.OnFrameDataReceived.Add(OnFrameDataReceived_AnyThread);
	}
}

void FLiveLinkClient::UnregisterForFrameDataReceived(const FLiveLinkSubjectKey& InSubjectKey, FDelegateHandle InStaticDataReceivedHandle, FDelegateHandle InFrameDataReceivedHandle)
{
	FScopeLock Lock(&SubjectFrameReceivedHandleseCriticalSection);

	if (FSubjectFramesReceivedHandles* Handles = SubjectFrameReceivedHandles.Find(InSubjectKey))
	{
		Handles->OnStaticDataReceived.Remove(InStaticDataReceivedHandle);
		Handles->OnFrameDataReceived.Remove(InFrameDataReceivedHandle);
	}
}

bool FLiveLinkClient::RegisterForSubjectFrames(FLiveLinkSubjectName InSubjectName, const FOnLiveLinkSubjectStaticDataAdded::FDelegate& InOnStaticDataAdded, const FOnLiveLinkSubjectFrameDataAdded::FDelegate& InOnFrameDataAdded, FDelegateHandle& OutStaticDataAddedHandle, FDelegateHandle& OutFrameDataAddedHandle, TSubclassOf<ULiveLinkRole>& OutSubjectRole, FLiveLinkStaticDataStruct* OutStaticData)
{
	if (const FLiveLinkCollectionSubjectItem* SubjectItem = Collection->FindEnabledSubject(InSubjectName))
	{
		//Register both delegates
		FSubjectFramesAddedHandles& Handles = SubjectFrameAddedHandles.FindOrAdd(InSubjectName);
		OutStaticDataAddedHandle = Handles.OnStaticDataAdded.Add(InOnStaticDataAdded);
		OutFrameDataAddedHandle = Handles.OnFrameDataAdded.Add(InOnFrameDataAdded);

		//Give back the current static data and role associated to the subject
		OutSubjectRole = SubjectItem->GetSubject()->GetRole();

		//Copy the current static data
		if (OutStaticData)
		{
			const FLiveLinkStaticDataStruct& CurrentStaticData = SubjectItem->GetSubject()->GetStaticData();
			if (CurrentStaticData.IsValid())
			{
				OutStaticData->InitializeWith(CurrentStaticData);
			}
			else
			{
				OutStaticData->Reset();
			}
		}

		return true;
	}

	return false;
}

void FLiveLinkClient::UnregisterSubjectFramesHandle(FLiveLinkSubjectName InSubjectName, FDelegateHandle InStaticDataReceivedHandle, FDelegateHandle InFrameDataReceivedHandle)
{
	if (FSubjectFramesAddedHandles* Handles = SubjectFrameAddedHandles.Find(InSubjectName))
	{
		Handles->OnStaticDataAdded.Remove(InStaticDataReceivedHandle);
		Handles->OnFrameDataAdded.Remove(InFrameDataReceivedHandle);
	}
}

FSimpleMulticastDelegate& FLiveLinkClient::OnLiveLinkSourcesChanged()
{
	return Collection->OnLiveLinkSourcesChanged();
}

FSimpleMulticastDelegate& FLiveLinkClient::OnLiveLinkSubjectsChanged()
{
	return Collection->OnLiveLinkSubjectsChanged();
}

FOnLiveLinkSourceChangedDelegate& FLiveLinkClient::OnLiveLinkSourceAdded()
{
	return Collection->OnLiveLinkSourceAdded();
}

FOnLiveLinkSourceChangedDelegate& FLiveLinkClient::OnLiveLinkSourceRemoved()
{
	return Collection->OnLiveLinkSourceRemoved();
}

FOnLiveLinkSubjectChangedDelegate& FLiveLinkClient::OnLiveLinkSubjectRemoved()
{
	return Collection->OnLiveLinkSubjectRemoved();
}

FOnLiveLinkSubjectChangedDelegate& FLiveLinkClient::OnLiveLinkSubjectAdded()
{
	return Collection->OnLiveLinkSubjectAdded();
}

FOnLiveLinkSubjectStateChanged& FLiveLinkClient::OnLiveLinkSubjectStateChanged()
{
	return Collection->OnLiveLinkSubjectStateChanged();
}

FOnLiveLinkSubjectEnabledDelegate& FLiveLinkClient::OnLiveLinkSubjectEnabledChanged()
{
	return Collection->OnLiveLinkSubjectEnabledChanged();
}

#if WITH_EDITOR
FOnLiveLinkSubjectEvaluated& FLiveLinkClient::OnLiveLinkSubjectEvaluated()
{
	return OnLiveLinkSubjectEvaluatedDelegate;
}
#endif

void FLiveLinkClient::UnregisterGlobalSubjectFramesDelegate(FDelegateHandle& InStaticDataAddedHandle, FDelegateHandle& InFrameDataAddedHandle, bool bUseUnmappedData)
{
	if (FSubjectFramesAddedHandles* Handles = SubjectFrameAddedHandles.Find(UE::LiveLink::Private::ALL_SUBJECTS_DELEGATE_TOKEN))
	{
		if (bUseUnmappedData)
		{
			Handles->OnUnmappedStaticDataAdded.Remove(InStaticDataAddedHandle);
			Handles->OnUnmappedFrameDataAdded.Remove(InFrameDataAddedHandle);
		}
		else
		{
			Handles->OnStaticDataAdded.Remove(InStaticDataAddedHandle);
			Handles->OnFrameDataAdded.Remove(InFrameDataAddedHandle);
		}
	}
}

bool FLiveLinkClient::RegisterGlobalSubjectFramesDelegate(const FOnLiveLinkSubjectStaticDataAdded::FDelegate& InOnStaticDataAdded,
	const FOnLiveLinkSubjectFrameDataAdded::FDelegate& InOnFrameDataAdded, FDelegateHandle& OutStaticDataAddedHandle,
	FDelegateHandle& OutFrameDataAddedHandle, bool bUseUnmappedData)
{
	FSubjectFramesAddedHandles& Handles = SubjectFrameAddedHandles.FindOrAdd(UE::LiveLink::Private::ALL_SUBJECTS_DELEGATE_TOKEN);
	OutStaticDataAddedHandle = bUseUnmappedData ? Handles.OnUnmappedStaticDataAdded.Add(InOnStaticDataAdded) : Handles.OnStaticDataAdded.Add(InOnStaticDataAdded);
	OutFrameDataAddedHandle = bUseUnmappedData ? Handles.OnUnmappedFrameDataAdded.Add(InOnFrameDataAdded) : Handles.OnFrameDataAdded.Add(InOnFrameDataAdded);

	return true;
}
