// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkClientReference.h"
#include "ILiveLinkClient.h"
#include "LiveLinkModule.h"
#include "Features/IModularFeatures.h"
#include "Misc/ConfigCacheIni.h"


ILiveLinkClient* FLiveLinkClientReference::GetClient() const
{
	// Compiling for LiveLinkHub should return the livelink client we registered as a modular feature.
	static const bool bUseModularClientReference = GConfig->GetBoolOrDefault(
		TEXT("LiveLink"), TEXT("bUseModularClientReference"), false, GEngineIni);

	if (bUseModularClientReference)
	{
		return &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	}
	else
	{
		return FLiveLinkModule::LiveLinkClient_AnyThread;
	}
}
