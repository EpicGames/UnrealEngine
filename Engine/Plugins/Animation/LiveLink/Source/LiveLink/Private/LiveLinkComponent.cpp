// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "Roles/LiveLinkAnimationBlueprintStructs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkComponent)

// Sets default values for this component's properties
ULiveLinkComponent::ULiveLinkComponent()
	: bIsDirty(false)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_PrePhysics;
	bTickInEditor = true;
}

void ULiveLinkComponent::OnRegister()
{
	bIsDirty = true;
	Super::OnRegister();
}


// Called every frame
void ULiveLinkComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	// If we have been recently registered then ensure all Skeletal Mesh Components on the actor run in editor
	if (bIsDirty)
	{
		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		GetOwner()->GetComponents(SkeletalMeshComponents);
		for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
		{
			SkeletalMeshComponent->SetUpdateAnimationInEditor(true);
		}
		bIsDirty = false;
	}
	
	if (OnLiveLinkUpdated.IsBound())
	{
		FEditorScriptExecutionGuard ScriptGuard;

		OnLiveLinkUpdated.Broadcast(DeltaTime);
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}

bool ULiveLinkComponent::HasLiveLinkClient()
{
	if (LiveLinkClient == nullptr)
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		}
	}

	return (LiveLinkClient != nullptr);
}
