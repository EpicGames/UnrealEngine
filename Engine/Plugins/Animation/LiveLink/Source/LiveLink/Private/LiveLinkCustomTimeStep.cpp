// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCustomTimeStep.h"

#include "Features/IModularFeatures.h"
#include "GenlockedCustomTimeStep.h"
#include "ILiveLinkClient.h"
#include "LiveLinkClient.h"
#include "LiveLinkTypes.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkCustomTimeStep)

namespace LiveLinkCustomTimeStepUtils
{
	static const FString NoSubjectSelected = TEXT("<NO_SUBJECT>");
}

bool ULiveLinkCustomTimeStep::Initialize(UEngine* InEngine)
{
	State = ECustomTimeStepSynchronizationState::Synchronizing;
	EventLiveLink->Reset();

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.OnModularFeatureRegistered().AddUObject(this, &ULiveLinkCustomTimeStep::OnLiveLinkClientRegistered);
	ModularFeatures.OnModularFeatureUnregistered().AddUObject(this, &ULiveLinkCustomTimeStep::OnLiveLinkClientUnregistered);

	InitLiveLinkClient();

	return true;
}

void ULiveLinkCustomTimeStep::Shutdown(UEngine* InEngine)
{
	UninitLiveLinkClient();
}

bool ULiveLinkCustomTimeStep::UpdateTimeStep(UEngine* InEngine)
{
	const bool bIsSequencer = bIsSequencerSourceAnyThread.load(std::memory_order_relaxed);
	const bool bSynchronized = (State == ECustomTimeStepSynchronizationState::Synchronized);

	// Skip genlock if not synchronized OR if subject is from sequencer
	if (!bSynchronized || bIsSequencer)
	{
		EventWaitForSync->Trigger(); // Prevents start/restart quasi-deadlock.

		return true; // means that the Engine's TimeStep should be performed.
	}

	// Update anythread variable shadows
	{
		bLockStepModeAnyThread = bLockStepMode;
		FrameRateDividerAnyThread = FrameRateDivider;
		TimeoutInSecondsAnyThread = TimeoutInSeconds;
	}

	UpdateApplicationLastTime(); // Copies "CurrentTime" (used during the previous frame) in "LastTime"
	
	const double StartPlatformTime = FPlatformTime::Seconds();

	if (!WaitForSync())
	{
		State = ECustomTimeStepSynchronizationState::Synchronizing;
		return true;
	}

	const double EndPlatformTime = FPlatformTime::Seconds();

	const double ElapsedTimeWaitingForSync = EndPlatformTime - StartPlatformTime;

	// Subtract the multiple of the sync counts that corresponds to the desired divided live link data frame rate
	// For example if there are 3 sync counts and the frame rate divider is 2, then we'd want to use up 2 of the 3
	// sync counts and leave the 3rd one for the next cycle.

	LastSyncCountDelta = SyncCount;

	if (FrameRateDivider > 1)
	{
		LastSyncCountDelta -= (LastSyncCountDelta % FrameRateDivider);
	}

	SyncCount -= LastSyncCountDelta;

	// We are using real elapsed time for the time before sync, which will ultimately be used to calculate idle time. 
	// These values won't make sense if the frame rate settings are not correct.
	const double TimeAfterSync = FApp::GetLastTime() + GetLastSyncCountDelta() * GetSyncRate().AsInterval();
	const double TimeBeforeSync = TimeAfterSync - ElapsedTimeWaitingForSync;

	UpdateAppTimes(TimeBeforeSync, TimeAfterSync);

	return false; // false means that the Engine's TimeStep should NOT be performed.
}

ECustomTimeStepSynchronizationState ULiveLinkCustomTimeStep::GetSynchronizationState() const
{
	return State;
}

FFrameRate ULiveLinkCustomTimeStep::GetFixedFrameRate() const
{
	FFrameRate Rate = LiveLinkDataRate;

	if (FrameRateDivider > 1)
	{
		Rate.Denominator *= FrameRateDivider;
	}

	return Rate;
}

FFrameRate ULiveLinkCustomTimeStep::GetSyncRate() const
{
	return LiveLinkDataRate;
}

FString ULiveLinkCustomTimeStep::GetDisplayName() const
{
	const FString PlainName = GetFName().GetPlainNameString();
	const FText FrameRate = GetFixedFrameRate().ToPrettyText();
	const int32 NameIndex = GetFName().GetNumber();
	const FString& SubjectName = !SubjectKey.SubjectName.IsNone() ? SubjectKey.SubjectName.ToString() : LiveLinkCustomTimeStepUtils::NoSubjectSelected;

	return FString::Printf(TEXT("%s %s - %s (%d)"), *PlainName, *FrameRate.ToString(), *SubjectName, NameIndex);
}

uint32 ULiveLinkCustomTimeStep::GetLastSyncCountDelta() const
{
	return LastSyncCountDelta;
}

bool ULiveLinkCustomTimeStep::IsLastSyncDataValid() const
{
	return true;
}

uint32 ULiveLinkCustomTimeStep::GetExpectedSyncCountDelta() const
{
	return UGenlockedCustomTimeStep::GetExpectedSyncCountDelta();
}

bool ULiveLinkCustomTimeStep::WaitForSync()
{
	EventWaitForSync->Trigger();

	TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkCustomTimeStep::EventLiveLink->Wait);
	return EventLiveLink->Wait(FTimespan(TimeoutInSeconds * ETimespan::TicksPerSecond));
}

FQualifiedFrameTime ULiveLinkCustomTimeStep::GetGenlockTriggerSceneTime() const
{
	FScopeLock Lock(&GenlockSceneTimeLock);
	return GenlockTriggerSceneTime;
}


void ULiveLinkCustomTimeStep::BeginDestroy()
{
	UninitLiveLinkClient();
	Super::BeginDestroy();
}

#if WITH_EDITOR
void ULiveLinkCustomTimeStep::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkCustomTimeStep, SubjectKey))
	{
		// If was already registered
		if (RegisterForFrameDataReceivedHandle.IsValid() && RegisteredSubjectKey != SubjectKey)
		{
			UnregisterLiveLinkSubject();
			RegisterLiveLinkSubject();
		}
		// If was waiting for the subject to be added to the client
		else if (LiveLinkClient && LiveLinkClient->GetSubjectSettings(SubjectKey))
		{
			OnLiveLinkSubjectAdded(SubjectKey);
		}
	}
}
#endif

void ULiveLinkCustomTimeStep::InitLiveLinkClient()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		LiveLinkClient->OnLiveLinkSubjectAdded().AddUObject(this, &ULiveLinkCustomTimeStep::OnLiveLinkSubjectAdded);
		LiveLinkClient->OnLiveLinkSubjectRemoved().AddUObject(this, &ULiveLinkCustomTimeStep::OnLiveLinkSubjectRemoved);

		// if the subject already exist
		if (LiveLinkClient->GetSubjectSettings(SubjectKey))
		{
			OnLiveLinkSubjectAdded(SubjectKey);
		}
	}
}


void ULiveLinkCustomTimeStep::UninitLiveLinkClient()
{
	if (LiveLinkClient)
	{
		UnregisterLiveLinkSubject();

		LiveLinkClient->OnLiveLinkSubjectAdded().RemoveAll(this);
		LiveLinkClient->OnLiveLinkSubjectRemoved().RemoveAll(this);
		LiveLinkClient = nullptr;
		State = ECustomTimeStepSynchronizationState::Closed;
	}
}


void ULiveLinkCustomTimeStep::RegisterLiveLinkSubject()
{
	RegisteredSubjectKey = SubjectKey;

	if (!LiveLinkClient)
	{
		return;
	}

	// Check if this is a sequencer source and cache the result

	FText SourceType = LiveLinkClient->GetSourceType(SubjectKey.Source);
	FString SourceTypeStr = SourceType.ToString();
	bool bIsSequencer = SourceTypeStr.Contains(UE::LiveLink::SourceTypes::SequencerSourcePrefix, ESearchCase::IgnoreCase);
	bIsSequencerSourceAnyThread.store(bIsSequencer, std::memory_order_relaxed);

	if (bIsSequencer)
	{
		UE_LOGF(LogLiveLink, Log, "Sequencer source registered for '%ls', so genlock will be bypassed to avoid deadlocks",
			*SubjectKey.SubjectName.ToString());
	}

	FDelegateHandle DummyStaticDelegateHandle;

	LiveLinkClient->RegisterForFrameDataReceived(
		RegisteredSubjectKey
		, FOnLiveLinkSubjectStaticDataReceived::FDelegate()
		, FOnLiveLinkSubjectFrameDataReceived::FDelegate::CreateUObject(this, &ULiveLinkCustomTimeStep::OnLiveLinkFrameDataReceived_AnyThread)
		, DummyStaticDelegateHandle
		, RegisterForFrameDataReceivedHandle);
}


void ULiveLinkCustomTimeStep::UnregisterLiveLinkSubject()
{
	if (RegisterForFrameDataReceivedHandle.IsValid())
	{
		// Signal the frame data callback to skip the lockstep wait. Without this, the game
		// thread can stall on the broadcast critical section for up to LockStepTimeout because
		// OnLiveLinkFrameDataReceived_AnyThread holds that lock during its lockstep wait loop.
		bSkipLockStepWait.store(true, std::memory_order_release);
		SyncCount.store(0, std::memory_order_release);
		EventWaitForSync->Trigger();

		LiveLinkClient->UnregisterForFrameDataReceived(RegisteredSubjectKey, FDelegateHandle(), RegisterForFrameDataReceivedHandle);
		RegisterForFrameDataReceivedHandle.Reset();

		bSkipLockStepWait.store(false, std::memory_order_release);
	}

	// Reset sequencer flag when subject is unregistered
	bIsSequencerSourceAnyThread.store(false, std::memory_order_relaxed);

	RegisteredSubjectKey = FLiveLinkSubjectKey();
}


void ULiveLinkCustomTimeStep::OnLiveLinkClientRegistered(const FName& Type, class IModularFeature* ModularFeature)
{
	if (Type == ILiveLinkClient::ModularFeatureName && !LiveLinkClient)
	{
		InitLiveLinkClient();
	}
}


void ULiveLinkCustomTimeStep::OnLiveLinkClientUnregistered(const FName& Type, class IModularFeature* ModularFeature)
{
	if (Type == ILiveLinkClient::ModularFeatureName && ModularFeature == LiveLinkClient)
	{
		UninitLiveLinkClient();
		InitLiveLinkClient();
	}
}


void ULiveLinkCustomTimeStep::OnLiveLinkSubjectAdded(FLiveLinkSubjectKey InSubjectKey)
{
	if (InSubjectKey.SubjectName == SubjectKey.SubjectName)
	{
		// Check if already registered to a subject with same name.
		if (RegisteredSubjectKey.SubjectName == InSubjectKey.SubjectName && RegisteredSubjectKey.Source.IsValid())
		{
			UnregisterLiveLinkSubject();
		}

		// Preserve original configured key before updating
		FLiveLinkSubjectKey OriginalConfiguredKey = SubjectKey;

		// Update the subject key
		SubjectKey = InSubjectKey;
		RegisterLiveLinkSubject();

		// If successfully registered, clear error state
		if (State == ECustomTimeStepSynchronizationState::Error)
		{
			State = ECustomTimeStepSynchronizationState::Synchronizing;
		}

		// Restore configured key if registered to sequencer
		if (bIsSequencerSourceAnyThread.load(std::memory_order_relaxed))
		{
			SubjectKey = OriginalConfiguredKey;
		}
	}
}


void ULiveLinkCustomTimeStep::OnLiveLinkSubjectRemoved(FLiveLinkSubjectKey InSubjectKey)
{
	if (InSubjectKey.SubjectName == RegisteredSubjectKey.SubjectName)
	{
		// Check if this is a sequencer source being removed
		const bool bWasSequencerSource = bIsSequencerSourceAnyThread.load(std::memory_order_relaxed);

		// Unregister from the removed source
		UnregisterLiveLinkSubject();

		// If the removed source was sequencer's, try to re-register to the original external subject
		if (bWasSequencerSource && LiveLinkClient)
		{
			// Check if the original configured subject still exists
			if (LiveLinkClient->GetSubjectSettings(SubjectKey))
			{
				UE_LOGF(LogLiveLink, Warning, "OnLiveLinkSubjectRemoved: Original external subject '%ls' found, re-registering",
					*SubjectKey.SubjectName.ToString());

				// Re-register to the original external subject
				RegisterLiveLinkSubject();

				// Clear error state
				if (State == ECustomTimeStepSynchronizationState::Error)
				{
					State = ECustomTimeStepSynchronizationState::Synchronizing;
				}
			}
			else
			{
				UE_LOGF(LogLiveLink, Log, "OnLiveLinkSubjectRemoved: Original subject not found, entering Error state");
				State = ECustomTimeStepSynchronizationState::Error;
			}
		}
		else
		{
			// External subject was removed, enter error state
			UE_LOGF(LogLiveLink, Log, "OnLiveLinkSubjectRemoved: External subject removed, entering Error state");
			State = ECustomTimeStepSynchronizationState::Error;
		}
	}
}


void ULiveLinkCustomTimeStep::OnLiveLinkFrameDataReceived_AnyThread(const FLiveLinkFrameDataStruct& InFrameData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkCustomTimeStep::OnLiveLinkFrameDataReceived_AnyThread);

	// Increment sync counter based on frame id. For example, if frame id is incremented by 2, it means that a frame
	// was skipped and we must report 2 sync signals, even though only one was received. The purpose of this is to
	// reflect source delta times in the engine delta times.

	const FLiveLinkFrameIdentifier FrameId = InFrameData.GetBaseData()->FrameId;
	const FQualifiedFrameTime CurrentSceneTime = InFrameData.GetBaseData()->MetaData.SceneTime;

	// Receiving data from the desired live link source automatically means that we are in synchronized state.
	if (State != ECustomTimeStepSynchronizationState::Synchronized)
	{
		State = ECustomTimeStepSynchronizationState::Synchronized;

		// Ensure our initial delta frame id is 1.
		LastFrameId = FrameId - 1;

		// Reset tracking for phase alignment
		LastCapturedTimecodeFrameNumber.store(INDEX_NONE, std::memory_order_release);
		ReferenceFrameId.store(INDEX_NONE, std::memory_order_release);

		// Calculate frames per timecode from LiveLink data rate and subject timecode rate

		const double SourceFPS = LiveLinkDataRate.AsDecimal();
		const double TimecodeFPS = CurrentSceneTime.Rate.AsDecimal();

		if (SourceFPS > 0.0 && TimecodeFPS > 0.0)
		{
			const int32 FramesPerTC = FMath::Max(1, FMath::RoundToInt(SourceFPS / TimecodeFPS));
			FramesPerTimecodeAnyThread.store(FramesPerTC, std::memory_order_release);
		}
		else
		{
			UE_LOGF(LogLiveLink, Error, "LiveLinkCustomTimeStep: Invalid frame rates - SourceFPS=%.2f, TimecodeFPS=%.2f", SourceFPS, TimecodeFPS);
		}
	}

	// Loop around cannot be correctly handled unless we know the lower and upper bounds of the frame id.
	// For now, consider them as single increments even though we may miss skipped frames.
	// If FrameId is not changing, then we assume that the source does not support this, and we treat this
	// as a single sync increment as we otherwise lack the means of detecting skipped source frames.
	if (LastFrameId >= FrameId)
	{
		SyncCount++;
	}
	else
	{
		SyncCount += FrameId - LastFrameId;
	}

	// Note: We don't expect OnLiveLinkFrameDataReceived_AnyThread to be called for the same live link source concurrently,
	// otherwise we would need to avoid race conditions here when updating and comparing with FrameId.
	const FLiveLinkFrameIdentifier PrevFrameId = LastFrameId.load(std::memory_order_acquire);
	LastFrameId = FrameId;

	// Extract timecode frame for deterministic subframe capture

	const int32 CurrentTimecodeFrame = CurrentSceneTime.Time.FloorToFrame().Value;
	const int32 PreviousTimecodeFrame = LastCapturedTimecodeFrameNumber.load(std::memory_order_acquire);

	int32 RefFrameId = ReferenceFrameId.load(std::memory_order_acquire);

	const int32 FramesPerTC = FramesPerTimecodeAnyThread.load(std::memory_order_acquire);
	const int32 Divider = static_cast<int32>(FrameRateDividerAnyThread.load(std::memory_order_relaxed));

	// Establish reference frame on first new timecode with consecutive FrameId
	// This ensures we capture from the actual first frame (subframe 0.0) and not a potentially skipped subframe
	if (RefFrameId == INDEX_NONE || CurrentTimecodeFrame != PreviousTimecodeFrame)
	{
		// Check if timecode changed and frame is consecutive

		const bool bIsFirstFrame = (PrevFrameId == FrameId - 1) || (PrevFrameId == 0);

		// Note: FrameId should be filled in by our framework before this point, so bIsFirstFrame should be valid
		if (CurrentTimecodeFrame != PreviousTimecodeFrame && bIsFirstFrame)
		{
			ReferenceFrameId.store(FrameId, std::memory_order_release);
			LastCapturedTimecodeFrameNumber.store(CurrentTimecodeFrame, std::memory_order_release);
			RefFrameId = FrameId; // Update local copy

			// Capture this reference frame's time for genlock
			{
				FScopeLock Lock(&GenlockSceneTimeLock);
				GenlockTriggerSceneTime = CurrentSceneTime;
			}

			UE_LOGF(LogLiveLink, Verbose, "LiveLinkCustomTimeStep::OnLiveLinkFrameDataReceived_AnyThread FrameId=%d, TimecodeFrame=%d, SceneTimeFrame=%d",
				FrameId, CurrentTimecodeFrame, CurrentSceneTime.Time.FloorToFrame().Value);
		}
	}

	// Update genlock trigger time on frames aligned with the divider pattern

	if (RefFrameId != INDEX_NONE && FramesPerTC > 0 && Divider > 0)
	{
		const int32 FrameOffset = static_cast<int32>(FrameId - RefFrameId);

		// Calculate capture stride using GCD for proper phase alignment
		const int32 CaptureStride = FMath::GreatestCommonDivisor(Divider, FramesPerTC);

		if (FrameOffset % CaptureStride == 0 && FrameOffset > 0)
		{
			// This frame is aligned with the divider pattern, so we update genlock trigger time

			FScopeLock Lock(&GenlockSceneTimeLock);
			GenlockTriggerSceneTime = CurrentSceneTime;
		}
	}

	// Trigger the event to release WaitForSync.
	// If the engine cannot keep up, it means that SyncCount will be greater than FrameRateDividerAnyThread and delta time will be multiplied accordingly
	if (SyncCount >= FrameRateDividerAnyThread)
	{
		EventLiveLink->Trigger();

		// In lockstep mode, the engine can stall the live link producer until it has had a chance to process its current data.
		// We can't block if this function is being called in the game thread because it would deadlock.
		if (bLockStepModeAnyThread && !IsInGameThread() && !bSkipLockStepWait.load(std::memory_order_acquire))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ULiveLinkCustomTimeStep::EventWaitForSync->Wait);

			// Wait until the engine has consumed SyncCount.

			const FTimespan LockStepTimeout((TimeoutInSecondsAnyThread / 2) * ETimespan::TicksPerSecond);

			while (SyncCount.load(std::memory_order_acquire) >= FrameRateDividerAnyThread.load(std::memory_order_relaxed))
			{
				if (!EventWaitForSync->Wait(LockStepTimeout) || bSkipLockStepWait.load(std::memory_order_acquire))
				{
					break;
				}
			}
		}
	}
}

