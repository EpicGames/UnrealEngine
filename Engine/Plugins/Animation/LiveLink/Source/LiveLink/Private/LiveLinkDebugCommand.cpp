// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDebugCommand.h"

#include "Engine/GameViewportClient.h"
#include "LiveLinkDebugView.h"

#if WITH_EDITOR
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#else
#include "Engine/Engine.h"
#endif

static const FName LevelEditorName("LevelEditor");

FLiveLinkDebugCommand::FLiveLinkDebugCommand(FLiveLinkClient& InClient)
	: Client(InClient)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, CommandShow(MakeUnique<FAutoConsoleCommand>(TEXT("LiveLink.ShowDebugInfo"), TEXT("Show debug information for LiveLink Sources and Subjects"), FConsoleCommandDelegate::CreateRaw(this, &FLiveLinkDebugCommand::ShowDebugInfo)))
	, CommandHide(MakeUnique<FAutoConsoleCommand>(TEXT("LiveLink.HideDebugInfo"), TEXT("Hide debug information for LiveLink Sources and Subjects"), FConsoleCommandDelegate::CreateRaw(this, &FLiveLinkDebugCommand::HideDebugInfo)))
#endif
	, bRenderDebugInfo(false)
	, DebugViewEditor(nullptr)
	, DebugViewGame(nullptr)
{
}

void FLiveLinkDebugCommand::ShowDebugInfo()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (bRenderDebugInfo)
		return;
	
	bRenderDebugInfo = true;

#if WITH_EDITOR
	if (FModuleManager::Get().IsModuleLoaded(LevelEditorName))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorName);
		TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
		if (ActiveLevelViewport.IsValid())
		{
			TSharedRef<SLiveLinkDebugView> DebugView = SNew(SLiveLinkDebugView, &Client);
			DebugViewEditor = DebugView;
			ActiveLevelViewport->AddOverlayWidget(DebugView);
		}
	}
#endif

	if (GEngine && GEngine->GameViewport)
	{
		TSharedRef<SLiveLinkDebugView> DebugView = SNew(SLiveLinkDebugView, &Client);
		DebugViewGame = DebugView;
		GEngine->GameViewport->AddViewportWidgetContent(DebugView);
	}
#endif
}

void FLiveLinkDebugCommand::HideDebugInfo()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!bRenderDebugInfo)
		return;

	bRenderDebugInfo = false;

#if WITH_EDITOR
	if (FModuleManager::Get().IsModuleLoaded(LevelEditorName))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorName);
		TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
		if (ActiveLevelViewport.IsValid() && DebugViewEditor.IsValid())
		{
			if (TSharedPtr<SLiveLinkDebugView> Pinned = DebugViewEditor.Pin())
			{
				ActiveLevelViewport->RemoveOverlayWidget(Pinned->AsShared());
			}
		}
	}
#endif

	if (GEngine && GEngine->GameViewport && DebugViewGame.IsValid())
	{
		if (TSharedPtr<SLiveLinkDebugView> Pinned = DebugViewGame.Pin())
		{
			GEngine->GameViewport->RemoveViewportWidgetContent(Pinned->AsShared());
		}
	}

	DebugViewEditor.Reset();
	DebugViewGame.Reset();
#endif
}
