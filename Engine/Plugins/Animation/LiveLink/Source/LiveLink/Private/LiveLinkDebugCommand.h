// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FAutoConsoleCommand;
class FLiveLinkClient;
class SLiveLinkDebugView;

class FLiveLinkDebugCommand
{
public:
	FLiveLinkDebugCommand(FLiveLinkClient& InClient);

private:
	void ShowDebugInfo();
	void HideDebugInfo();

private:
	FLiveLinkClient& Client;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TUniquePtr<FAutoConsoleCommand> CommandShow;
	TUniquePtr<FAutoConsoleCommand> CommandHide;
#endif

	bool bRenderDebugInfo;
	TWeakPtr<SLiveLinkDebugView> DebugViewEditor;
	TWeakPtr<SLiveLinkDebugView> DebugViewGame;
};
