// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDebugView.h"

#include "Framework/Views/TableViewMetadata.h"
#include "LiveLinkClient.h"
#include "LiveLinkModule.h"
#include "LiveLinkSettings.h"

#include "Brushes/SlateColorBrush.h"
#include "Styling/SlateStyle.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Views/SListView.h"

namespace
{
	static const int IndentationSource = 0;
	static const int IndentationSubject = 12;
	static const FLinearColor BackgroundColorSource(FColor(62, 62, 62, 180));
	static const FLinearColor BackgroundColorSubject(FColor(62, 62, 62, 120));
}

// Structure that defines a single entry in the debug UI
struct FLiveLinkDebugUIEntry : public TSharedFromThis<FLiveLinkDebugUIEntry>
{
public:
	FLiveLinkDebugUIEntry(FLiveLinkSubjectKey InSubjectKey, FLiveLinkClient* InClient)
		: SubjectKey(InSubjectKey)
		, Client(InClient)
	{}

	bool IsSubject() const { return !SubjectKey.SubjectName.IsNone(); }
	bool IsSubjectEnabled() const { return Client->IsSubjectEnabled(SubjectKey, true); }
	bool IsSubjectValid() const { return Client->IsSubjectValid(SubjectKey.SubjectName); }
	bool IsSource() const { return SubjectKey.SubjectName.IsNone(); }
	bool IsSourceValid() const { return Client->IsSourceStillValid(SubjectKey.Source); }
	bool IsPaused() const { return Client->GetSubjectState(SubjectKey.SubjectName) == ELiveLinkSubjectState::Paused; }

	FText GetItemText() const
	{
		if (IsSource())
		{
			return Client->GetSourceType(SubjectKey.Source);
		}
		else
		{
			return FText::FromName(SubjectKey.SubjectName);
		}
	}

private:
	FLiveLinkSubjectKey SubjectKey;
	FLiveLinkClient* Client;
};


void SLiveLinkDebugView::Construct(const FArguments& Args, FLiveLinkClient* InClient)
{
	check(InClient);
	Client = InClient;

	// don't react on input so it is passed on to the other widgets rendered on top of the viewport
	SetVisibility(EVisibility::HitTestInvisible);

	BackgroundBrushSource = MakeShared<FSlateColorBrush>(BackgroundColorSource);
	BackgroundBrushSubject = MakeShared<FSlateColorBrush>(BackgroundColorSubject);

	RefreshSourceItems();

	Client->OnLiveLinkSourcesChanged().AddSP(this, &SLiveLinkDebugView::HandleSourcesChanged);
	Client->OnLiveLinkSubjectsChanged().AddSP(this, &SLiveLinkDebugView::HandleSourcesChanged);

	SAssignNew(DebugItemView, SListView<FLiveLinkDebugUIEntryPtr>)
		.ListItemsSource(&DebugItemData)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SLiveLinkDebugView::GenerateRow);

	if (TSharedPtr<FSlateStyleSet> Style = ILiveLinkModule::Get().GetStyle())
	{
		ValidBrush = Style->GetBrush("LiveLink.Subject.Okay");
		InvalidBrush = Style->GetBrush("LiveLink.Subject.Warning");
		PausedBrush = Style->GetBrush("LiveLink.Subject.Paused");
	}

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		[
			DebugItemView.ToSharedRef()
		]
	];
}

SLiveLinkDebugView::~SLiveLinkDebugView()
{
	if (Client)
	{
		Client->OnLiveLinkSourcesChanged().RemoveAll(this);
	}
}

TSharedRef<ITableRow> SLiveLinkDebugView::GenerateRow(FLiveLinkDebugUIEntryPtr Data, const TSharedRef<STableViewBase>& OwnerTable)
{
	const int Indentation = Data->IsSource() ? IndentationSource : IndentationSubject;
	const int TextSize = Data->IsSource() ? GetDefault<ULiveLinkSettings>()->TextSizeSource : GetDefault<ULiveLinkSettings>()->TextSizeSubject;
	FSlateColorBrush* BgBrush = Data->IsSource() ? BackgroundBrushSource.Get() : BackgroundBrushSubject.Get();

	return SNew(STableRow<FLiveLinkDebugUIEntryPtr>, OwnerTable)
	[
		SNew(SBorder)
		.BorderImage(BgBrush)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MaxWidth(20.0f)
			.Padding(4, 0, 6, 0)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SLiveLinkDebugView::GetSubjectIcon, Data)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(Indentation, 0, 0, 0)
			[
				SNew(STextBlock)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", TextSize))
				.Text(Data.Get(), &FLiveLinkDebugUIEntry::GetItemText)
			]
		]
	];
}

void SLiveLinkDebugView::HandleSourcesChanged()
{
	RefreshSourceItems();
	DebugItemView->RebuildList();
}

void SLiveLinkDebugView::RefreshSourceItems()
{
	DebugItemData.Reset();

	for (FGuid SourceGuid : Client->GetDisplayableSources())
	{
		DebugItemData.Add(MakeShared<FLiveLinkDebugUIEntry>(FLiveLinkSubjectKey(SourceGuid, FName()), Client));

		const TArray<FLiveLinkSubjectKey> Subjects = Client->GetSubjects(true, true);
		for (const FLiveLinkSubjectKey& SubjectKey : Subjects)
		{
			if (SubjectKey.Source == SourceGuid)
			{
				DebugItemData.Add(MakeShared<FLiveLinkDebugUIEntry>(SubjectKey, Client));
			}
		}
	}
}

const FSlateBrush* SLiveLinkDebugView::GetSubjectIcon(FLiveLinkDebugUIEntryPtr SubjectEntry) const
{
	const FSlateBrush* IconBrush = nullptr;

	if (SubjectEntry->IsSubjectEnabled())
	{
		if (!SubjectEntry->IsPaused())
		{
			IconBrush = SubjectEntry->IsSubjectValid() ? ValidBrush : InvalidBrush;
		}
		else
		{
			IconBrush = PausedBrush;
		}
	}
	else
	{
		IconBrush = DisabledBrush;
	}

	return IconBrush;
}
