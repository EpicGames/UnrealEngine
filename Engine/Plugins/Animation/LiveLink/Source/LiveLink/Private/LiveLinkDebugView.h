// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
template <typename ItemType> class SListView;

class FLiveLinkClient;
struct FSlateBrush;
struct FSlateColorBrush;

struct FLiveLinkDebugUIEntry;
using FLiveLinkDebugUIEntryPtr = TSharedPtr<FLiveLinkDebugUIEntry>;

class SLiveLinkDebugView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkDebugView) {}
	SLATE_END_ARGS()

	virtual ~SLiveLinkDebugView() override;

	void Construct(const FArguments& Args, FLiveLinkClient* InClient);

private:
	TSharedRef<ITableRow> GenerateRow(FLiveLinkDebugUIEntryPtr Data, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleSourcesChanged();
	void RefreshSourceItems();
	/** Get the subject icon according to the subject's status. */
	const FSlateBrush* GetSubjectIcon(FLiveLinkDebugUIEntryPtr SubjectEntry) const;

private:
	FLiveLinkClient* Client;
	TArray<FLiveLinkDebugUIEntryPtr> DebugItemData;

	TSharedPtr<SListView<FLiveLinkDebugUIEntryPtr>> DebugItemView;
	TSharedPtr<FSlateColorBrush> BackgroundBrushSource;
	TSharedPtr<FSlateColorBrush> BackgroundBrushSubject;

	/** Cached brush for valid subjects. */
	const FSlateBrush* ValidBrush = nullptr;
	/** Cached brush for invalid subjects. */
	const FSlateBrush* InvalidBrush = nullptr;
	/** Cached brush for paused subjects. */
	const FSlateBrush* PausedBrush = nullptr;
	/** Cached brush for disabled subjects */
	const FSlateBrush* DisabledBrush = nullptr;
};
