// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHeartbeatEmitter.h"

#include "LiveLinkMessages.h"
#include "LiveLinkSettings.h"

#include "HAL/RunnableThread.h"
#include "MessageEndpoint.h"


bool FLiveLinkHeartbeatEmitter::FHeartbeatRecipient::operator==(const FLiveLinkHeartbeatEmitter::FHeartbeatRecipient& other) const
{
	return other.MessageEndpoint == this->MessageEndpoint && other.ConnectionAddress == this->ConnectionAddress;
}


FLiveLinkHeartbeatEmitter::FLiveLinkHeartbeatEmitter()
	: bIsRunning(false)
	, HeartbeatFrequencyInMs(GetDefault<ULiveLinkSettings>()->GetMessageBusHeartbeatFrequency() * 1000.0f)
	, HeartbeatEvent(nullptr)
{

}

FLiveLinkHeartbeatEmitter::~FLiveLinkHeartbeatEmitter() = default;

void FLiveLinkHeartbeatEmitter::StartHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint)
{
	if (!bIsRunning)
	{
		bIsRunning = true;
		HeartbeatEvent = FGenericPlatformProcess::GetSynchEventFromPool();
		check(HeartbeatEvent);
		Thread.Reset(FRunnableThread::Create(this, TEXT("LiveLinkHeartbeatEmitter")));
	}

	FScopeLock Lock(&CriticalSection);
	HeartbeatRecipients.Add({ MessageEndpoint, RecipientAddress });
}

void FLiveLinkHeartbeatEmitter::StopHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint)
{
	FScopeLock Lock(&CriticalSection);
	HeartbeatRecipients.Remove({ MessageEndpoint, RecipientAddress });
}

uint32 FLiveLinkHeartbeatEmitter::Run()
{
	while (bIsRunning)
	{
		check(HeartbeatEvent);
		HeartbeatEvent->Wait(HeartbeatFrequencyInMs);

		FScopeLock Lock(&CriticalSection);
		if (bIsRunning) // make sure we were not told to exit during the wait
		{
			for (const FHeartbeatRecipient& Recipient : HeartbeatRecipients)
			{
				if (TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = Recipient.MessageEndpoint.Pin())
				{
					MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FLiveLinkHeartbeatMessage>(), Recipient.ConnectionAddress);
				}
			}
		}
	}

	return 0;
}

void FLiveLinkHeartbeatEmitter::Exit()
{
	if (!bIsRunning)
	{
		return;
	}

	bIsRunning = false;
	{
		FScopeLock Lock(&CriticalSection);
		HeartbeatRecipients.Empty();
	}

	if (HeartbeatEvent)
	{
		HeartbeatEvent->Trigger();
	}

	Thread->WaitForCompletion();

	if (HeartbeatEvent)
	{
		FGenericPlatformProcess::ReturnSynchEventToPool(HeartbeatEvent);
		HeartbeatEvent = nullptr;
	}
}
