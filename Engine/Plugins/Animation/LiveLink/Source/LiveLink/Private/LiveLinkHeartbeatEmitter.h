// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"

#include "IMessageContext.h"

class FMessageEndpoint;

class  FEvent;
class  FRunnableThread;

/**
 * Sends out heartbeat signals at the rate defined by LiveLinkSettings::MessageBusHeartbeatFrequency to all recipients.
 *
 * Recipients for the heartbeat can be added by calling StartHeartbeat().
 * On the first call to StartHeartbeat a thread is spawned that keeps sending the heartbeat to everyone.
 * The heartbeats and the sender thread can be stopped by calling Exit().
 */
class FLiveLinkHeartbeatEmitter : public FRunnable
{
public:
	FLiveLinkHeartbeatEmitter();
	~FLiveLinkHeartbeatEmitter();

	/**
	 * Start sending a heartbeat to the specified Recipient by using the given MessageEndpoint.
	 * @note This will spawn a new thread when first called.
	 * @note It is assumed that each source that sends out heartbeats has its own MessageEndpoint.
	 */
	void StartHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint);

	/**
	 * Stop sending a heartbeat to the specified Recipient.
	 */
	void StopHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint);

public:
	/**
	 * FRunnable Interface
	 */
	virtual uint32 Run() override;
	virtual void Exit() override;

private:
	struct FHeartbeatRecipient
	{
		TWeakPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
		FMessageAddress ConnectionAddress;

		bool operator==(const FHeartbeatRecipient& other) const;
	};

private:
	mutable FCriticalSection CriticalSection;
	TArray<FHeartbeatRecipient> HeartbeatRecipients;
	FThreadSafeBool bIsRunning;
	float HeartbeatFrequencyInMs;

	FEvent* HeartbeatEvent;
	TUniquePtr<FRunnableThread> Thread;
};
