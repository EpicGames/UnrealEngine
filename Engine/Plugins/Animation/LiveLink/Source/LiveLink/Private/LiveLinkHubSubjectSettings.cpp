// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubSubjectSettings.h"

#include "Features/IModularFeatures.h"
#include "LiveLinkClient.h"
#include "LiveLinkModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkHubSubjectSettings)

namespace UE::Private::Utils
{
	/** Notify connected clients that this subject's name has changed. */
	static bool ValidateOutboundName(const FString& SubjectName, FName PreviousOutboundName, const FString& InOutboundNameCandidate)
	{
		if (InOutboundNameCandidate.IsEmpty() || FName(InOutboundNameCandidate) == NAME_None)
		{
			return false;
		}

		if (InOutboundNameCandidate == SubjectName)
		{
			return true;
		}

		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		// Can't rename to an existing subject, so check if it exists by querying for its role.
		return LiveLinkClient->GetSubjectRole_AnyThread(FLiveLinkSubjectName(*InOutboundNameCandidate)).Get() == nullptr;
	}
}

void ULiveLinkHubSubjectSettings::Initialize(FLiveLinkSubjectKey InSubjectKey)
{
	if (Key != InSubjectKey)
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		SubjectName = InSubjectKey.SubjectName.ToString();
		Key = InSubjectKey;

		OutboundName = SubjectName;

		Source = LiveLinkClient->GetSourceType(InSubjectKey.Source).ToString();
	}
}

#if WITH_EDITOR
void ULiveLinkHubSubjectSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, OutboundName))
	{
		PreviousOutboundName = *OutboundName;
	}
}

void ULiveLinkHubSubjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const bool bIsInLiveLinkHubApp = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni);

	if (!bIsInLiveLinkHubApp)
	{
		// We're not using most of these settings in editor, so there's no use in calling CacheSubjectSettings or modifying the OutboundName.
		return;
	}

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, OutboundName))
	{
		if (PreviousOutboundName != *OutboundName)
		{
			if (UE::Private::Utils::ValidateOutboundName(SubjectName, PreviousOutboundName, OutboundName))
			{
				// LiveLinkModule->NotifyRename
				FLiveLinkModule& LiveLinkModule = FModuleManager::Get().GetModuleChecked<FLiveLinkModule>("LiveLink");
				LiveLinkModule.OnSubjectOutboundNameModified().Broadcast(Key, PreviousOutboundName.ToString(), OutboundName);
			}
			else
			{
				OutboundName = PreviousOutboundName.ToString();
			}
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, Translators)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, PreProcessors)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, Remapper))
	{
		FLiveLinkClient* LiveLinkClient = static_cast<FLiveLinkClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
		LiveLinkClient->CacheSubjectSettings(Key, this);
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, TranslatorsProxy))
	{
		Translators.Reset(1);
		if (TranslatorsProxy)
		{
			Translators.Add(TranslatorsProxy);
		}

		ValidateProcessors();

		// Re-assign TranslatorsProxy in case the translator was denied in the validate function.
		if (Translators.Num())
		{
			TranslatorsProxy = Translators[0];
		}
		else
		{
			TranslatorsProxy = nullptr;
		}


		FLiveLinkClient* LiveLinkClient = static_cast<FLiveLinkClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
		LiveLinkClient->CacheSubjectSettings(Key, this);
	}
}
#endif
