// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkLogInstance.h"
#include "HAL/IConsoleManager.h"
#include "LiveLinkClient.h"

#if WITH_EDITOR
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#endif

LLM_DEFINE_TAG(LiveLink_LiveLinkLogInstance);
#define LOCTEXT_NAMESPACE "LiveLinkLogInstance"


namespace LiveLinkLogDetail
{
	FLiveLinkLogInstance* PrivateInstance = nullptr;

	static FAutoConsoleCommand CLiveLinkLogEnableMessage(
		TEXT("LiveLink.Log.EnableMessage"),
		TEXT("Enable a LiveLink Message ID that was previously disabled.")
		TEXT("Use: \"LiveLink.Log.EnableMessage MessageID\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() > 0)
				{
					FLiveLinkLogInstance::SetEnableMessage(*(Args[0]), true);
				}
			}),
		ECVF_Cheat);

	static FAutoConsoleCommand CLiveLinkLogDisableMessage(
		TEXT("LiveLink.Log.DisableMessage"),
		TEXT("Disable a LiveLink Message ID.")
		TEXT("Use: \"LiveLink.Log.DisableMessage MessageID\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() > 0)
				{
					FLiveLinkLogInstance::SetEnableMessage(*(Args[0]), false);
				}
			}),
		ECVF_Cheat);

	static FAutoConsoleCommand CLiveLinkLogSilent(
		TEXT("LiveLink.Log.SilentMode"),
		TEXT("Silent all log from LiveLink.")
		TEXT("To silent: \"LiveLink.Log.SilentMode\"")
		TEXT("To set: \"LiveLink.Log.SilentMode TRUE|FALSE\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() > 0)
				{
					FLiveLinkLogInstance::SetSilent(Args[0].ToBool());
				}
				else
				{
					FLiveLinkLogInstance::SetSilent(true);
				}
			}),
		ECVF_Cheat);

	static FAutoConsoleCommand CLiveLinkLogRepeated(
		TEXT("LiveLink.Log.LogRepeated"),
		TEXT("Log all messages event if they are not log for the first time.")
		TEXT("To log: \"LiveLink.Log.LogRepeated\"")
		TEXT("To set: \"LiveLink.Log.LogRepeated TRUE|FALSE\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() > 0)
				{
					FLiveLinkLogInstance::SetLogRepeatedMessage(Args[0].ToBool());
				}
				else
				{
					FLiveLinkLogInstance::SetLogRepeatedMessage(true);
				}
			}),
		ECVF_Cheat);

}


void FLiveLinkLogInstance::CreateInstance()
{
	LLM_SCOPE_BYTAG(LiveLink_LiveLinkLogInstance);
	ensure(FLiveLinkLog::Instance == nullptr);
	if (FLiveLinkLog::Instance)
	{
		DestroyInstance();
	}
	LiveLinkLogDetail::PrivateInstance = new FLiveLinkLogInstance;
	FLiveLinkLog::Instance.Reset(LiveLinkLogDetail::PrivateInstance);
}


void FLiveLinkLogInstance::DestroyInstance()
{
	LLM_SCOPE_BYTAG(LiveLink_LiveLinkLogInstance);
	ensure(FLiveLinkLog::Instance);
	FLiveLinkLog::Instance.Reset();
	FLiveLinkLog::Instance = nullptr;
	LiveLinkLogDetail::PrivateInstance = nullptr;
}


bool FLiveLinkLogInstance::IsMessageEnabled(FName MessageID)
{
	if (FLiveLinkLog::Instance && FLiveLinkLog::Instance.Get() == LiveLinkLogDetail::PrivateInstance)
	{
		return LiveLinkLogDetail::PrivateInstance->IsMessageEnabled_Internal(MessageID);
	}
	return false;
}


void FLiveLinkLogInstance::SetEnableMessage(FName MessageID, bool bEnabled)
{
	if (FLiveLinkLog::Instance && FLiveLinkLog::Instance.Get() == LiveLinkLogDetail::PrivateInstance)
	{
		return LiveLinkLogDetail::PrivateInstance->SetEnableMessage_Internal(MessageID, bEnabled);
	}
}


void FLiveLinkLogInstance::SetSilent(bool bInSilentMode)
{
	if (FLiveLinkLog::Instance && FLiveLinkLog::Instance.Get() == LiveLinkLogDetail::PrivateInstance)
	{
		LiveLinkLogDetail::PrivateInstance->bSilentMode = bInSilentMode;
	}
}


void FLiveLinkLogInstance::SetLogRepeatedMessage(bool bInLog)
{
	if (FLiveLinkLog::Instance && FLiveLinkLog::Instance.Get() == LiveLinkLogDetail::PrivateInstance)
	{
		LiveLinkLogDetail::PrivateInstance->bLogAllRepeatableMessage = bInLog;
	}
}


FLiveLinkLogInstance::FLiveLinkLogInstance()
	: ErrorCount(0)
	, WarningCount(0)
	, InfoCount(0)
{
	const FDateTime DateTime = FDateTime::Now();
	double HighPerformanceClock = FPlatformTime::Seconds();
	const FTimespan Timespan = DateTime.GetTimeOfDay();
	HighPerformanceClockDelta = Timespan.GetTotalSeconds() - HighPerformanceClock;

#if WITH_EDITOR
	const FName LogName = "Live Link";
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogInitOptions;
	MessageLogModule.RegisterLogListing(LogName, LOCTEXT("LogLiveLink", "LiveLink"), LogInitOptions);
	LogListing = MessageLogModule.GetLogListing(LogName);
	LogListing->OnSelectionChanged().AddRaw(this, &FLiveLinkLogInstance::OnSelectionChanged);
	LogListing->OnDataChanged().AddRaw(this, &FLiveLinkLogInstance::OnDataChanged);

	FCoreDelegates::OnEndFrame.AddRaw(this, &FLiveLinkLogInstance::Update);
#endif
}


FLiveLinkLogInstance::~FLiveLinkLogInstance()
{
#if WITH_EDITOR
	FCoreDelegates::OnEndFrame.RemoveAll(this);
	LogListing->OnDataChanged().RemoveAll(this);
	LogListing->OnSelectionChanged().RemoveAll(this);
#endif
}


void FLiveLinkLogInstance::SetEnableMessage_Internal(FName MessageID, bool bEnabled)
{
	if (!MessageID.IsNone())
	{
		if (bEnabled)
		{
			DisabledMessages.RemoveSingleSwap(MessageID);
		}
		else
		{
			DisabledMessages.AddUnique(MessageID);
		}
	}
}


TPair<int32, FTimespan> FLiveLinkLogInstance::GetOccurrence(FName MessageID, FLiveLinkSubjectKey SubjectKey) const
{
	FScopeLock Lock(&CriticalSection);
	const FRepeatableMessage* FoundRepeatableMessage = FindRepeatableMessage(MessageID, SubjectKey);
	if (FoundRepeatableMessage)
	{
		return TPair<int32, FTimespan>(FoundRepeatableMessage->Data->Occurrence.Load(), FoundRepeatableMessage->Data->LastTimeOccured.Load());
	}
	return TPair<int32, FTimespan>(0, FTimespan(0));
}


TPair<int32, FTimespan> FLiveLinkLogInstance::GetSelectedOccurrence() const
{
#if WITH_EDITOR
	return SelectedMessageOccurrence;
#else
	return TPair<int32, FTimespan>(0, FTimespan(0));
#endif
}


void FLiveLinkLogInstance::GetLogCount(int32& OutErrorCount, int32& OutWarningCount, int32& OutInfoCount) const
{
	OutErrorCount = ErrorCount;
	OutWarningCount = WarningCount;
	OutInfoCount = InfoCount;
}


void FLiveLinkLogInstance::Update()
{
#if WITH_EDITOR
	FScopeLock Lock(&CriticalSection);
	for (TSharedRef<FTokenizedMessage>& TokenizedMessage : ToUpdateMessages)
	{
		LogListing->AddMessage(TokenizedMessage);
	}
#endif
	ToUpdateMessages.Reset();
}


void FLiveLinkLogInstance::LogMessage(EMessageSeverity::Type Severity, FName MessageID, const FLiveLinkSubjectKey& SubjectKey, FString&& Message)
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		CreateTokenizedMessage(Severity, MessageID, SubjectKey, MoveTemp(Message));
		return;
	}
#endif

	if (!IsMessageEnabled_Internal(MessageID))
	{
		return;
	}

	IncrementLogCount(Severity);
	if (!SubjectKey.SubjectName.IsNone())
	{
		Message.ReplaceInline(TEXT("@@"), *SubjectKey.SubjectName.ToString());
	}

	if (MessageID.IsNone())
	{
		if (!bSilentMode)
		{
			LogMessage(Severity, Message);
		}
	}
	else
	{
		bool bLog = !bSilentMode;
		{
			FScopeLock Lock(&CriticalSection);
			FRepeatableMessage* FoundRepeatableMessage = FindRepeatableMessage(MessageID, SubjectKey);

			// Is it the first time
			if (FoundRepeatableMessage == nullptr)
			{
				FoundRepeatableMessage = &RepeatableMessages[RepeatableMessages.AddZeroed()];
				FoundRepeatableMessage->MessageID = MessageID;
				FoundRepeatableMessage->SubjectKey = SubjectKey;
				FoundRepeatableMessage->Data = MakeShared<FRepeatableMessageData>();
			}
			else
			{
				bLog = bLog && bLogAllRepeatableMessage;
			}

			++FoundRepeatableMessage->Data->Occurrence;
			FoundRepeatableMessage->Data->LastTimeOccured = FTimespan::FromSeconds(FPlatformTime::Seconds() + HighPerformanceClockDelta);

			if (bLog)
			{
				Message = FString::Printf(TEXT("Occurred %d time. Last occurrence %s. %s")
					, FoundRepeatableMessage->Data->Occurrence.Load()
					, *(FoundRepeatableMessage->Data->LastTimeOccured.Load().ToString())
					, *Message);
			}
		}

		if (bLog)
		{
			LogMessage(Severity, Message);
		}
	}
}


TSharedPtr<FTokenizedMessage> FLiveLinkLogInstance::CreateTokenizedMessage(EMessageSeverity::Type Severity, FName MessageID, const FLiveLinkSubjectKey& SubjectKey, FString&& Message)
{
	if (!IsMessageEnabled_Internal(MessageID))
	{
		return TSharedPtr<FTokenizedMessage>();
	}

	IncrementLogCount(Severity);

	// Is this something repeatable
	if (MessageID.IsNone())
	{
		TSharedRef<FTokenizedMessage> Token = FTokenizedMessage::Create(Severity);
		Token->AddToken(FTextToken::Create(FText::FromString(Message)));
		Token->SetIdentifier(MessageID);

#if WITH_EDITOR
		{
			FScopeLock Lock(&CriticalSection);
			ToUpdateMessages.Add(Token);
		}
#endif

		if (!bSilentMode)
		{
			LogMessage(Severity, Message);
		}

		return Token;
	}
	else
	{
		TSharedPtr<FTokenizedMessage> Token;
		bool bIsNewToken = false;
		bool bLog = !bSilentMode;
		{
			FScopeLock Lock(&CriticalSection);
			FRepeatableMessage* FoundRepeatableMessage = FindRepeatableMessage(MessageID, SubjectKey);

			if (FoundRepeatableMessage != nullptr)
			{
				Token = FoundRepeatableMessage->Token.Pin();
			}

			// Is it the first time
			if (FoundRepeatableMessage == nullptr)
			{
				FoundRepeatableMessage = &RepeatableMessages[RepeatableMessages.AddZeroed()];
				FoundRepeatableMessage->MessageID = MessageID;
				FoundRepeatableMessage->SubjectKey = SubjectKey;
				FoundRepeatableMessage->Data = MakeShared<FRepeatableMessageData>();
			}
			// it may have been flushed by the UI and we need to recreate it, if so reset the value
			else if (!Token.IsValid())
			{
				FoundRepeatableMessage->Data->Occurrence = 0;
			}

			++FoundRepeatableMessage->Data->Occurrence;
			FoundRepeatableMessage->Data->LastTimeOccured = FTimespan::FromSeconds(FPlatformTime::Seconds() + HighPerformanceClockDelta);


			if (!Token.IsValid())
			{
				TSharedRef<FRepeatableMessageData> Data = FoundRepeatableMessage->Data;
				TSharedRef<FDynamicTextToken> OccurenceToken = FDynamicTextToken::Create(MakeAttributeLambda([Data]() -> FText
					{
						FNumberFormattingOptions Options;
						Options.MinimumIntegralDigits = 2;
						return FText::Format(LOCTEXT("OccuredFrequency", "({0})"), FText::AsNumber(Data->Occurrence, &Options));
					}));

				Token = FTokenizedMessage::Create(Severity);
				Token->AddToken(OccurenceToken);

				int32 SubjectIndex = Message.Find(TEXT("@@"));
				while (SubjectIndex != INDEX_NONE)
				{
					Token->AddToken(FTextToken::Create(FText::FromString(Message.Left(SubjectIndex))));
					Token->AddToken(FTextToken::Create(FText::FromString(SubjectKey.SubjectName.ToString())));
					Message.RemoveAt(0, SubjectIndex+2, EAllowShrinking::No);
					SubjectIndex = Message.Find(TEXT("@@"));
				}
				if (Message.Len() > 0)
				{
					Token->AddToken(FTextToken::Create(FText::FromString(Message)));
				}
				Token->SetIdentifier(MessageID);

				FoundRepeatableMessage->Token = Token;
				ToUpdateMessages.Add(Token.ToSharedRef());
				bIsNewToken = true;
			}
			else
			{
				bLog = bLog && bLogAllRepeatableMessage;
			}

#if WITH_EDITOR
			if (SelectedToken == Token)
			{
				SelectedMessageOccurrence.Get<0>() = FoundRepeatableMessage->Data->Occurrence;
				SelectedMessageOccurrence.Get<1>() = FoundRepeatableMessage->Data->LastTimeOccured;
			}
#endif
		}

		if (bLog)
		{
			check(Token.IsValid());
			LogMessage(Severity, Token->ToText().ToString());
		}

		return bIsNewToken ? Token : TSharedPtr<FTokenizedMessage>();
	}
}


void FLiveLinkLogInstance::LogMessage(EMessageSeverity::Type Severity, const FString& Message)
{
	// The regular editor message log already outputs to UE_LOG(), so if we're in editor, we don't want to do it twice.
	// GIsEditor will be false if in -game and MessageLog won't take care of it so need to process it in this case.
#if WITH_EDITOR
	if (GIsEditor)
	{
		return;
	}
#endif

	switch (Severity)
	{
	case EMessageSeverity::Error:
		UE_LOGF(LogLiveLink, Error, "%ls", *Message);
		break;
	case EMessageSeverity::PerformanceWarning:
	case EMessageSeverity::Warning:
		UE_LOGF(LogLiveLink, Warning, "%ls", *Message);
		break;
	default:
		UE_LOGF(LogLiveLink, Log, "%ls", *Message);
		break;
	}
}


void FLiveLinkLogInstance::IncrementLogCount(EMessageSeverity::Type Severity)
{
	switch (Severity)
	{
	case EMessageSeverity::Error:
		++ErrorCount;
		break;
	case EMessageSeverity::PerformanceWarning:
	case EMessageSeverity::Warning:
		++WarningCount;
		break;
	default:
		++InfoCount;
		break;
	}
}


FLiveLinkLogInstance::FRepeatableMessage* FLiveLinkLogInstance::FindRepeatableMessage(FName MessageID, const FLiveLinkSubjectKey& SubjectKey)
{
	return RepeatableMessages.FindByPredicate([MessageID, &SubjectKey](const FRepeatableMessage& Msg)
		{
			return Msg.MessageID == MessageID && Msg.SubjectKey == SubjectKey;
		});
}


const FLiveLinkLogInstance::FRepeatableMessage* FLiveLinkLogInstance::FindRepeatableMessage(FName MessageID, const FLiveLinkSubjectKey& SubjectKey) const
{
	return RepeatableMessages.FindByPredicate([MessageID, &SubjectKey](const FRepeatableMessage& Msg)
		{
			return Msg.MessageID == MessageID && Msg.SubjectKey == SubjectKey;
		});
}


#if WITH_EDITOR
void FLiveLinkLogInstance::OnSelectionChanged()
{
	FScopeLock Lock(&CriticalSection);
	check(LogListing);
	const TArray<TSharedRef<FTokenizedMessage>>& SelectedMessages = LogListing->GetSelectedMessages();

	SelectedMessageOccurrence.Get<0>() = 0;
	SelectedMessageOccurrence.Get<1>() = FTimespan(0);

	if (SelectedMessages.Num() > 0)
	{
		TSharedRef<FTokenizedMessage> SelectedMessage = SelectedMessages[0];
		SelectedToken = SelectedMessage;
		const FRepeatableMessage* RepeatableMessage = RepeatableMessages.FindByPredicate([&SelectedMessage](const FRepeatableMessage& Msg)
			{
				return Msg.Token == SelectedMessage;
			});
		if (RepeatableMessage)
		{
			SelectedMessageOccurrence.Get<0>() = RepeatableMessage->Data->Occurrence;
			SelectedMessageOccurrence.Get<1>() = RepeatableMessage->Data->LastTimeOccured;
		}
	}
}


void FLiveLinkLogInstance::OnDataChanged()
{
	FScopeLock Lock(&CriticalSection);
	check(LogListing);
	// Get the number of all messages. If it's 0, then the user cleared the list via the UI
	int32 NumberOfMessages = LogListing->NumMessages(EMessageSeverity::Info);
	if (NumberOfMessages == 0)
	{
		RepeatableMessages.Reset();
		SelectedMessageOccurrence.Get<0>() = 0;
		SelectedMessageOccurrence.Get<1>() = FTimespan(0);
		SelectedToken.Reset();
		ErrorCount = 0;
		WarningCount = 0;
		InfoCount = 0;
	}
}
#endif

#undef LOCTEXT_NAMESPACE
