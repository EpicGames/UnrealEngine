// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkLog.h"

class FTokenizedMessage;
class IMessageLogListing;
namespace EMessageSeverity { enum Type : int; }


#if WITH_EDITOR
#endif


class FLiveLinkLogInstance : public FLiveLinkLog
{
public:
	static void CreateInstance();
	static void DestroyInstance();

public:
	/** Is the message ID is enabled. */
	static bool IsMessageEnabled(FName MessageID);

	/** Set if a Message ID is enabled. */
	static void SetEnableMessage(FName MessageID, bool bEnabled);

	/** Set if we should log the message. */
	static void SetSilent(bool bSilentMode);

	/** Set if we should log all the message even if they are not the log for the first time. */
	static void SetLogRepeatedMessage(bool bLog);

	virtual TPair<int32, FTimespan> GetOccurrence(FName MessageID, FLiveLinkSubjectKey SubjectKey) const override;
	virtual TPair<int32, FTimespan> GetSelectedOccurrence() const override;
	virtual void GetLogCount(int32& OutErrorCount, int32& OutWarningCount, int32& OutInfoCount) const override;

private:
	FLiveLinkLogInstance();
	~FLiveLinkLogInstance();
	FLiveLinkLogInstance(const FLiveLinkLogInstance&) = delete;
	FLiveLinkLogInstance& operator=(const FLiveLinkLogInstance&) = delete;

protected:
	virtual void LogMessage(EMessageSeverity::Type Severity, FName MessageID, const FLiveLinkSubjectKey& SubjectKey, FString&& Message) override;
	virtual TSharedPtr<FTokenizedMessage> CreateTokenizedMessage(EMessageSeverity::Type Severity, FName MessageID, const FLiveLinkSubjectKey& SubjectKey, FString&& Message) override;

private:
	struct FRepeatableMessageData
	{
		TAtomic<FTimespan> LastTimeOccured;
		TAtomic<int32> Occurrence;

		FRepeatableMessageData()
			: Occurrence(0)
		{ }
	};

	struct FRepeatableMessage
	{
		FName MessageID;
		FLiveLinkSubjectKey SubjectKey;
		TSharedRef<FRepeatableMessageData> Data;
		TWeakPtr<FTokenizedMessage> Token;
	};

private:
	void Update();
	FRepeatableMessage* FindRepeatableMessage(FName MessageID, const FLiveLinkSubjectKey& SubjectKey);
	const FRepeatableMessage* FindRepeatableMessage(FName MessageID, const FLiveLinkSubjectKey& SubjectKey) const;

	void LogMessage(EMessageSeverity::Type Severity, const FString& Message);
	void IncrementLogCount(EMessageSeverity::Type Severity);

	bool IsMessageEnabled_Internal(FName MessageID)
	{
		return MessageID.IsNone() || !DisabledMessages.Contains(MessageID);
	}

	void SetEnableMessage_Internal(FName MessageID, bool bEnabled);

#if WITH_EDITOR
	void OnSelectionChanged();
	void OnDataChanged();
#endif

private:
	/** Critical section for synchronizing access to the different list. */
	mutable FCriticalSection CriticalSection;

	/** List of tokenized messages. */
	TArray<TSharedRef<FTokenizedMessage>> ToUpdateMessages;
	
	/** List of all repeatable messages. */
	TArray<FRepeatableMessage> RepeatableMessages;

#if WITH_EDITORONLY_DATA
	/** Listing used in the editor by the Message Log. */
	TSharedPtr<IMessageLogListing> LogListing;

	/** Last selected message occurrence. */
	TPair<int32, FTimespan> SelectedMessageOccurrence;

	/** Last selected message. */
	TWeakPtr<FTokenizedMessage> SelectedToken;
#endif

	/** Disabled messages ID. */
	TArray<FName> DisabledMessages;

	/** Number of error. */
	TAtomic<int32> ErrorCount;

	/** Number of warning. */
	TAtomic<int32> WarningCount;

	/** Number of info. */
	TAtomic<int32> InfoCount;

	/** Should we be silent? */
	bool bSilentMode = false;

	/** Should we all instance of the repeatable message or only the first one and last one? */
	bool bLogAllRepeatableMessage = false;

	/** Delta time between the fast clock and slow clock. */
	double HighPerformanceClockDelta;
};
