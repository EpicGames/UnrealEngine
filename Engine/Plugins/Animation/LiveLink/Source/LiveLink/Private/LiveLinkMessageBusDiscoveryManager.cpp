// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMessageBusDiscoveryManager.h"

#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "ILiveLinkClient.h"
#include "LiveLinkMessages.h"
#include "LiveLinkSettings.h"
#include "MessageEndpointBuilder.h"

LLM_DEFINE_TAG(LiveLink_LiveLinkMessageBusDiscoveryManager);

FLiveLinkMessageBusDiscoveryManager::FLiveLinkMessageBusDiscoveryManager()
	: bRunning(true)
	, Thread(nullptr)
{
	LLM_SCOPE_BYTAG(LiveLink_LiveLinkMessageBusDiscoveryManager);

	PingRequestCounter = 0;
	PingRequestFrequency = FTimespan::FromSeconds(GetDefault<ULiveLinkSettings>()->GetMessageBusPingRequestFrequency());

	PollEvent = FPlatformProcess::GetSynchEventFromPool();

	MessageEndpoint = FMessageEndpoint::Builder(TEXT("LiveLinkMessageHeartbeatManager"))
		.Handling<FLiveLinkPongMessage>(this, &FLiveLinkMessageBusDiscoveryManager::HandlePongMessage);

	bRunning = MessageEndpoint.IsValid();
	if (bRunning)
	{
		Thread = FRunnableThread::Create(this, TEXT("LiveLinkMessageBusDiscoveryManager"));
	}
}

FLiveLinkMessageBusDiscoveryManager::~FLiveLinkMessageBusDiscoveryManager()
{
	{
		FScopeLock Lock(&SourcesCriticalSection);
		
		// Disable the Endpoint message handling since the message could keep it alive a bit.
		if (MessageEndpoint)
		{
			MessageEndpoint->Disable();
			MessageEndpoint.Reset();
		}
	}

	if (Thread)
	{
		Thread->Kill(true);
		Thread = nullptr;
	}

	FPlatformProcess::ReturnSynchEventToPool(PollEvent);
	PollEvent = nullptr;
}

uint32 FLiveLinkMessageBusDiscoveryManager::Run()
{
	while (bRunning)
	{
		{
			FScopeLock Lock(&SourcesCriticalSection);

			if (PingRequestCounter > 0)
			{
				LastProviderPoolResults.Reset();
				LastPingRequest = FGuid::NewGuid();
				const int32 Version = ILiveLinkClient::LIVELINK_VERSION;
				MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FLiveLinkPingMessage>(LastPingRequest, Version));
			}
		}
		PollEvent->Wait(PingRequestFrequency.GetTotalMilliseconds());
	}
	return 0;
}

void FLiveLinkMessageBusDiscoveryManager::Stop()
{
	bRunning = false;
	PollEvent->Trigger();
}

void FLiveLinkMessageBusDiscoveryManager::AddDiscoveryMessageRequest()
{
	FScopeLock Lock(&SourcesCriticalSection);
	if (++PingRequestCounter == 1)
	{
		LastProviderPoolResults.Reset();
	}
}

void FLiveLinkMessageBusDiscoveryManager::RemoveDiscoveryMessageRequest()
{
	--PingRequestCounter;
}

TArray<FProviderPollResultPtr> FLiveLinkMessageBusDiscoveryManager::GetDiscoveryResults() const
{
	FScopeLock Lock(&SourcesCriticalSection);

	return LastProviderPoolResults;
}

bool FLiveLinkMessageBusDiscoveryManager::IsRunning() const
{
	return bRunning;
}

FMessageAddress FLiveLinkMessageBusDiscoveryManager::GetEndpointAddress() const
{
	return MessageEndpoint->GetAddress();
}

void FLiveLinkMessageBusDiscoveryManager::HandlePongMessage(const FLiveLinkPongMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FScopeLock Lock(&SourcesCriticalSection);

	if (Message.PollRequest == LastPingRequest)
	{
		// Verify Message.LiveLinkVersion to consider validity of discovered provider. Older UE always sends 1
		constexpr bool bIsValidProvider = true;
		const double MachineTimeOffset = LiveLinkMessageBusHelper::CalculateProviderMachineOffset(Message.CreationPlatformTime, Context);

		FProviderPollResult PollResult = { Context->GetSender(), Message.ProviderName, Message.MachineName, MachineTimeOffset, bIsValidProvider, Context->GetAnnotations() };
		PollResult.DiscoveryProtocolVersion = Message.DiscoveryProtocolVersion;

		LastProviderPoolResults.Emplace(MakeShared<FProviderPollResult, ESPMode::ThreadSafe>(MoveTemp(PollResult)));
	}
}
