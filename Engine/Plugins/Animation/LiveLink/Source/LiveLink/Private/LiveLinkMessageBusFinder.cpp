// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMessageBusFinder.h"

#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkMessages.h"
#include "LiveLinkMessageBusSource.h"
#include "LiveLinkMessageBusSourceFactory.h"
#include "MessageEndpointBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkMessageBusFinder)


namespace LiveLinkMessageBusHelper
{
	double CalculateProviderMachineOffset(double SourceMachinePlatformSeconds, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
	{
		const FDateTime Now = FDateTime::UtcNow();
		const double MyMachineSeconds = FPlatformTime::Seconds();
		const FTimespan Latency = Now - Context->GetTimeSent();
		double MachineTimeOffset = 0.0f;
		if (SourceMachinePlatformSeconds >= 0.0)
		{
			MachineTimeOffset = MyMachineSeconds - SourceMachinePlatformSeconds - Latency.GetTotalSeconds();
		}

		return MachineTimeOffset;
	}
}


ULiveLinkMessageBusFinder::ULiveLinkMessageBusFinder()
	: MessageEndpoint(nullptr)
{

};

void ULiveLinkMessageBusFinder::GetAvailableProviders(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, float Duration, TArray<FProviderPollResult>& AvailableProviders)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FLiveLinkMessageBusFinderAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			PollNetwork();

			FLiveLinkMessageBusFinderAction* NewAction = new FLiveLinkMessageBusFinderAction(LatentInfo, this, Duration, AvailableProviders);
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "GetAvailableProviders not executed. The previous action hasn't finished yet.");
		}
	}
};

void ULiveLinkMessageBusFinder::GetPollResults(TArray<FProviderPollResult>& AvailableProviders)
{
	FScopeLock ScopedLock(&PollDataCriticalSection);
	AvailableProviders = PollData;
};

void ULiveLinkMessageBusFinder::PollNetwork()
{
	if (!MessageEndpoint.IsValid())
	{
		MessageEndpoint = FMessageEndpoint::Builder(TEXT("LiveLinkMessageBusSource"))
			.Handling<FLiveLinkPongMessage>(this, &ULiveLinkMessageBusFinder::HandlePongMessage);
	}

	PollData.Reset();
	CurrentPollRequest = FGuid::NewGuid();
	const int32 Version = ILiveLinkClient::LIVELINK_VERSION;
	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FLiveLinkPingMessage>(CurrentPollRequest, Version));
};

void ULiveLinkMessageBusFinder::HandlePongMessage(const FLiveLinkPongMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.PollRequest == CurrentPollRequest)
	{
		FScopeLock ScopedLock(&PollDataCriticalSection);

		bool bIsValidProvider = true;

		if (Message.LiveLinkVersion < ILiveLinkClient::LIVELINK_VERSION) // UE4 Remote Clients always send 1 as the LiveLink version
		{
			bIsValidProvider = false;
		}

		const double MachineTimeOffset = LiveLinkMessageBusHelper::CalculateProviderMachineOffset(Message.CreationPlatformTime, Context);

		FProviderPollResult PollResult = { Context->GetSender(), Message.ProviderName, Message.MachineName, MachineTimeOffset, bIsValidProvider, Context->GetAnnotations() };
		PollResult.DiscoveryProtocolVersion = Message.DiscoveryProtocolVersion;

		PollData.Add(MoveTemp(PollResult));
	}
};

void ULiveLinkMessageBusFinder::ConnectToProvider(UPARAM(ref) FProviderPollResult& Provider, FLiveLinkSourceHandle& SourceHandle)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		TSharedPtr<FLiveLinkMessageBusSource> NewSource = MakeShared<FLiveLinkMessageBusSource>(FText::FromString(Provider.Name), FText::FromString(Provider.MachineName), Provider.Address, Provider.MachineTimeOffset);
		FGuid NewSourceGuid = LiveLinkClient->AddSource(NewSource);
		if (NewSourceGuid.IsValid())
		{
			if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(NewSourceGuid))
			{
				Settings->ConnectionString = ULiveLinkMessageBusSourceFactory::CreateConnectionString(Provider);
				Settings->Factory = ULiveLinkMessageBusSourceFactory::StaticClass();
			}
		}
		SourceHandle.SetSourcePointer(NewSource);
	}
	else
	{
		SourceHandle.SetSourcePointer(nullptr);
	}
};

ULiveLinkMessageBusFinder* ULiveLinkMessageBusFinder::ConstructMessageBusFinder()
{
	return NewObject<ULiveLinkMessageBusFinder>();
}



