// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMessageBusSource.h"

#include "ClientNetworkStatisticsModel.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkModule.h"
#include "INetworkMessagingExtension.h"
#include "LiveLinkClient.h"
#include "LiveLinkCompression.h"
#include "LiveLinkHeartbeatEmitter.h"
#include "LiveLinkLog.h"
#include "Misc/ScopeLock.h"

#include <atomic>

#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
#include "LiveLinkMessageBusDiscoveryManager.h"
#endif

#include "LiveLinkMessageBusSourceSettings.h"
#include "LiveLinkMessages.h"
#include "LiveLinkRole.h"
#include "LiveLinkRoleTrait.h"
#include "LiveLinkSettings.h"

#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Roles/LiveLinkAnimationTypes.h"


int32 GLiveLinkMessageBusSourceReconnectAfterTimeout = 1;
static FAutoConsoleVariableRef CVarLiveLinkMessageBusSourceReconnectAfterTimeout(
	TEXT("LiveLink.MessageBus.Source.ReconnectAfterTimeout"),
	GLiveLinkMessageBusSourceReconnectAfterTimeout,
	TEXT("When enabled, when the connection times out, it will try to re-connect instead of removing the source."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarLiveLinkSupportCompressPayloads(
	TEXT("LiveLink.SupportCompressPayloads"), 1,
	TEXT("Whether to add the annotation indicating that we support compressed animation data. Can be set to 0 to simulate that compressed payloads are not supported."),
	ECVF_RenderThreadSafe);

int32 GLiveLinkMessageBusEnableResequencing = 1;
static FAutoConsoleVariableRef CVarLiveLinkMessageBusEnableResequencing(
	TEXT("LiveLink.MessageBus.EnableResequencing"),
	GLiveLinkMessageBusEnableResequencing,
	TEXT("Enable frame resequencing to handle async delivery from UDP Message Bus."),
	ECVF_Default);

namespace UE::LiveLink::Private
{
	/**
	 * Returns true if FrameA comes before FrameB, handling int32 wraparound.
	 */
	FORCEINLINE bool IsFrameIdEarlier(int32 FrameA, int32 FrameB)
	{
		return int32(uint32(FrameA) - uint32(FrameB)) < 0;
	}

	/**
	 * Returns true if TestFrameId is the next sequential frame after BaseFrameId, handling wraparound.
	 */
	FORCEINLINE bool IsNextFrameId(int32 TestFrameId, int32 BaseFrameId)
	{
		return TestFrameId == int32(uint32(BaseFrameId) + 1u);
	}

	/**
	 * Returns true if the difference between two FrameIds is suspiciously large, indicating wraparound, restart, or extended pause.
	 */
	FORCEINLINE bool ShouldResyncFrameId(int32 IncomingFrameId, int32 LastDeliveredFrameId, int32 MaxBufferSize)
	{
		const int32 Diff = int32(uint32(IncomingFrameId) - uint32(LastDeliveredFrameId));
		const uint32 AbsDiff = (Diff >= 0) ? uint32(Diff) : (0u - uint32(Diff));

		// 2x buffer size as headroom, and a minimum in case the buffer is small

		const int32 Threshold = FMath::Max(MaxBufferSize * 2, 10);

		return AbsDiff > uint32(Threshold);
	}

	/**
	 * Returns true if frame A should be delivered before frame B based on FrameId.
	 */
	bool IsFrameEarlierThan(const FLiveLinkFrameDataStruct& A, const FLiveLinkFrameDataStruct& B)
	{
		const FLiveLinkBaseFrameData* DataA = A.GetBaseData();
		const FLiveLinkBaseFrameData* DataB = B.GetBaseData();

		if (!DataA || !DataB)
		{
			return false;
		}

		// Sort by FrameId when available

		const bool bHasFrameIdA = DataA->FrameId != INDEX_NONE;
		const bool bHasFrameIdB = DataB->FrameId != INDEX_NONE;

		if (bHasFrameIdA && bHasFrameIdB)
		{
			return UE::LiveLink::Private::IsFrameIdEarlier(DataA->FrameId, DataB->FrameId);
		}

		// If FrameIds are not available, keep reception order
		return false;
	}
}

FText FLiveLinkMessageBusSource::ValidSourceStatus()
{
	return NSLOCTEXT("LiveLinkMessageBusSource", "ActiveStatus", "Active");
}

FText FLiveLinkMessageBusSource::InvalidSourceStatus()
{
	return NSLOCTEXT("LiveLinkMessageBusSource", "InvalidConnection", "Waiting for connection");
}

FText FLiveLinkMessageBusSource::TimeoutSourceStatus()
{
	return NSLOCTEXT("LiveLinkMessageBusSource", "TimeoutStatus", "Not responding");
}

FLiveLinkMessageBusSource::FLiveLinkMessageBusSource(const FText& InSourceType, const FText& InSourceMachineName, const FMessageAddress& InConnectionAddress, double InMachineTimeOffset)
	: ConnectionAddress(InConnectionAddress)
	, bIsValid(false)
	, Client(nullptr)
	, SourceType(InSourceType)
	, SourceMachineName(InSourceMachineName)
	, ConnectionLastActive(0.0)
	, MachineTimeOffset(InMachineTimeOffset)
	, bDiscovering(false)
	, bInitialized(false )
{
}

void FLiveLinkMessageBusSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	if (ULiveLinkMessageBusSourceSettings* MessageBusSettings = Cast<ULiveLinkMessageBusSourceSettings>(Settings))
	{
		SourceSettings = TWeakObjectPtr<ULiveLinkMessageBusSourceSettings>(MessageBusSettings);

#if WITH_EDITOR
		MessageBusSettings->OnPropertyChangedDelegate.AddRaw(this, &FLiveLinkMessageBusSource::CacheMessageBusSourceSettings);
#endif

		CacheMessageBusSourceSettings();
	}
}

void FLiveLinkMessageBusSource::CacheMessageBusSourceSettings()
{
	if (const ULiveLinkMessageBusSourceSettings* MessageBusSettings = SourceSettings.Get())
	{
		bCachedEnableFrameResequencing.store(MessageBusSettings->bEnableFrameResequencing, std::memory_order_relaxed);
		CachedResequencerMaxBufferSize.store(MessageBusSettings->ResequencerMaxBufferSize, std::memory_order_relaxed);
		CachedResequencerMaxWaitTimeSeconds.store(MessageBusSettings->ResequencerMaxWaitTimeSeconds, std::memory_order_relaxed);
	}
}


void FLiveLinkMessageBusSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;

	Initialize();
}

void FLiveLinkMessageBusSource::Update()
{
	if (!ConnectionAddress.IsValid())
	{
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
		StartDiscovering();

		FLiveLinkMessageBusDiscoveryManager& DiscoveryManager = ILiveLinkModule::Get().GetMessageBusDiscoveryManager();
		for (const FProviderPollResultPtr& Result : DiscoveryManager.GetDiscoveryResults())
		{
			if (Client->GetSourceType(SourceGuid).ToString() == Result->Name)
			{
				ConnectionAddress = Result->Address;
				SourceMachineName = FText::FromString(Result->MachineName);
				MachineTimeOffset = Result->MachineTimeOffset;
				StopDiscovering();
				SendConnectMessage();
				UpdateConnectionLastActive();
				break;
			}
		}
#endif
	}
	else
	{
		const double HeartbeatTimeout = GetDefault<ULiveLinkSettings>()->GetMessageBusHeartbeatTimeout();
		const double CurrentTime = FPlatformTime::Seconds();

		float BytesLastSecond = 0.f;
		if (TOptional<FMessageTransportStatistics> Statistics = UE::LiveLinkHub::FClientNetworkStatisticsModel::GetLatestNetworkStatistics(ConnectionAddress))
		{
			BytesLastSecond = Statistics->TotalBytesReceived - AccumulatedBytes;
			AccumulatedBytes = Statistics->TotalBytesReceived;
		}

		if (CurrentTime - LastThroughputUpdate > 1.0)
		{
			LastThroughputUpdate = CurrentTime;
			CachedThroughput = BytesLastSecond / 1'000;
		}

		bIsValid = CurrentTime - ConnectionLastActive < HeartbeatTimeout;
		if (!bIsValid)
		{
			const double DeadSourceTimeout = GetDeadSourceTimeout();
			if (CurrentTime - ConnectionLastActive > DeadSourceTimeout)
			{
				RequestSourceShutdown();

				if (GLiveLinkMessageBusSourceReconnectAfterTimeout)
				{
					Initialize();
				}
				else
				{
					Client->RemoveSource(SourceGuid);
				}
			}
		}
	}
}

void FLiveLinkMessageBusSource::Initialize()
{
	if (bInitialized)
	{
		return;
	}

	for (const TSubclassOf<ULiveLinkRole>& RoleClass : FLiveLinkRoleTrait::GetRoles())
	{
		RoleInstances.Add(RoleClass->GetDefaultObject<ULiveLinkRole>());
	}

	CreateAndInitializeMessageEndpoint();

	if (ConnectionAddress.IsValid())
	{
		SendConnectMessage();
	}
	else
	{
		StartDiscovering();
		bIsValid = false;
	}

	UpdateConnectionLastActive();

	bInitialized = true;
}

void FLiveLinkMessageBusSource::InternalHandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UpdateConnectionLastActive();

	if (!Context->IsValid())
	{
		return;
	}

	if (bIsShuttingDown)
	{
		return;
	}

	UScriptStruct* MessageTypeInfo = Context->GetMessageTypeInfo().Get();
	if (MessageTypeInfo == nullptr)
	{
		return;
	}

	const bool bIsStaticData = MessageTypeInfo->IsChildOf(FLiveLinkBaseStaticData::StaticStruct());
	const bool bIsFrameData = MessageTypeInfo->IsChildOf(FLiveLinkBaseFrameData::StaticStruct());
	const bool bIsSerializedData = MessageTypeInfo->IsChildOf(FLiveLinkSerializedFrameData::StaticStruct());

	if (!bIsStaticData && !bIsFrameData && !bIsSerializedData)
	{
		return;
	}

	FName SubjectName = NAME_None;
	if (const FString* SubjectNamePtr = Context->GetAnnotations().Find(FLiveLinkMessageAnnotation::SubjectAnnotation))
	{
		SubjectName = *(*SubjectNamePtr);
	}
	if (SubjectName == NAME_None)
	{
		static const FName NAME_InvalidSubject = "LiveLinkMessageBusSource_InvalidSubject";
		FLiveLinkLog::ErrorOnce(NAME_InvalidSubject, FLiveLinkSubjectKey(SourceGuid, NAME_None), TEXT("No Subject Name was provided for connection '%s'"), *GetSourceMachineName().ToString());
		return;
	}

	// Find the role.
	TSubclassOf<ULiveLinkRole> SubjectRole;
	if (bIsStaticData)
	{
		// Check if it's in the Annotation first
		FName RoleName = NAME_None;
		if (const FString* RoleNamePtr = Context->GetAnnotations().Find(FLiveLinkMessageAnnotation::RoleAnnotation))
		{
			RoleName = *(*RoleNamePtr);
		}

		for (TWeakObjectPtr<ULiveLinkRole> WeakRole : RoleInstances)
		{
			if (ULiveLinkRole* Role = WeakRole.Get())
			{
				if (RoleName != NAME_None)
				{
					if (RoleName == Role->GetClass()->GetFName())
					{
						if (bIsStaticData && MessageTypeInfo->IsChildOf(Role->GetStaticDataStruct()))
						{
							SubjectRole = Role->GetClass();
							break;
						}
						if (bIsFrameData && MessageTypeInfo->IsChildOf(Role->GetFrameDataStruct()))
						{
							SubjectRole = Role->GetClass();
							break;
						}
					}
				}
				else
				{
					if (Role->GetStaticDataStruct() == MessageTypeInfo)
					{
						SubjectRole = Role->GetClass();
						break;
					}
				}
			}
		}

		if (SubjectRole.Get() == nullptr)
		{
			static const FName NAME_InvalidRole = "LiveLinkMessageBusSource_InvalidRole";
			FLiveLinkLog::ErrorOnce(NAME_InvalidRole, FLiveLinkSubjectKey(SourceGuid, SubjectName), TEXT("No Role was provided or found for subject '%s' with connection '%s'"), *SubjectName.ToString(), *GetSourceMachineName().ToString());
			return;
		}

	}

	const FLiveLinkSubjectKey SubjectKey(SourceGuid, SubjectName);
	if (bIsStaticData)
	{
		InitializeAndPushStaticData_AnyThread(SubjectName, SubjectRole, SubjectKey, Context, MessageTypeInfo);
	}
	else
	{
		InitializeAndPushFrameData_AnyThread(SubjectName, SubjectKey, Context, MessageTypeInfo);
	}
}

bool FLiveLinkMessageBusSource::IsSourceStillValid() const
{
	return ConnectionAddress.IsValid() && bIsValid;
}

FText FLiveLinkMessageBusSource::GetSourceStatus() const
{
	if (!ConnectionAddress.IsValid())
	{
		return InvalidSourceStatus();
	}
	else if (IsSourceStillValid())
	{
		return ValidSourceStatus();
	}
	return TimeoutSourceStatus();
}

FText FLiveLinkMessageBusSource::GetSourceToolTip() const
{
	const FString TemplateString = FString::Printf(TEXT("Throughput: %.1f KB/s"), CachedThroughput);
	return FText::FromString(TemplateString);
}

TSubclassOf<ULiveLinkSourceSettings> FLiveLinkMessageBusSource::GetSettingsClass() const
{
	return ULiveLinkMessageBusSourceSettings::StaticClass();
}

FMessageAddress FLiveLinkMessageBusSource::GetAddress() const
{
	FMessageAddress Address;
	if (MessageEndpoint)
	{
		Address = MessageEndpoint->GetAddress();
	}
	return Address;
}

void FLiveLinkMessageBusSource::StartHeartbeatEmitter()
{
	FLiveLinkHeartbeatEmitter& HeartbeatEmitter = ILiveLinkModule::Get().GetHeartbeatEmitter();
	HeartbeatEmitter.StartHeartbeat(ConnectionAddress, MessageEndpoint);
}

void FLiveLinkMessageBusSource::CreateAndInitializeMessageEndpoint()
{
	FMessageEndpointBuilder EndpointBuilder = FMessageEndpoint::Builder(GetSourceName());
	InitializeMessageEndpoint(EndpointBuilder);
	PostInitializeMessageEndpoint(MessageEndpoint);
}

void FLiveLinkMessageBusSource::InitializeAndPushStaticData_AnyThread(FName SubjectName,
																	  TSubclassOf<ULiveLinkRole> SubjectRole,
																	  const FLiveLinkSubjectKey& SubjectKey,
																	  const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
																	  UScriptStruct* MessageTypeInfo)
{
	check(MessageTypeInfo->IsChildOf(FLiveLinkBaseStaticData::StaticStruct()));

	FLiveLinkStaticDataStruct DataStruct(MessageTypeInfo);
	DataStruct.InitializeWith(MessageTypeInfo, reinterpret_cast<const FLiveLinkBaseStaticData*>(Context->GetMessage()));
	PushClientSubjectStaticData_AnyThread(SubjectKey, SubjectRole, MoveTemp(DataStruct));
}

void FLiveLinkMessageBusSource::InitializeAndPushFrameData_AnyThread(FName SubjectName,
																	 const FLiveLinkSubjectKey& SubjectKey,
																	 const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
																	 UScriptStruct* MessageTypeInfo)
{
	FLiveLinkFrameDataStruct DataStruct;

	if (MessageTypeInfo && MessageTypeInfo->IsChildOf(FLiveLinkSerializedFrameData::StaticStruct()))
	{
		// Extract the message from the compressed serialized data.
		const FLiveLinkSerializedFrameData* SerializedMessage = reinterpret_cast<const FLiveLinkSerializedFrameData*>(Context->GetMessage());
		FStructOnScope Payload;
		SerializedMessage->GetPayload(Payload);

		if (!ensure(Payload.IsValid()))
		{
			return;
		}

		// Special case: We want to send anim data.
		if (Payload.GetStruct() == FLiveLinkFloatAnimationFrameData::StaticStruct())
		{
			const FLiveLinkFloatAnimationFrameData* FloatAnimData = reinterpret_cast<const FLiveLinkFloatAnimationFrameData*>(Payload.GetStructMemory());
			FLiveLinkAnimationFrameData DoubleFrameData = FLiveLinkFloatAnimationFrameData::ToAnimData(*FloatAnimData);

			DataStruct.InitializeWith(FLiveLinkAnimationFrameData::StaticStruct(), &DoubleFrameData);
			DataStruct.GetBaseData()->WorldTime = DoubleFrameData.WorldTime.GetOffsettedTime();
		}
		else
		{
			check(Payload.GetStruct()->IsChildOf<FLiveLinkBaseFrameData>());

			const FLiveLinkBaseFrameData* Message = reinterpret_cast<const FLiveLinkBaseFrameData*>(Payload.GetStructMemory());
			DataStruct.InitializeWith((UScriptStruct*)Payload.GetStruct(), Message);
			DataStruct.GetBaseData()->WorldTime = Message->WorldTime.GetOffsettedTime();
		}
	}
	else if (MessageTypeInfo && MessageTypeInfo->IsChildOf(FLiveLinkBaseFrameData::StaticStruct()))
	{
		const FLiveLinkBaseFrameData* Message = reinterpret_cast<const FLiveLinkBaseFrameData*>(Context->GetMessage());
		DataStruct.InitializeWith(MessageTypeInfo, Message);
		DataStruct.GetBaseData()->WorldTime = Message->WorldTime.GetOffsettedTime();
	}
	else
	{
		static const FName NAME_InvalidFrameData = "LiveLinkMessageBusSource_InvalidFrameData";
		FLiveLinkLog::ErrorOnce(NAME_InvalidFrameData, SubjectKey, TEXT("Invalid frame data was provided for '%s' with connection '%s'"), *SubjectName.ToString(), *GetSourceMachineName().ToString());
		return;
	}

	// Push frame (sorted or not depending on the user setting)

	if (bCachedEnableFrameResequencing.load(std::memory_order_relaxed) && GLiveLinkMessageBusEnableResequencing)
	{
		ResequenceAndPushFrame(SubjectKey, MoveTemp(DataStruct));
	}
	else
	{
		PushClientSubjectFrameData_AnyThread(SubjectKey, MoveTemp(DataStruct));
	}
}

void FLiveLinkMessageBusSource::ResequenceAndPushFrame(const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct&& FrameData)
{
	FScopeLock Lock(&ResequencerCriticalSection);

	FFrameResequencer& Resequencer = SubjectResequencers.FindOrAdd(SubjectKey.SubjectName);
	const double CurrentTime = FPlatformTime::Seconds();

	// Update resequencer configuration from cached settings
	Resequencer.MaxBufferSize = CachedResequencerMaxBufferSize.load(std::memory_order_relaxed);
	Resequencer.MaxWaitTimeSeconds = CachedResequencerMaxWaitTimeSeconds.load(std::memory_order_relaxed);

	// Check if this is the next expected frame, and in that case deliver it immediately

	const FLiveLinkBaseFrameData* IncomingData = FrameData.GetBaseData();

	if (IncomingData && IncomingData->FrameId != INDEX_NONE)
	{
		const bool bIsFirstFrame = !Resequencer.LastDeliveredFrameId.IsSet();
		const bool bIsNextExpectedFrame = Resequencer.LastDeliveredFrameId.IsSet() &&
			UE::LiveLink::Private::IsNextFrameId(IncomingData->FrameId, Resequencer.LastDeliveredFrameId.GetValue());

		// Check if we need to resync the FrameId expectation
		const bool bShouldResync = Resequencer.LastDeliveredFrameId.IsSet() &&
			UE::LiveLink::Private::ShouldResyncFrameId(IncomingData->FrameId, Resequencer.LastDeliveredFrameId.GetValue(), Resequencer.MaxBufferSize);

		if (bIsFirstFrame || bIsNextExpectedFrame || bShouldResync)
		{
			const FQualifiedFrameTime& SceneTime = IncomingData->MetaData.SceneTime;

			if (bShouldResync)
			{
				UE_LOGF(LogLiveLink, Warning, "Resequencer [%ls] RESYNC: FrameId=%d (was %d), SceneTime=%d+%.4f - Flushing %d buffered frames",
					*SubjectKey.SubjectName.ToString(),
					IncomingData->FrameId,
					Resequencer.LastDeliveredFrameId.GetValue(),
					SceneTime.Time.GetFrame().Value,
					SceneTime.Time.GetSubFrame(),
					Resequencer.BufferedFrames.Num());

				// Flush old buffered frames on resync
				Resequencer.BufferedFrames.Empty();
				Resequencer.OldestFrameBufferTime = 0.0;
			}

			// Update tracking before delivering
			Resequencer.UpdateLastDelivered(FrameData);

			// Deliver immediately
			PushClientSubjectFrameData_AnyThread(SubjectKey, MoveTemp(FrameData));

			// Try to deliver any buffered frames that are now ready

			FLiveLinkFrameDataStruct ReadyFrame;

			while (Resequencer.TryPopNextFrame(ReadyFrame, CurrentTime))
			{
				const FLiveLinkBaseFrameData* BufferedData = ReadyFrame.GetBaseData();
				PushClientSubjectFrameData_AnyThread(SubjectKey, MoveTemp(ReadyFrame));
			}
			return;
		}
	}

	// Frame is out of order, so we buffer it while we wait for the right one.

	if (IncomingData)
	{
		const FQualifiedFrameTime& SceneTime = IncomingData->MetaData.SceneTime;
	}

	Resequencer.AddFrame(MoveTemp(FrameData), CurrentTime);

	// Track peak buffer depth
	Resequencer.PeakBufferDepth = FMath::Max(Resequencer.PeakBufferDepth, Resequencer.BufferedFrames.Num());

	// Try to deliver frames from buffer in order

	FLiveLinkFrameDataStruct ReadyFrame;

	while (Resequencer.TryPopNextFrame(ReadyFrame, CurrentTime))
	{
		const FLiveLinkBaseFrameData* BufferedData = ReadyFrame.GetBaseData();
		PushClientSubjectFrameData_AnyThread(SubjectKey, MoveTemp(ReadyFrame));
	}
}

double FLiveLinkMessageBusSource::GetDeadSourceTimeout() const
{
	return GetDefault<ULiveLinkSettings>()->GetMessageBusTimeBeforeRemovingDeadSource();
}

void FLiveLinkMessageBusSource::PushClientSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& SubjectKey,
																	  TSubclassOf<ULiveLinkRole> Role,
																	  FLiveLinkStaticDataStruct&& StaticData)
{
	Client->PushSubjectStaticData_AnyThread(SubjectKey, Role, MoveTemp(StaticData));
}

void FLiveLinkMessageBusSource::PushClientSubjectFrameData_AnyThread(const FLiveLinkSubjectKey& SubjectKey,
																	 FLiveLinkFrameDataStruct&& FrameData)
{
	Client->PushSubjectFrameData_AnyThread(SubjectKey, MoveTemp(FrameData));
}

void FLiveLinkMessageBusSource::HandleHeartbeat(const FLiveLinkHeartbeatMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UpdateConnectionLastActive();
}

void FLiveLinkMessageBusSource::HandleClearSubject(const FLiveLinkClearSubject& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UpdateConnectionLastActive();

	if (!Message.SubjectName.IsNone())
	{
		const FLiveLinkSubjectKey SubjectKey(SourceGuid, Message.SubjectName);

		// Flush any buffered frames for this subject before removing
		{
			FScopeLock Lock(&ResequencerCriticalSection);

			if (FFrameResequencer* Resequencer = SubjectResequencers.Find(Message.SubjectName))
			{
				TArray<FLiveLinkFrameDataStruct> RemainingFrames;
				Resequencer->Flush(RemainingFrames);

				for (FLiveLinkFrameDataStruct& Frame : RemainingFrames)
				{
					PushClientSubjectFrameData_AnyThread(SubjectKey, MoveTemp(Frame));
				}

				SubjectResequencers.Remove(Message.SubjectName);
			}
		}

		Client->RemoveSubject_AnyThread(SubjectKey);
	}
}

FORCEINLINE void FLiveLinkMessageBusSource::UpdateConnectionLastActive()
{
	FScopeLock ConnectionTimeLock(&ConnectionLastActiveSection);

	ConnectionLastActive = FPlatformTime::Seconds();
}

void FLiveLinkMessageBusSource::StartDiscovering()
{
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	if (bDiscovering)
	{
		return;
	}

	FLiveLinkMessageBusDiscoveryManager& DiscoveryManager = ILiveLinkModule::Get().GetMessageBusDiscoveryManager();
	DiscoveryManager.AddDiscoveryMessageRequest();
#endif
}

void FLiveLinkMessageBusSource::StopDiscovering()
{
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	if (!bDiscovering)
	{
		return;
	}

	FLiveLinkMessageBusDiscoveryManager& DiscoveryManager = ILiveLinkModule::Get().GetMessageBusDiscoveryManager();
	DiscoveryManager.RemoveDiscoveryMessageRequest();
#endif
}

void FLiveLinkMessageBusSource::SendConnectMessage()
{
	FLiveLinkConnectMessage* ConnectMessage = FMessageEndpoint::MakeMessage<FLiveLinkConnectMessage>();
	ConnectMessage->LiveLinkVersion = ILiveLinkClient::LIVELINK_VERSION;

	TMap<FName, FString> Annotations;
	AddAnnotations(Annotations);
	SendMessage(ConnectMessage, Annotations);
	StartHeartbeatEmitter();
	bIsValid = true;
	bIsShuttingDown = false;
}

bool FLiveLinkMessageBusSource::RequestSourceShutdown()
{
#if WITH_EDITOR
	if (ULiveLinkMessageBusSourceSettings* MessageBusSettings = SourceSettings.Get())
	{
		MessageBusSettings->OnPropertyChangedDelegate.RemoveAll(this);
	}
#endif

	if (!bInitialized)
	{
		return true;
	}

	StopDiscovering();

	FLiveLinkHeartbeatEmitter& HeartbeatEmitter = ILiveLinkModule::Get().GetHeartbeatEmitter();
	HeartbeatEmitter.StopHeartbeat(ConnectionAddress, MessageEndpoint);

	// Disable the Endpoint message handling since the message could keep it alive a bit.
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Disable();
	}
	MessageEndpoint.Reset();
	ConnectionAddress.Invalidate();

	RoleInstances.Empty();

	// Clean up resequencer
	{
		FScopeLock Lock(&ResequencerCriticalSection);
		SubjectResequencers.Empty();
	}

	bInitialized = false;
	bIsShuttingDown = true;

	return true;
}

const FName& FLiveLinkMessageBusSource::GetSourceName() const
{
	static FName Name(TEXT("LiveLinkMessageBusSource"));
	return Name;
}

void FLiveLinkMessageBusSource::InitializeMessageEndpoint(FMessageEndpointBuilder& EndpointBuilder)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MessageEndpoint = EndpointBuilder
	.Handling<FLiveLinkHeartbeatMessage>(this, &FLiveLinkMessageBusSource::HandleHeartbeat)
	.Handling<FLiveLinkClearSubject>(this, &FLiveLinkMessageBusSource::HandleClearSubject)
	.ReceivingOnAnyThread()
	.WithCatchall(this, &FLiveLinkMessageBusSource::InternalHandleMessage);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FLiveLinkMessageBusSource::FFrameResequencer::AddFrame(FLiveLinkFrameDataStruct&& Frame, double CurrentTime)
{
	// Check if frame is too late (already delivered a later frame)
	if (IsFrameTooLate(Frame))
	{
		DroppedLateFrameCount++;
		UE_LOGF(LogLiveLink, Verbose, "Dropped late frame (already delivered later frame)");
		return;
	}

	// Add to buffer
	FFrameResequencer::FBufferedFrame& NewEntry = BufferedFrames.AddDefaulted_GetRef();
	NewEntry.Frame = MoveTemp(Frame);
	NewEntry.BufferedAtSeconds = CurrentTime;

	// Sort by ordering tier
	BufferedFrames.StableSort([](const FFrameResequencer::FBufferedFrame& A, const FFrameResequencer::FBufferedFrame& B)
	{
		return UE::LiveLink::Private::IsFrameEarlierThan(A.Frame, B.Frame);
	});

	// Update oldest frame buffer time if this is the first frame
	if (BufferedFrames.Num() == 1)
	{
		OldestFrameBufferTime = CurrentTime;
	}
}

bool FLiveLinkMessageBusSource::FFrameResequencer::TryPopNextFrame(FLiveLinkFrameDataStruct& OutFrame, double CurrentTime)
{
	if (BufferedFrames.Num() == 0)
	{
		return false;
	}

	auto PopOldestByBufferedTime = [&]()
	{
		int32 OldestIndex = 0;
		double OldestTime = BufferedFrames[0].BufferedAtSeconds;
		for (int32 Index = 1; Index < BufferedFrames.Num(); ++Index)
		{
			if (BufferedFrames[Index].BufferedAtSeconds < OldestTime)
			{
				OldestTime = BufferedFrames[Index].BufferedAtSeconds;
				OldestIndex = Index;
			}
		}
		OutFrame = MoveTemp(BufferedFrames[OldestIndex].Frame);
		BufferedFrames.RemoveAt(OldestIndex);
	};

	auto UpdateOldestBufferTime = [&]()
	{
		if (BufferedFrames.Num() > 0)
		{
			double NewOldest = BufferedFrames[0].BufferedAtSeconds;
			for (int32 Index = 1; Index < BufferedFrames.Num(); ++Index)
			{
				NewOldest = FMath::Min(NewOldest, BufferedFrames[Index].BufferedAtSeconds);
			}
			OldestFrameBufferTime = NewOldest;
		}
		else
		{
			OldestFrameBufferTime = 0.0;
		}
	};

	// Check for buffer overflow
	if (BufferedFrames.Num() >= MaxBufferSize)
	{
		UE_LOGF(LogLiveLink, Verbose, "Resequencer buffer overflow (%d frames), flushing oldest", BufferedFrames.Num());

		PopOldestByBufferedTime();

		UpdateLastDelivered(OutFrame);

		UpdateOldestBufferTime();

		ReorderedFrameCount++;

		return true;
	}

	// Check for timeout
	if (CurrentTime - OldestFrameBufferTime > MaxWaitTimeSeconds)
	{
		UE_LOGF(LogLiveLink, Verbose, "Resequencer timeout (%.3fs), flushing %d frames", CurrentTime - OldestFrameBufferTime, BufferedFrames.Num());

		PopOldestByBufferedTime();

		UpdateLastDelivered(OutFrame);

		UpdateOldestBufferTime();

		return true;
	}

	// Deliver buffered frame only if it's the next expected frame
	// Rely on timeout/overflow handling above for frames that are truly lost
	if (BufferedFrames.Num() > 0)
	{
		const FLiveLinkBaseFrameData* OldestData = BufferedFrames[0].Frame.GetBaseData();
		if (OldestData && OldestData->FrameId != INDEX_NONE)
		{
			const bool bIsNextExpected = LastDeliveredFrameId.IsSet() &&
				UE::LiveLink::Private::IsNextFrameId(OldestData->FrameId, LastDeliveredFrameId.GetValue());

			if (bIsNextExpected)
			{
				OutFrame = MoveTemp(BufferedFrames[0].Frame);
				BufferedFrames.RemoveAt(0);
				UpdateLastDelivered(OutFrame);
				UpdateOldestBufferTime();
				ReorderedFrameCount++;
				return true;
			}
		}
	}

	return false;
}

void FLiveLinkMessageBusSource::FFrameResequencer::Flush(TArray<FLiveLinkFrameDataStruct>& OutFrames)
{
	OutFrames.Empty(BufferedFrames.Num());
	for (FFrameResequencer::FBufferedFrame& BufferedFrame : BufferedFrames)
	{
		OutFrames.Add(MoveTemp(BufferedFrame.Frame));
	}
	BufferedFrames.Empty();
	OldestFrameBufferTime = 0.0;
}

bool FLiveLinkMessageBusSource::FFrameResequencer::IsFrameTooLate(const FLiveLinkFrameDataStruct& Frame) const
{
	const FLiveLinkBaseFrameData* Data = Frame.GetBaseData();
	if (!Data)
	{
		return false;
	}

	// Check if frame arrived too late based on FrameId (with wraparound support)
	if (LastDeliveredFrameId.IsSet() && Data->FrameId != INDEX_NONE)
	{
		// If gap is suspiciously large (>2x buffer), allow resync instead of dropping
		// This handles wraparound recovery (e.g., LastId=INT_MAX-5, IncomingId=1)
		if (UE::LiveLink::Private::ShouldResyncFrameId(Data->FrameId, LastDeliveredFrameId.GetValue(), MaxBufferSize))
		{
			return false; // Don't drop - allow this frame to trigger resync
		}

		return UE::LiveLink::Private::IsFrameIdEarlier(Data->FrameId, LastDeliveredFrameId.GetValue());
	}

	return false;
}

void FLiveLinkMessageBusSource::FFrameResequencer::UpdateLastDelivered(const FLiveLinkFrameDataStruct& Frame)
{
	const FLiveLinkBaseFrameData* Data = Frame.GetBaseData();
	if (!Data)
	{
		return;
	}

	// Update last delivered FrameId
	if (Data->FrameId != INDEX_NONE)
	{
		LastDeliveredFrameId = Data->FrameId;
	}
}

void FLiveLinkMessageBusSource::AddAnnotations(TMap<FName, FString>& InOutAnnotations) const
{
	if (CVarLiveLinkSupportCompressPayloads.GetValueOnAnyThread())
	{
		// The presence of this flag in the annotation will inform our provider that we support receiving compressed animation.
		InOutAnnotations.Add({ FLiveLinkMessageAnnotation::CompressedPayloadSupport, TEXT("") });
	}
}
