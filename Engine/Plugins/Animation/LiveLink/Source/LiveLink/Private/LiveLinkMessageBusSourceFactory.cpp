// Copyright Epic Games, Inc. All Rights Reserved.
#include "LiveLinkMessageBusSourceFactory.h"

#include "ILiveLinkClient.h"
#include "LiveLinkMessageBusSource.h"

#include "SLiveLinkMessageBusSourceFactory.h"

#include "Features/IModularFeatures.h"
#include "Misc/MessageDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkMessageBusSourceFactory)

#define LOCTEXT_NAMESPACE "LiveLinkMessageBusSourceFactory"

FText ULiveLinkMessageBusSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "Message Bus Source");
}

FText ULiveLinkMessageBusSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Creates a connection to a Message Bus based Live Link Source");
}

TSharedPtr<SWidget> ULiveLinkMessageBusSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SLiveLinkMessageBusSourceFactory)
		.OnSourceSelected(FOnLiveLinkMessageBusSourceSelected::CreateUObject(this, &ULiveLinkMessageBusSourceFactory::OnSourceSelected, InOnLiveLinkSourceCreated))
		.FactoryClass(GetClass());
}

TSharedPtr<FLiveLinkMessageBusSource> ULiveLinkMessageBusSourceFactory::MakeSource(const FText& Name,
																				   const FText& MachineName,
																				   const FMessageAddress& Address,
																				   double TimeOffset) const
{
	return MakeShared<FLiveLinkMessageBusSource>(Name, MachineName, Address, TimeOffset);
}

TSharedPtr<ILiveLinkSource> ULiveLinkMessageBusSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FString Name;
	if (!FParse::Value(*ConnectionString, TEXT("Name="), Name))
	{
		return TSharedPtr<ILiveLinkSource>();
	}

	const double TimeOffset = 0.0;
	return MakeSource(FText::FromString(Name), FText::GetEmpty(), FMessageAddress(), TimeOffset);
}


FString ULiveLinkMessageBusSourceFactory::CreateConnectionString(const FProviderPollResult& Result)
{
	return FString::Printf(TEXT("Name=\"%s\""), *Result.Name);
}

void ULiveLinkMessageBusSourceFactory::OnSourceSelected(FProviderPollResultPtr SelectedSource, FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	if (SelectedSource.IsValid())
	{
#if WITH_EDITOR
		bool bDoesAlreadyExist = false;
		{
			ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
			TArray<FGuid> Sources = LiveLinkClient.GetSources();
			for (FGuid SourceGuid : Sources)
			{
				if (LiveLinkClient.GetSourceType(SourceGuid).ToString() == SelectedSource->Name)
				{
					bDoesAlreadyExist = true;
					break;
				}
			}
		}

		if (bDoesAlreadyExist)
		{
			if (EAppReturnType::No == FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, LOCTEXT("AddSourceWithSameName", "This provider name already exist. Are you sure you want to add a new one?")))
			{
				return;
			}
		}
#endif

		TSharedPtr<FLiveLinkMessageBusSource> SharedPtr = MakeSource(FText::FromString(SelectedSource->Name), FText::FromString(SelectedSource->MachineName), SelectedSource->Address, SelectedSource->MachineTimeOffset);
		FString ConnectionString = CreateConnectionString(*SelectedSource.Get());
		InOnLiveLinkSourceCreated.ExecuteIfBound(SharedPtr, MoveTemp(ConnectionString));
	}
}

#undef LOCTEXT_NAMESPACE
