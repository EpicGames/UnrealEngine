// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMessageBusSourceSettings.h"

#include "LiveLinkSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkMessageBusSourceSettings)

ULiveLinkMessageBusSourceSettings::ULiveLinkMessageBusSourceSettings()
{
	Mode = GetDefault<ULiveLinkSettings>()->DefaultMessageBusSourceMode;
}

