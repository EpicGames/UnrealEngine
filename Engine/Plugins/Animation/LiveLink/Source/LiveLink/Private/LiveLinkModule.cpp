// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkModule.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "LiveLinkDebugCommand.h"
#include "LiveLinkLogInstance.h"
#include "LiveLinkHeartbeatEmitter.h"
#include "LiveLinkPreset.h"
#include "LiveLinkMessageBusDiscoveryManager.h"
#include "LiveLinkSettings.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

LLM_DEFINE_TAG(LiveLink);
#define LOCTEXT_NAMESPACE "LiveLinkModule"


namespace LiveLinkModuleUtils
{
	FString InPluginContent(const FString& RelativePath, const ANSICHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("LiveLink"))->GetContentDir();
		return (ContentDir / RelativePath) + Extension;
	}
}


#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( LiveLinkModuleUtils::InPluginContent( RelativePath, ".svg" ), __VA_ARGS__ )

FLiveLinkClient* FLiveLinkModule::LiveLinkClient_AnyThread = nullptr;

FLiveLinkModule::FLiveLinkModule()
	: LiveLinkClient()
	, LiveLinkMotionController(LiveLinkClient)
	, HeartbeatEmitter(MakeUnique<FLiveLinkHeartbeatEmitter>())
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	, DiscoveryManager(MakeUnique<FLiveLinkMessageBusDiscoveryManager>())
#endif
	, LiveLinkDebugCommand(MakeUnique<FLiveLinkDebugCommand>(LiveLinkClient))
{
}

void FLiveLinkModule::StartupModule()
{
	LLM_SCOPE_BYTAG(LiveLink);
	FLiveLinkLogInstance::CreateInstance();
	CreateStyle();

	const bool bUseModularClientReference = GConfig->GetBoolOrDefault(
		TEXT("LiveLink"), TEXT("bUseModularClientReference"), false, GEngineIni);

	if (!bUseModularClientReference)
	{
		FPlatformAtomics::InterlockedExchangePtr((void**)&LiveLinkClient_AnyThread, &LiveLinkClient);
		IModularFeatures::Get().RegisterModularFeature(FLiveLinkClient::ModularFeatureName, &LiveLinkClient);
	}

	LiveLinkMotionController.RegisterController();

	//Register for engine initialization completed so we can load default preset if any. Presets could depend on plugins loaded at a later stage.
	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FLiveLinkModule::OnEngineLoopInitComplete);
}

void FLiveLinkModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(LiveLink);
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

	HeartbeatEmitter->Exit();
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	DiscoveryManager->Stop();
#endif
	LiveLinkMotionController.UnregisterController();

	const bool bUseModularClientReference = GConfig->GetBoolOrDefault(
		TEXT("LiveLink"), TEXT("bUseModularClientReference"), false, GEngineIni);

	if (!bUseModularClientReference)
	{
		IModularFeatures::Get().UnregisterModularFeature(FLiveLinkClient::ModularFeatureName, &LiveLinkClient);
		FPlatformAtomics::InterlockedExchangePtr((void**)&LiveLinkClient_AnyThread, nullptr);
	}

	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
	FLiveLinkLogInstance::DestroyInstance();
}

FDelegateHandle FLiveLinkModule::RegisterMessageBusSourceFilter(const FOnLiveLinkShouldDisplaySource& Delegate)
{
	FDelegateHandle Handle = Delegate.GetHandle();
	RegisteredSourceFilters.FindOrAdd(Handle) = Delegate;
	return Handle;
}

void FLiveLinkModule::UnregisterMessageBusSourceFilter(FDelegateHandle Handle)
{
	RegisteredSourceFilters.Remove(Handle);
}

void FLiveLinkModule::CreateStyle()
{
	static FName LiveLinkStyle(TEXT("LiveLinkCoreStyle"));
	StyleSet = MakeShared<FSlateStyleSet>(LiveLinkStyle);
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

	FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("LiveLink"))->GetContentDir();

	const FVector2D Icon16x16(16.0f, 16.0f);

	StyleSet->Set("LiveLinkIcon", new FSlateImageBrush((ContentDir / TEXT("LiveLink_16x")) + TEXT(".png"), Icon16x16));

	const FSlateColor ValidColor = FSlateColor(FLinearColor(0.0146f, 0.7874f, 0.0736f));
	const FSlateColor InvalidColor = FSlateColor(FLinearColor(1.0f, 0.4654f, 0.0f));
	const FSlateColor ErrorColor = FSlateColor(FLinearColor(0.8524f, 0.0372f, 0.0372f));
	const FSlateColor PausedColor = FSlateColor(FLinearColor(0.2159f, 0.2159f, 0.2159f));


	StyleSet->Set("LiveLink.Color.Valid", ValidColor);
	StyleSet->Set("LiveLink.Color.Invalid", InvalidColor);
	StyleSet->Set("LiveLink.Color.Error", ErrorColor);
	StyleSet->Set("LiveLink.Color.Paused", PausedColor);

	StyleSet->Set("LiveLink.Subject.Okay", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Checkmark"), Icon16x16, ValidColor));
	StyleSet->Set("LiveLink.Subject.Warning", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Warning"), Icon16x16, InvalidColor));
	StyleSet->Set("LiveLink.Subject.Pause", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Pause"), Icon16x16, PausedColor));
	StyleSet->Set("LiveLink.Subject.Error", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Error"), Icon16x16, ErrorColor));
}

void FLiveLinkModule::OnEngineLoopInitComplete()
{
	ULiveLinkPreset* StartupPreset = nullptr;
	const FString& CommandLine = FCommandLine::Get();
	const TCHAR* PresetStr = TEXT("LiveLink.Preset.Apply Preset="); // expected inside an -ExecCmds="" argument. So our command should end either on ',' or '"'.
	const int32 CommandStartPos = CommandLine.Find(PresetStr);

	if (CommandStartPos != INDEX_NONE)
	{
		int32 PresetEndPos = CommandLine.Find(",", ESearchCase::IgnoreCase, ESearchDir::FromStart, CommandStartPos);
		const int32 NextDoubleQuotesPos = CommandLine.Find("\"", ESearchCase::IgnoreCase, ESearchDir::FromStart, CommandStartPos);

		if ((PresetEndPos != INDEX_NONE) && (NextDoubleQuotesPos != INDEX_NONE))
		{
			PresetEndPos = FMath::Min(PresetEndPos, NextDoubleQuotesPos);
		}
		else if (NextDoubleQuotesPos != INDEX_NONE)
		{
			PresetEndPos = NextDoubleQuotesPos;
		}

		if (PresetEndPos != INDEX_NONE)
		{
			const int32 PresetStartPos = CommandStartPos + FCString::Strlen(PresetStr);
			if (CommandLine.IsValidIndex(PresetStartPos) && CommandLine.IsValidIndex(PresetEndPos))
			{
				const FString LiveLinkPresetName = CommandLine.Mid(PresetStartPos, PresetEndPos - PresetStartPos);
				StartupPreset = Cast<ULiveLinkPreset>(StaticLoadObject(ULiveLinkPreset::StaticClass(), nullptr, *LiveLinkPresetName));
			}
		}
	}

	if (StartupPreset == nullptr)
	{
		StartupPreset = GetDefault<ULiveLinkSettings>()->DefaultLiveLinkPreset.LoadSynchronous();
	}

	if (StartupPreset)
	{
		StartupPreset->ApplyToClientLatent();
	}
}

IMPLEMENT_MODULE(FLiveLinkModule, LiveLink);

#undef LOCTEXT_NAMESPACE
