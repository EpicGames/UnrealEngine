// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkModule.h"


#include "LiveLinkMotionController.h"


class FLiveLinkDebugCommand;

/**
 * Implements the Messaging module.
 */

 struct FLiveLinkClientReference;

class FLiveLinkModule : public ILiveLinkModule
{
public:
	FLiveLinkModule();

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override { return false; }
	//~ End IModuleInterface interface

	//~ Begin ILiveLinkModule interface
	virtual FLiveLinkHeartbeatEmitter& GetHeartbeatEmitter() override { return *HeartbeatEmitter; }
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	virtual FLiveLinkMessageBusDiscoveryManager& GetMessageBusDiscoveryManager() override { return *DiscoveryManager; }
#endif
	virtual TSharedPtr<FSlateStyleSet> GetStyle() override { return StyleSet; }

	virtual FDelegateHandle RegisterMessageBusSourceFilter(const FOnLiveLinkShouldDisplaySource& SourceFilter) override;
	virtual void UnregisterMessageBusSourceFilter(FDelegateHandle Handle) override;

	virtual const TMap<FDelegateHandle, FOnLiveLinkShouldDisplaySource>& GetSourceFilters() const override
	{
		return RegisteredSourceFilters;
	}

	virtual FOnSubjectOutboundNameModified& OnSubjectOutboundNameModified() override
	{
		return OutboundNameModifiedDelegate;
	}

	//~ End ILiveLinkModule interface

private:
	void CreateStyle();
	void OnEngineLoopInitComplete();

private:
	friend FLiveLinkClientReference;
	static FLiveLinkClient* LiveLinkClient_AnyThread;

	FLiveLinkClient LiveLinkClient;
	FLiveLinkMotionController LiveLinkMotionController;

	TSharedPtr<FSlateStyleSet> StyleSet;

	TUniquePtr<FLiveLinkHeartbeatEmitter> HeartbeatEmitter;
	TUniquePtr<FLiveLinkMessageBusDiscoveryManager> DiscoveryManager;
	TUniquePtr<FLiveLinkDebugCommand> LiveLinkDebugCommand;

	// Filters registered externally used to filter what message bus sources should be displayed in the UI.
	TMap<FDelegateHandle, FOnLiveLinkShouldDisplaySource> RegisteredSourceFilters;

	// Delegate called by LiveLinkHubSubjectSettings to notify the hub of an outbound name change.
	FOnSubjectOutboundNameModified OutboundNameModifiedDelegate;
};
