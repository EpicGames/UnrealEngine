// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeatures.h"
#include "IMotionController.h"
#include "LiveLinkClient.h"

#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"


#define LOCTEXT_NAMESPACE "LiveLinkMotionController"


class FLiveLinkMotionController : public IMotionController
{
	// Internal structure for caching enumerated data
	struct FLiveLinkMotionControllerEnumeratedSource
	{
		// Subject key for talking to live link
		FLiveLinkSubjectKey SubjectKey;

		// MotionSource name for interacting with Motion Controller system
		FName MotionSource;

		FLiveLinkMotionControllerEnumeratedSource(const FLiveLinkSubjectKey& Key, FName MotionSourceName) : SubjectKey(Key), MotionSource(MotionSourceName) {}
	};

	// Built array of Live Link Sources to give to Motion Controller system
	TArray<FLiveLinkMotionControllerEnumeratedSource> EnumeratedSources;

public:
	FLiveLinkMotionController(FLiveLinkClient& InClient) : Client(InClient)
	{ 
		BuildSourceData();
		OnSubjectsChangedHandle = Client.OnLiveLinkSubjectsChanged().AddRaw(this, &FLiveLinkMotionController::OnSubjectsChangedHandler);
		WildcardSource = FGuid::NewGuid();
	}

	~FLiveLinkMotionController()
	{
		Client.OnLiveLinkSubjectsChanged().Remove(OnSubjectsChangedHandle);
		OnSubjectsChangedHandle.Reset();
	}

	void RegisterController()
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	void UnregisterController()
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	}

	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float) const override
	{
		FLiveLinkSubjectKey SubjectKey = GetSubjectKeyFromMotionSource(MotionSource);

		FLiveLinkSubjectFrameData FrameData;
		if (Client.EvaluateFrame_AnyThread(SubjectKey.SubjectName, ULiveLinkTransformRole::StaticClass(), FrameData))
		{
			if (FLiveLinkTransformFrameData* TransformFrameData = FrameData.FrameData.Cast<FLiveLinkTransformFrameData>())
			{
				OutPosition = TransformFrameData->Transform.GetLocation();
				OutOrientation = TransformFrameData->Transform.GetRotation().Rotator();
				return true;
			}
		}
		return false;
	}

	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override
	{
		OutbProvidedLinearVelocity = false;
		OutbProvidedAngularVelocity = false;
		OutbProvidedLinearAcceleration = false;
		return GetControllerOrientationAndPosition(ControllerIndex, MotionSource, OutOrientation, OutPosition, WorldToMetersScale);
	}
	
	virtual bool GetControllerOrientationAndPositionForTime(const int32 ControllerIndex, const FName MotionSource, FTimespan Time, bool& OutTimeWasUsed, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override
	{
		OutTimeWasUsed = false;
		OutbProvidedLinearVelocity = false;
		OutbProvidedAngularVelocity = false;
		OutbProvidedLinearAcceleration = false;
		return GetControllerOrientationAndPosition(ControllerIndex, MotionSource, OutOrientation, OutPosition, WorldToMetersScale);
	}

	float GetCustomParameterValue(const FName MotionSource, FName ParameterName, bool& bValueFound) const override
	{
		FLiveLinkSubjectKey SubjectKey = GetSubjectKeyFromMotionSource(MotionSource);

		FLiveLinkSubjectFrameData EvaluatedData;
		if (Client.EvaluateFrame_AnyThread(SubjectKey.SubjectName, ULiveLinkBasicRole::StaticClass(), EvaluatedData))
		{
			if (EvaluatedData.FrameData.GetBaseData())
			{
				FLiveLinkBaseFrameData& BaseFrameData = *EvaluatedData.FrameData.GetBaseData();

				float FoundValue = 0.f;
				if (EvaluatedData.StaticData.GetBaseData()->FindPropertyValue(BaseFrameData, ParameterName, FoundValue))
				{
					bValueFound = true;
					return FoundValue;
				}
			}
		}

		bValueFound = false;
		return 0.f;
	}

	virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const override
	{
		FLiveLinkSubjectKey SubjectKey = GetSubjectKeyFromMotionSource(MotionSource);

		FLiveLinkSubjectFrameData EvaluatedData;
		if (Client.EvaluateFrame_AnyThread(SubjectKey.SubjectName, ULiveLinkBasicRole::StaticClass(), EvaluatedData))
		{
			return ETrackingStatus::Tracked;
		}
		return ETrackingStatus::NotTracked;
	}


	virtual FName GetMotionControllerDeviceTypeName() const override
	{
		static FName LiveLinkMotionControllerName(TEXT("LiveLinkMotionController"));
		return LiveLinkMotionControllerName;
	}

	// Builds cached source data for passing to motion controller system
	void BuildSourceData()
	{
		TArray<FLiveLinkSubjectKey> SubjectKeys = Client.GetSubjects(true, true);

		TMap<FGuid, TArray<FName>> LiveLinkSubjects;
		LiveLinkSubjects.Add(WildcardSource);

		for (const FLiveLinkSubjectKey& Subject : SubjectKeys)
		{
			LiveLinkSubjects.FindOrAdd(Subject.Source).Add(Subject.SubjectName);
			LiveLinkSubjects.FindChecked(WildcardSource).Add(Subject.SubjectName);
		}

		TArray<FGuid> SourceGuids;
		LiveLinkSubjects.GenerateKeyArray(SourceGuids);

		typedef TPair<FGuid, FText> FHeaderEntry;
		TArray<FHeaderEntry> Headers;
		Headers.Reserve(SourceGuids.Num());

		for (const FGuid& Source : SourceGuids)
		{
			FText SourceName = (Source == WildcardSource) ? LOCTEXT("LiveLinkAnySource", "Any") : Client.GetSourceType(Source);
			Headers.Emplace(Source, SourceName);
		}

		{
			FGuid& CaptureWildcardSource = WildcardSource;
			Headers.Sort([CaptureWildcardSource](const FHeaderEntry& A, const FHeaderEntry& B) { return A.Key == CaptureWildcardSource || A.Value.CompareToCaseIgnored(B.Value) <= 0; });
		}

		//Build EnumeratedSources data
		EnumeratedSources.Reset();
		EnumeratedSources.Reserve(SubjectKeys.Num());

		for (const FHeaderEntry& Header : Headers)
		{
			TArray<FName> Subjects = LiveLinkSubjects.FindChecked(Header.Key);
			Subjects.Sort(FNameLexicalLess());
			for (FName Subject : Subjects)
			{
				FName FullName = *FString::Format(TEXT("{0} ({1})"), { Subject.ToString(), Header.Value.ToString() });
				EnumeratedSources.Emplace(FLiveLinkSubjectKey(Header.Key, Subject), FullName);
			}
		}
	}

	virtual void EnumerateSources(TArray<FMotionControllerSource>& Sources) const override
	{
		for (const FLiveLinkMotionControllerEnumeratedSource& Source : EnumeratedSources)
		{
			FMotionControllerSource SourceDesc(Source.MotionSource);
#if WITH_EDITOR
			static const FName LiveLinkCategoryName(TEXT("LiveLink"));
			SourceDesc.EditorCategory = LiveLinkCategoryName;
#endif
			Sources.Add(SourceDesc);
		}
	}

	virtual bool GetHandJointPosition(const FName MotionSource, int jointIndex, FVector& OutPosition) const override { return false; }

private:
	FLiveLinkSubjectKey GetSubjectKeyFromMotionSource(FName MotionSource) const
	{
		const FLiveLinkMotionControllerEnumeratedSource* EnumeratedSource = EnumeratedSources.FindByPredicate([&](const FLiveLinkMotionControllerEnumeratedSource& Item) { return Item.MotionSource == MotionSource; });
		if (EnumeratedSource)
		{
			return EnumeratedSource->SubjectKey;
		}
		return FLiveLinkSubjectKey(FGuid(), MotionSource);
	}

	// Registered with the client and called when client's subjects change
	void OnSubjectsChangedHandler() { BuildSourceData(); }

	// Reference to the live link client
	FLiveLinkClient& Client;

	// Handle to delegate registered with client so we can update when subject state changes
	FDelegateHandle OnSubjectsChangedHandle;

	// Wildcard source, we don't care about the source itself, just the subject name
	FGuid WildcardSource;
};

#undef LOCTEXT_NAMESPACE
