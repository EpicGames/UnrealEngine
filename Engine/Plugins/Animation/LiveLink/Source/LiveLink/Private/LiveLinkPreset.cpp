// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPreset.h"

#include "LiveLinkPresetTypes.h"
#include "Misc/CoreDelegates.h"
#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "LatentActions.h"
#include "LiveLinkClient.h"
#include "LiveLinkLog.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkPreset)


namespace
{
	static FAutoConsoleCommand LiveLinkPresetApplyCmd(
		TEXT("LiveLink.Preset.Apply"),
		TEXT("Apply a LiveLinkPreset. Use: LiveLink.Preset.Apply Preset=/Game/Folder/MyLiveLinkPreset.MyLiveLinkPreset"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				//LiveLinkModule now looks for this commandline argument when starting up. No need to execute it twice at launch
				if (GFrameCounter > 1)
				{
					for (const FString& Element : Args)
					{
						const TCHAR* PresetStr = TEXT("Preset=");
						const int32 FoundElement = Element.Find(PresetStr);
						if (FoundElement != INDEX_NONE)
						{
							UObject* Object = StaticLoadObject(ULiveLinkPreset::StaticClass(), nullptr, *Element + FoundElement + FCString::Strlen(PresetStr));
							if (Object)
							{
								CastChecked<ULiveLinkPreset>(Object)->ApplyToClientLatent();
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand LiveLinkPresetAddCmd(
		TEXT("LiveLink.Preset.Add"),
		TEXT("Add a LiveLinkPreset. Use: LiveLink.Preset.Add Preset=/Game/Folder/MyLiveLinkPreset.MyLiveLinkPreset"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				for (const FString& Element : Args)
				{
					const TCHAR* PresetStr = TEXT("Preset=");
					const int32 FoundElement = Element.Find(PresetStr);
					if (FoundElement != INDEX_NONE)
					{
						UObject* Object = StaticLoadObject(ULiveLinkPreset::StaticClass(), nullptr, *Element + FoundElement + FCString::Strlen(PresetStr));
						if (Object)
						{
							CastChecked<ULiveLinkPreset>(Object)->AddToClient();
						}
					}
				}
			})
		);

	static FAutoConsoleCommand LiveLinkPresetClearCmd(
		TEXT("LiveLink.Preset.Clear"),
		TEXT("Remove all LiveLink sources from the client"),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
				{
					FLiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName);
					LiveLinkClient.RemoveAllSources();
					LiveLinkClient.Tick();
					FLiveLinkLog::Info(TEXT("Removed all LiveLink sources"));
				}
				else
				{
					FLiveLinkLog::Error(TEXT("LiveLink client not available"));
				}
			})
		);
}

struct FApplyToClientPollingOperation
{
	/** Holds a weak pointer to the preset that will be applied. */
	TWeakObjectPtr<ULiveLinkPreset> WeakPreset;
	/** Keeps track of remaining time before aborting  */
	double RemainingTime = 1.0;
	/** Keeps track of the last time the Update function was called. */
	double LastTimeSinceUpdate = 0.0;
	
	/** Result of a call to Update() */
	enum class EApplyToClientUpdateResult : uint8
	{
		Success, // Preset was successfully applied.
		Failure, // Preset could not be applied.
		Pending  // Operation is still ongoing.
	};

	FApplyToClientPollingOperation(ULiveLinkPreset* Preset)
		: WeakPreset(Preset)
	{
	}
	
	EApplyToClientUpdateResult Update()
	{
		bool bResult = true;
        
        ULiveLinkPreset* Preset = WeakPreset.Get();
        if (!Preset || !IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
        {
        	FLiveLinkLog::Error(TEXT("Could not apply preset"));
        	return EApplyToClientUpdateResult::Failure;
        }
        
        FLiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName);

        if (LastTimeSinceUpdate == 0.0)
        {
        	// Start the process of removing sources.
        	LiveLinkClient.RemoveAllSources();
        	LastTimeSinceUpdate = FApp::GetCurrentTime();
        }

        LiveLinkClient.Tick();

        constexpr bool bEvenIfPendingKill = true;
        if (LiveLinkClient.GetSources(bEvenIfPendingKill).Num() == 0)
        {
        	for (const FLiveLinkSourcePreset& SourcePreset : Preset->GetSourcePresets())
        	{
        		bResult &= LiveLinkClient.CreateSource(SourcePreset);
        	}

        	for (const FLiveLinkSubjectPreset& SubjectPreset : Preset->GetSubjectPresets())
        	{
        		bResult &= LiveLinkClient.CreateSubject(SubjectPreset);
        	}
        
        	if (bResult)
        	{
        		FLiveLinkLog::Info(TEXT("Applied '%s'"), *Preset->GetFullName());
        	}
        	else
        	{
        		FLiveLinkLog::Error(TEXT("Could not apply '%s'"), *Preset->GetFullName());
        	}

        	return bResult ? EApplyToClientUpdateResult::Success : EApplyToClientUpdateResult::Failure;
        }

        RemainingTime -= FApp::GetCurrentTime() - LastTimeSinceUpdate;
        if (RemainingTime <= 0.0)
        {
        	return EApplyToClientUpdateResult::Failure;
        }
        
        LastTimeSinceUpdate = FApp::GetCurrentTime();
		return EApplyToClientUpdateResult::Pending;
	}
};

TPimplPtr<FApplyToClientPollingOperation> ULiveLinkPreset::ApplyToClientPollingOperation;

class FApplyToClientLatentAction : public FPendingLatentAction
{
public:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	FApplyToClientPollingOperation ApplyOperation;
	
	FApplyToClientLatentAction(const FLatentActionInfo& InLatentInfo, ULiveLinkPreset* InOwnerPreset)
        : ExecutionFunction(InLatentInfo.ExecutionFunction)
        , OutputLink(InLatentInfo.Linkage)
        , CallbackTarget(InLatentInfo.CallbackTarget)
		, ApplyOperation(InOwnerPreset)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		if (ApplyOperation.Update() != FApplyToClientPollingOperation::EApplyToClientUpdateResult::Pending)
		{
			Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
		}
	}
};

ULiveLinkPreset::~ULiveLinkPreset()
{
	ClearApplyToClientTimer();
}

bool ULiveLinkPreset::ApplyToClient() const
{
	bool bResult = false;
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		FLiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		LiveLinkClient.RemoveAllSources();
		LiveLinkClient.Tick();

		bResult = true;
		for (const FLiveLinkSourcePreset& SourcePreset : Sources)
		{
			bResult &= LiveLinkClient.CreateSource(SourcePreset);
		}

		for (const FLiveLinkSubjectPreset& SubjectPreset : Subjects)
		{
			bResult &= LiveLinkClient.CreateSubject(SubjectPreset);
		}
	}

	if (bResult)
	{
		FLiveLinkLog::Info(TEXT("Applied '%s'"), *GetFullName());
	}
	else
	{
		FLiveLinkLog::Error(TEXT("Could not apply '%s'"), *GetFullName());
	}

	return bResult;
}

void ULiveLinkPreset::ApplyToClientLatent(UObject* WorldContextObject, FLatentActionInfo LatentInfo)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (!LatentActionManager.FindExistingAction<FApplyToClientLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID))
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FApplyToClientLatentAction(LatentInfo, this));
		}
	}
}

void ULiveLinkPreset::ApplyToClientLatent(TFunction<void(bool)> CompletionCallback)
{
	if (ApplyToClientPollingOperation.IsValid())
	{
		FLiveLinkLog::Error(TEXT("Could not apply '%s', operation is already in progress."), *GetFullName());
		if (CompletionCallback != nullptr)
		{
			CompletionCallback(false);
		}
		return;
	}

	ApplyToClientPollingOperation = MakePimpl<FApplyToClientPollingOperation>(this);

	
	ApplyToClientEndFrameHandle = FCoreDelegates::OnEndFrame.AddLambda([WeakThis = TWeakObjectPtr<ULiveLinkPreset>(this), WrappedCallback = MoveTemp(CompletionCallback), Count = ApplyCount]()
	{
		if (ensure(WeakThis->ApplyToClientPollingOperation))
		{
			FApplyToClientPollingOperation::EApplyToClientUpdateResult Result = WeakThis->ApplyToClientPollingOperation->Update();
			if (Result != FApplyToClientPollingOperation::EApplyToClientUpdateResult::Pending)
			{
				WeakThis->ApplyToClientPollingOperation.Reset();
				
				if (WrappedCallback != nullptr)
				{
					WrappedCallback(Result == FApplyToClientPollingOperation::EApplyToClientUpdateResult::Success ? true : false);
				}

				UE_LOGF(LogTemp, Verbose, "Clearing preset %ls (%d)", *WeakThis->GetFullName(), Count);

				WeakThis->ClearApplyToClientTimer();
			}
		}
	});

	UE_LOGF(LogTemp, Verbose, "Applying preset %ls (%d)", *GetFullName(), ApplyCount++);
}

void ULiveLinkPreset::CancelLatentAction()
{
	ClearApplyToClientTimer();
	ApplyToClientPollingOperation.Reset();
}

bool ULiveLinkPreset::AddToClient(const bool bRecreatePresets) const
{
	bool bResult = false;
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		FLiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		TArray<FGuid> AllSources;
		AllSources.Append(LiveLinkClient.GetSources());
		AllSources.Append(LiveLinkClient.GetVirtualSources());
		const TArray<FLiveLinkSubjectKey> AllSubjects = LiveLinkClient.GetSubjects(true, true);

		TSet<FGuid> FoundSources;
		TSet<FLiveLinkSubjectKey> FoundSubjects;

		for (const FLiveLinkSourcePreset& SourcePreset : Sources)
		{
			if (AllSources.Contains(SourcePreset.Guid))
			{
				if (bRecreatePresets)
				{
					LiveLinkClient.RemoveSource(SourcePreset.Guid);
				}
				else
				{
					FoundSources.Add(SourcePreset.Guid);
				}
			}
		}

		for (const FLiveLinkSubjectPreset& SubjectPreset : Subjects)
		{
			if (AllSubjects.Contains(SubjectPreset.Key))
			{
				if (bRecreatePresets)
				{
					LiveLinkClient.RemoveSubject_AnyThread(SubjectPreset.Key);
				}
				else
				{
					FoundSubjects.Add(SubjectPreset.Key);
				}
			}
		}

		LiveLinkClient.Tick();

		bResult = true;
		for (const FLiveLinkSourcePreset& SourcePreset : Sources)
		{
			bResult &= (FoundSources.Contains(SourcePreset.Guid) || LiveLinkClient.CreateSource(SourcePreset));
		}

		for (const FLiveLinkSubjectPreset& SubjectPreset : Subjects)
		{
			bResult &= (FoundSubjects.Contains(SubjectPreset.Key) || LiveLinkClient.CreateSubject(SubjectPreset));
		}
	}

	if (bResult)
	{
		FLiveLinkLog::Info(TEXT("Added '%s'"), *GetFullName());
	}
	else
	{
		FLiveLinkLog::Error(TEXT("Could not add '%s'"), *GetFullName());
	}

	return bResult;
}

void ULiveLinkPreset::BuildFromClient()
{
	Sources.Reset();
	Subjects.Reset();

	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		FLiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		for (const FGuid& SourceGuid : LiveLinkClient.GetSources())
		{
			Sources.Add(LiveLinkClient.GetSourcePreset(SourceGuid, this));
		}

		for (const FGuid& SourceGuid : LiveLinkClient.GetVirtualSources())
		{
			FLiveLinkSourcePreset NewPreset = LiveLinkClient.GetSourcePreset(SourceGuid, this);
			if (NewPreset.Guid.IsValid())
			{
				Sources.Add(MoveTemp(NewPreset));
			}
		}

		for (FLiveLinkSubjectKey SubjectKey : LiveLinkClient.GetSubjects(true, true))
		{
			Subjects.Add(LiveLinkClient.GetSubjectPreset(SubjectKey, this));
		}
	}
}

void ULiveLinkPreset::ClearApplyToClientTimer()
{
	if (ApplyToClientEndFrameHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(ApplyToClientEndFrameHandle);
	}
}

