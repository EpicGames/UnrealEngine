// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSettings.h"
#include "LiveLinkRole.h"
#include "LiveLinkSubjectSettings.h"
#include "LiveLinkSourceSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkSettings)

FLiveLinkRoleProjectSetting::FLiveLinkRoleProjectSetting(TSubclassOf<ULiveLinkSubjectSettings> DefaultSettingsClass)
	: SettingClass(DefaultSettingsClass)
{
}


ULiveLinkSettings::ULiveLinkSettings()
	: ClockOffsetCorrectionStep(100e-6f)
	, DefaultMessageBusSourceMode(ELiveLinkSourceMode::EngineTime)
	, MessageBusPingRequestFrequency(1.0)
	, MessageBusHeartbeatFrequency(1.0)
	, MessageBusHeartbeatTimeout(2.0)
	, MessageBusTimeBeforeRemovingInactiveSource(5.0)
	, TimeWithoutFrameToBeConsiderAsInvalid(0.5)
	, ValidColor(FColor(31, 228, 75))
	, InvalidColor(FColor(255, 184, 0))
	, TextSizeSource(16)
	, TextSizeSubject(12)
{
}

FLiveLinkRoleProjectSetting ULiveLinkSettings::GetDefaultSettingForRole(TSubclassOf<ULiveLinkRole> Role) const
{
	int32 IndexOf = DefaultRoleSettings.IndexOfByPredicate([Role](const FLiveLinkRoleProjectSetting& Other) {return Other.Role == Role; });
	if (IndexOf != INDEX_NONE)
	{
		return DefaultRoleSettings[IndexOf];
	}

	TSubclassOf<ULiveLinkSubjectSettings> DefaultClass = ULiveLinkSubjectSettings::StaticClass();
	if (UClass* Class = DefaultSettingsClass.TryLoadClass<ULiveLinkSubjectSettings>())
	{
		DefaultClass = Class;
	}

	FLiveLinkRoleProjectSetting Result{MoveTemp(DefaultClass)};
	Result.Role = Role;
	return Result;
}

void ULiveLinkSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// If the deprecated preset save directory has a valid path, update the replacement setting in LiveLinkUserSettings with its value
	if (!PresetSaveDir_DEPRECATED.Path.IsEmpty())
	{
		ULiveLinkUserSettings* UserSettings = GetMutableDefault<ULiveLinkUserSettings>();
		UserSettings->PresetSaveDir = PresetSaveDir_DEPRECATED;
		UserSettings->SaveConfig();

		// Empty the path of the deprecated setting so that this won't overwrite the user setting again
		PresetSaveDir_DEPRECATED.Path.Empty();
		SaveConfig();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif //WITH_EDITOR
}
