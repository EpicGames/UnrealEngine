// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSourceCollection.h"

#include "EngineAnalytics.h"
#include "Experimental/Async/MultiUniqueLock.h"
#include "IAnalyticsProviderET.h"
#include "LiveLinkSettings.h"
#include "LiveLinkVirtualSource.h"
#include "UObject/Package.h"

/**
 * Default VirtualSubject Source.
 */ 
struct FLiveLinkDefaultVirtualSubjectSource : public FLiveLinkVirtualSubjectSource
{
	FLiveLinkDefaultVirtualSubjectSource() = default;
	virtual ~FLiveLinkDefaultVirtualSubjectSource() = default;

	virtual bool CanBeDisplayedInUI() const override { return false; }
};

namespace LiveLinkSourceCollection
{
	static void SendAnalyticsSourceAdded(const ILiveLinkSource* Source)
	{
		if (!Source || !FEngineAnalytics::IsAvailable())
		{
			return;
		}

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Type"), Source->GetSourceType().ToString()));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Usage.LiveLink.SourceAdded"), EventAttributes);
	}
}


bool FLiveLinkCollectionSourceItem::IsVirtualSource() const
{
	return bIsVirtualSource;
}

FLiveLinkCollectionSubjectItem::FLiveLinkCollectionSubjectItem(FLiveLinkSubjectKey InKey, TUniquePtr<FLiveLinkSubject> InLiveSubject, ULiveLinkSubjectSettings* InSettings, bool bInEnabled)
	: Key(InKey)
	, bEnabled(bInEnabled)
	, bPendingKill(false)
	, Setting(InSettings)
	, LiveSubject(MoveTemp(InLiveSubject))
	, VirtualSubject(nullptr)
{
}


FLiveLinkCollectionSubjectItem::FLiveLinkCollectionSubjectItem(FLiveLinkSubjectKey InKey, ULiveLinkVirtualSubject* InVirtualSubject, bool bInEnabled)
	: Key(InKey)
	, bEnabled(bInEnabled)
	, bPendingKill(false)
	, Setting(nullptr)
	, VirtualSubject(InVirtualSubject)
{
}

const FGuid FLiveLinkSourceCollection::DefaultVirtualSubjectGuid{ 0x4ed2dc4e, 0xcc5911ce, 0x4af0635d, 0xa8b24a5a };


FLiveLinkSourceCollection::FLiveLinkSourceCollection()
{
	FLiveLinkCollectionSourceItem Data;
	Data.Source = MakeShared<FLiveLinkDefaultVirtualSubjectSource>();
	Data.Guid = DefaultVirtualSubjectGuid;
	ULiveLinkVirtualSubjectSourceSettings* NewSettings = NewObject<ULiveLinkVirtualSubjectSourceSettings>(GetTransientPackage(), ULiveLinkVirtualSubjectSourceSettings::StaticClass());
	NewSettings->SourceName = TEXT("DefaultVirtualSource");
	Data.Setting = TStrongObjectPtr(NewSettings);
	Data.bIsVirtualSource = true;
	Data.Source->InitializeSettings(NewSettings);

	if (!IsInGameThread())
	{
		// If the settings object was created outside of the game thread, we need to clear the async flag to allow the object to be garbage collected.
		Data.Setting->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
	}

	Sources.Add(MoveTemp(Data));
}

void FLiveLinkSourceCollection::AddSource(FLiveLinkCollectionSourceItem InSource)
{
	FGuid SourceGuid;
	{
		UE::TUniqueLock Lock(SourcesLock);
		FLiveLinkCollectionSourceItem& SourceItem = Sources.Add_GetRef(MoveTemp(InSource));
		LiveLinkSourceCollection::SendAnalyticsSourceAdded(SourceItem.Source.Get());
		SourceGuid = SourceItem.Guid;
	}

	OnLiveLinkSourceAdded().Broadcast(SourceGuid);
	OnLiveLinkSourcesChanged().Broadcast();

}


void FLiveLinkSourceCollection::RemoveSource(FGuid InSourceGuid)
{
	if (InSourceGuid != FLiveLinkSourceCollection::DefaultVirtualSubjectGuid)
	{
		UE::TMultiUniqueLock<UE::FTransactionallySafeRecursiveMutex> MultiLock({&SubjectsLock, &SourcesLock});

		int32 SourceIndex = Sources.IndexOfByPredicate([InSourceGuid](const FLiveLinkCollectionSourceItem& Other) { return Other.Guid == InSourceGuid; });
		if (SourceIndex != INDEX_NONE)
		{
			bool bRemovedSubject = false;
			FGuid SourceGuid = Sources[SourceIndex].Guid;
			for (int32 SubjectIndex = Subjects.Num() - 1; SubjectIndex >= 0; --SubjectIndex)
			{
				if (Subjects[SubjectIndex].Key.Source == SourceGuid)
				{
					bRemovedSubject = true;
					FLiveLinkSubjectKey Key = Subjects[SubjectIndex].Key;
					Subjects.RemoveAtSwap(SubjectIndex);
					BroadcastOnGameThread(OnLiveLinkSubjectRemoved(), Key);
				}
			}

			if (bRemovedSubject)
			{
				BroadcastOnGameThread(OnLiveLinkSubjectsChanged());
			}

			Sources.RemoveAtSwap(SourceIndex);
			BroadcastOnGameThread(OnLiveLinkSourceRemoved(), InSourceGuid);
			BroadcastOnGameThread(OnLiveLinkSourcesChanged());
		}
	}
}


void FLiveLinkSourceCollection::RemoveAllSources()
{
	bool bHasRemovedSubject = false;
	{
		UE::TUniqueLock Lock(SubjectsLock);
		for (int32 Index = Subjects.Num() - 1; Index >= 0; --Index)
		{
			bHasRemovedSubject = true;
			FLiveLinkSubjectKey Key = Subjects[Index].Key;
			Subjects.RemoveAtSwap(Index);
			BroadcastOnGameThread(OnLiveLinkSubjectRemoved(), Key);
		}
	}

	if (bHasRemovedSubject)
	{
		BroadcastOnGameThread(OnLiveLinkSubjectsChanged());
	}

	bool bHasRemovedSource = false;
	{
		UE::TUniqueLock Lock(SubjectsLock);
		for (int32 Index = Sources.Num() - 1; Index >= 0; --Index)
		{
			if (Sources[Index].Guid != FLiveLinkSourceCollection::DefaultVirtualSubjectGuid)
			{
				bHasRemovedSource = true;
				FGuid Key = Sources[Index].Guid;
				Sources.RemoveAtSwap(Index);
				BroadcastOnGameThread(OnLiveLinkSourceRemoved(), Key);
			}
		}
	}

	if (bHasRemovedSource)
	{
		BroadcastOnGameThread(OnLiveLinkSourcesChanged());
	}
}


FLiveLinkCollectionSourceItem* FLiveLinkSourceCollection::FindSource(TSharedPtr<ILiveLinkSource> InSource)
{
	UE::TUniqueLock Lock(SourcesLock);
	return Sources.FindByPredicate([InSource](const FLiveLinkCollectionSourceItem& Other) { return Other.Source == InSource; });
}


const FLiveLinkCollectionSourceItem* FLiveLinkSourceCollection::FindSource(TSharedPtr<ILiveLinkSource> InSource) const
{
	return const_cast<FLiveLinkSourceCollection*>(this)->FindSource(InSource);
}


FLiveLinkCollectionSourceItem* FLiveLinkSourceCollection::FindSource(FGuid InSourceGuid)
{
	UE::TUniqueLock Lock(SourcesLock);
	return Sources.FindByPredicate([InSourceGuid](const FLiveLinkCollectionSourceItem& Other) { return Other.Guid == InSourceGuid; });
}


const FLiveLinkCollectionSourceItem* FLiveLinkSourceCollection::FindSource(FGuid InSourceGuid) const
{
	return const_cast<FLiveLinkSourceCollection*>(this)->FindSource(InSourceGuid);
}


FLiveLinkCollectionSourceItem* FLiveLinkSourceCollection::FindVirtualSource(FName VirtualSourceName)
{
	UE::TUniqueLock Lock(SourcesLock);
	return Sources.FindByPredicate([VirtualSourceName](const FLiveLinkCollectionSourceItem& Other)
	{
		if (Other.IsVirtualSource())
		{
			if (ULiveLinkVirtualSubjectSourceSettings* VirtualSubjectSettings = Cast<ULiveLinkVirtualSubjectSourceSettings>(Other.Setting.Get()))
			{
				return VirtualSubjectSettings->SourceName == VirtualSourceName;
			}
		}
		return false;
	});
}

const FLiveLinkCollectionSourceItem* FLiveLinkSourceCollection::FindVirtualSource(FName VirtualSourceName) const
{
	return const_cast<FLiveLinkSourceCollection*>(this)->FindVirtualSource(VirtualSourceName);
}

int32 FLiveLinkSourceCollection::NumSources() const
{
	UE::TUniqueLock Lock(SourcesLock);
	return Sources.Num();
}

void FLiveLinkSourceCollection::AddSubject(FLiveLinkCollectionSubjectItem InSubject)
{
	FLiveLinkSubjectKey Key = InSubject.Key;

	if (FLiveLinkSubject* LiveLinkSubject = InSubject.GetLiveSubject())
	{
		LiveLinkSubject->OnStateChanged().BindRaw(this, &FLiveLinkSourceCollection::HandleSubjectStateChanged, Key);
	}

	{
		UE::TUniqueLock Lock(SubjectsLock);
		Subjects.Add(MoveTemp(InSubject));
	}

	BroadcastOnGameThread(OnLiveLinkSubjectAdded(), Key);
	BroadcastOnGameThread(OnLiveLinkSubjectsChanged());
}


void FLiveLinkSourceCollection::RemoveSubject(FLiveLinkSubjectKey InSubjectKey)
{
	bool bWasEnabled = false;

	TOptional<FLiveLinkSubjectKey> ReEnabledSubject;
	{
		UE::TUniqueLock Lock(SubjectsLock);
		int32 IndexOf = Subjects.IndexOfByPredicate([InSubjectKey](const FLiveLinkCollectionSubjectItem& Other) { return Other.Key == InSubjectKey; });
		if (IndexOf != INDEX_NONE)
		{
			bWasEnabled = Subjects[IndexOf].bEnabled;
			Subjects.RemoveAtSwap(IndexOf);
		}

		// If the removed subject was enabled and the setting is on, re-enable another subject
		// with the same name so it can take over. This prevents the case where removing a subject
		// that was shadowing another (same name, different source) leaves the subject disabled.
		if (bWasEnabled && GetDefault<ULiveLinkSettings>()->bReEnableSubjectOnRemoval)
		{
			for (FLiveLinkCollectionSubjectItem& SubjectItem : Subjects)
			{
				if (SubjectItem.Key.SubjectName == InSubjectKey.SubjectName && !SubjectItem.bPendingKill)
				{
					SubjectItem.bEnabled = true;
					ReEnabledSubject = SubjectItem.Key;
					break;
				}
			}
		}
	}

	if (ReEnabledSubject)
	{
		BroadcastOnGameThread(OnLiveLinkSubjectEnabledChanged(), *ReEnabledSubject, true);
	}

	BroadcastOnGameThread(OnLiveLinkSubjectRemoved(), InSubjectKey);
    BroadcastOnGameThread(OnLiveLinkSubjectsChanged());
}


FLiveLinkCollectionSubjectItem* FLiveLinkSourceCollection::FindSubject(FLiveLinkSubjectKey InSubjectKey)
{
	UE::TUniqueLock Lock(SubjectsLock);
	return Subjects.FindByPredicate([InSubjectKey](const FLiveLinkCollectionSubjectItem& Other) { return Other.Key == InSubjectKey; });
}


const FLiveLinkCollectionSubjectItem* FLiveLinkSourceCollection::FindSubject(FLiveLinkSubjectKey InSubjectKey) const
{
	return const_cast<FLiveLinkSourceCollection*>(this)->FindSubject(InSubjectKey);
}


const FLiveLinkCollectionSubjectItem* FLiveLinkSourceCollection::FindSubject(FLiveLinkSubjectName SubjectName) const
{
	UE::TUniqueLock Lock(SubjectsLock);
	return Subjects.FindByPredicate([SubjectName](const FLiveLinkCollectionSubjectItem& Other) { return Other.Key.SubjectName == SubjectName;  });
}

const FLiveLinkCollectionSubjectItem* FLiveLinkSourceCollection::FindEnabledSubject(FLiveLinkSubjectName InSubjectName) const
{
	UE::TUniqueLock Lock(SubjectsLock);
	return Subjects.FindByPredicate([InSubjectName](const FLiveLinkCollectionSubjectItem& Other) { return Other.Key.SubjectName == InSubjectName && Other.bEnabled && !Other.bPendingKill; });
}

int32 FLiveLinkSourceCollection::NumSubjects() const
{
	UE::TUniqueLock Lock(SubjectsLock);
	return Subjects.Num();
}


bool FLiveLinkSourceCollection::IsSubjectEnabled(FLiveLinkSubjectKey InSubjectKey) const
{
	UE::TUniqueLock Lock(SubjectsLock);
	if (const FLiveLinkCollectionSubjectItem* Item = FindSubject(InSubjectKey))
	{
		return Item->bEnabled;
	}
	return false;
}


void FLiveLinkSourceCollection::SetSubjectEnabled(FLiveLinkSubjectKey InSubjectKey, bool bEnabled)
{
	if (bEnabled)
	{
		// clear all bEnabled only if found
		if (FLiveLinkCollectionSubjectItem* NewEnabledItem = FindSubject(InSubjectKey))
		{
			UE::TUniqueLock Lock(SubjectsLock);

			NewEnabledItem->bEnabled = true;

			BroadcastOnGameThread(OnLiveLinkSubjectEnabledChanged(), NewEnabledItem->Key, NewEnabledItem->bEnabled);
			
			for (FLiveLinkCollectionSubjectItem& SubjectItem : Subjects)
			{
				if (SubjectItem.bEnabled && SubjectItem.Key.SubjectName == InSubjectKey.SubjectName && !(SubjectItem.Key == InSubjectKey))
				{
					SubjectItem.bEnabled = false;
					BroadcastOnGameThread(OnLiveLinkSubjectEnabledChanged(), SubjectItem.Key, SubjectItem.bEnabled);
				}
			}
		}
	}
	else
	{
		UE::TUniqueLock Lock(SubjectsLock);
		for (FLiveLinkCollectionSubjectItem& SubjectItem : Subjects)
		{
			if (SubjectItem.Key.SubjectName == InSubjectKey.SubjectName)
			{
				SubjectItem.bEnabled = false;
				BroadcastOnGameThread(OnLiveLinkSubjectEnabledChanged(), SubjectItem.Key, SubjectItem.bEnabled);
			}
		}
	}
}


void FLiveLinkSourceCollection::RemovePendingKill()
{
	UE::TMultiUniqueLock<UE::FTransactionallySafeRecursiveMutex> MultiLock({&SubjectsLock, &SourcesLock});

	for (int32 SourceIndex = Sources.Num() - 1; SourceIndex >= 0; --SourceIndex)
	{
		FLiveLinkCollectionSourceItem& SourceItem = Sources[SourceIndex];
		if (SourceItem.bPendingKill)
		{
			if (SourceItem.Guid == FLiveLinkSourceCollection::DefaultVirtualSubjectGuid)
			{
				// Keep the default virtual subject source but mark the subject as pending kill
				for (FLiveLinkCollectionSubjectItem& SubjectItem : Subjects)
				{
					if (SubjectItem.Key.Source == FLiveLinkSourceCollection::DefaultVirtualSubjectGuid)
					{
						SubjectItem.bPendingKill = true;
					}
				}
				SourceItem.bPendingKill = false;
			}
			else if (SourceItem.Source->RequestSourceShutdown())
			{
				RemoveSource(SourceItem.Guid);
			}
		}
	}

	// Remove Subjects that are pending kill
	for (int32 SubjectIndex = Subjects.Num() - 1; SubjectIndex >= 0; --SubjectIndex)
	{
		const FLiveLinkCollectionSubjectItem& SubjectItem = Subjects[SubjectIndex];
		if (SubjectItem.bPendingKill)
		{
			RemoveSubject(SubjectItem.Key);
		}
	}
}


bool FLiveLinkSourceCollection::RequestShutdown()
{
	{
		UE::TUniqueLock Lock(SubjectsLock);
		Subjects.Reset();
	}

	UE::TUniqueLock Lock(SourcesLock);
	for (int32 SourceIndex = Sources.Num() - 1; SourceIndex >= 0; --SourceIndex)
	{
		FLiveLinkCollectionSourceItem& SourceItem = Sources[SourceIndex];
		if (SourceItem.Source->RequestSourceShutdown())
		{
			Sources.RemoveAtSwap(SourceIndex);
		}
	}

	// No callback when we shutdown
	return Sources.Num() == 0;
}


void FLiveLinkSourceCollection::ForEachSubject(TFunctionRef<void(FLiveLinkCollectionSourceItem&, FLiveLinkCollectionSubjectItem&)> VisitorFunc)
{
	UE::TMultiUniqueLock<UE::FTransactionallySafeRecursiveMutex> MultiLock({&SubjectsLock, &SourcesLock});

	for (FLiveLinkCollectionSubjectItem& Subject : Subjects)
	{
		if (FLiveLinkCollectionSourceItem* SourceItem = FindSource(Subject.Key.Source))
		{
			VisitorFunc(*SourceItem, Subject);
		}
	}
}


void FLiveLinkSourceCollection::ForEachSubject(TFunctionRef<void(const FLiveLinkCollectionSourceItem&, const FLiveLinkCollectionSubjectItem&)> VisitorFunc) const
{
	UE::TMultiUniqueLock<UE::FTransactionallySafeRecursiveMutex> MultiLock({ &SubjectsLock, &SourcesLock });

	for (const FLiveLinkCollectionSubjectItem& Subject : Subjects)
	{
		if (const FLiveLinkCollectionSourceItem* SourceItem = FindSource(Subject.Key.Source))
		{
			VisitorFunc(*SourceItem, Subject);
		}
	}
}


void FLiveLinkSourceCollection::ForEachSource(TFunctionRef<void(FLiveLinkCollectionSourceItem&)> VisitorFunc)
{
	UE::TUniqueLock Lock(SourcesLock);

	for (FLiveLinkCollectionSourceItem& Source : Sources)
	{
		VisitorFunc(Source);
	}
}


void FLiveLinkSourceCollection::ForEachSource(TFunctionRef<void(const FLiveLinkCollectionSourceItem&)> VisitorFunc) const
{
	UE::TUniqueLock Lock(SourcesLock);

	for (const FLiveLinkCollectionSourceItem& Source : Sources)
	{
		VisitorFunc(Source);
	}
}

void FLiveLinkSourceCollection::ApplyToSubject(const FLiveLinkSubjectKey& InSubjectKey, TFunctionRef<void(FLiveLinkCollectionSubjectItem& /*SubjectItem*/)> VisitorFunc)
{
	if (FLiveLinkCollectionSubjectItem* Subject = FindSubject(InSubjectKey))
	{
		UE::TUniqueLock Lock(SubjectsLock);
		VisitorFunc(*Subject);
	}
}

void FLiveLinkSourceCollection::HandleSubjectStateChanged(ELiveLinkSubjectState NewState, FLiveLinkSubjectKey SubjectKey)
{
	BroadcastOnGameThread(OnLiveLinkSubjectStateChanged(), SubjectKey, NewState);
}