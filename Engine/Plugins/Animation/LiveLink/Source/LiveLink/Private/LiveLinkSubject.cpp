// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSubject.h"

#include "ITimeManagementModule.h"
#include "LiveLinkClient.h"
#include "LiveLinkLog.h"
#include "LiveLinkRole.h"
#include "LiveLinkRoleTrait.h"
#include "LiveLinkSettings.h"
#include "LiveLinkSubjectSettings.h"
#include "Misc/App.h"
#include "LiveLinkTimedDataInput.h"
#include "TimedDataInputCollection.h"

#if WITH_EDITOR
#include "SkinnedAssetCompiler.h"
#endif

FLiveLinkSubject::FSubjectEvaluationStatistics::FSubjectEvaluationStatistics()
	: BufferUnderflow(0)
	, BufferOverflow(0)
	, FrameDrop(0)
{
}

FLiveLinkSubject::FLiveLinkSubject(TSharedPtr<FLiveLinkTimedDataInput> InTimedDataGroup)
	: TimedDataGroup(InTimedDataGroup)
	, State(ELiveLinkSubjectState::Unknown)
{
	check(InTimedDataGroup.IsValid());
}

void FLiveLinkSubject::Initialize(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, ILiveLinkClient* InLiveLinkClient)
{
	SubjectKey = InSubjectKey;
	Role = InRole;

	State = ELiveLinkSubjectState::Disconnected;

	FrameData.Reset();
	ReceivedOrderedFrames.Empty();
	LastTimecodeFrameRate = FFrameRate(1, -1); //Initialized as invalid for first detection
	ResetBufferStats();

	if (TSharedPtr<FLiveLinkTimedDataInput> TimedDataGroupPin = TimedDataGroup.Pin())
	{
		TimedDataGroupPin->AddChannel(this);
	}

	ITimeManagementModule::Get().GetTimedDataInputCollection().Add(this);
}

FLiveLinkSubject::~FLiveLinkSubject()
{
	if (TSharedPtr<FLiveLinkTimedDataInput> TimedDataGroupPin = TimedDataGroup.Pin())
	{
		TimedDataGroupPin->RemoveChannel(this);
	}
	ITimeManagementModule::Get().GetTimedDataInputCollection().Remove(this);
}

void FLiveLinkSubject::Update()
{
	FScopeLock Lock(&SettingsCriticalSection);

	if (bClearOverrideStaticData)
	{
		ClearFrames();
		OverrideStaticData.Reset();
		bClearOverrideStaticData = false;
	}

	// Clear all frames that are too old
	if (FrameData.Num() > CachedSettings.BufferSettings.MaxNumberOfFrameToBuffered)
	{
		const int32 NumberOfFrameToRemove = FrameData.Num() - CachedSettings.BufferSettings.MaxNumberOfFrameToBuffered;
		const int32 Count = CachedSettings.BufferSettings.bKeepAtLeastOneFrame && FrameData.Num() == NumberOfFrameToRemove ? NumberOfFrameToRemove-1 : NumberOfFrameToRemove;
		RemoveFrames(Count);
	}

	if (bPaused)
	{
		// No need to process if it's paused, just keep the same frame snapshot.
		FWriteScopeLock WriteLock(FrameSnapshotRWLock);
		if (FrameSnapshot.StaticData.IsValid() && FrameSnapshot.FrameData.IsValid())
		{
			FrameSnapshot.FrameData.GetBaseData()->WorldTime = FPlatformTime::Seconds();
			// Only return if we have a valid snapshot. Otherwise we'll try to recreate it, since it may have been cleared.
			return;
		}
	}

	if (GetMode() == ELiveLinkSourceMode::EngineTime)
	{
		if (CachedSettings.BufferSettings.bValidEngineTimeEnabled)
		{
			//Engine time threshold is made from current time minus all desired offset (user offset, validity offset, clock offset)
			const double ValidEngineTime = FApp::GetCurrentTime() - CachedSettings.BufferSettings.EngineTimeOffset - CachedSettings.BufferSettings.ValidEngineTime - CachedSettings.BufferSettings.EngineTimeClockOffset;
			int32 FrameIndex = 0;
			for (const FLiveLinkFrameDataStruct& SourceFrameData : FrameData)
			{
				const double FrameTime = SourceFrameData.GetBaseData()->WorldTime.GetSourceTime();
				if (FrameTime > ValidEngineTime)
				{
					break;
				}
				++FrameIndex;
			}

			if (FrameIndex - 1 >= 0)
			{
				const int32 Count = CachedSettings.BufferSettings.bKeepAtLeastOneFrame && FrameData.Num() == FrameIndex ? FrameIndex - 1 : FrameIndex;
				RemoveFrames(Count);
			}
		}
	}
	else if (GetMode() == ELiveLinkSourceMode::Timecode)
	{
		if (CachedSettings.BufferSettings.bValidTimecodeFrameEnabled)
		{
			if (FApp::GetCurrentFrameTime().IsSet())
			{
				//Take the current Engine Timecode and substract all the offset in subject's frame space (clock offset if any, frame offset and the desired frame distance threshold)
				const FQualifiedFrameTime CurrentSyncTime = FApp::GetCurrentFrameTime().GetValue();

				FFrameTime ClockOffset;
				if (CachedSettings.BufferSettings.bUseTimecodeSmoothLatest)
				{
					ClockOffset = LastTimecodeFrameRate.AsFrameTime(CachedSettings.BufferSettings.TimecodeClockOffset);
				}

				const FFrameTime CurrentFrameTimeInFrameSpace = CurrentSyncTime.ConvertTo(LastTimecodeFrameRate);
				const FFrameTime FrameTimeThresholdFrameSpace = CurrentFrameTimeInFrameSpace - (ClockOffset + FFrameTime::FromDecimal(CachedSettings.BufferSettings.TimecodeFrameOffset) + CachedSettings.BufferSettings.ValidTimecodeFrame);
				int32 FrameIndex = 0;
				for (const FLiveLinkFrameDataStruct& SourceFrameData : FrameData)
				{
					const FFrameTime FrameTime = SourceFrameData.GetBaseData()->MetaData.SceneTime.Time;
					if (FrameTime > FrameTimeThresholdFrameSpace)
					{
						break;
					}
					++FrameIndex;
				}

				if (FrameIndex - 1 >= 0)
				{
					const int32 Count = CachedSettings.BufferSettings.bKeepAtLeastOneFrame && FrameData.Num() == FrameIndex ? FrameIndex - 1 : FrameIndex;
					RemoveFrames(Count);
				}
			}
		}

		// no warning if GetCurrentFrameTime is not set, the warning is done below after GetFrameAtSceneTime
	}

	// Build a snapshot for this role
	// Protect FrameSnapshot with write lock during snapshot construction
	{
		FWriteScopeLock WriteLock(FrameSnapshotRWLock);

		bool bSnapshotIsValid = false;
		if (FrameData.Num() > 0)
		{
			switch(GetMode())
			{
			case ELiveLinkSourceMode::Timecode:
			{
				if (FApp::GetCurrentFrameTime().IsSet())
				{
					//If Timecode smooth latest is enabled, use the clock offset to adjust read time.
					//@note For now, snapshot created will have the source time. We are not overriding its time with the read time. If a use case arise for this, we'll adjust.
					const FQualifiedFrameTime EngineTimecode = FApp::GetCurrentFrameTime().GetValue();

					FFrameTime ClockOffset;
					if (CachedSettings.BufferSettings.bUseTimecodeSmoothLatest)
					{
						ClockOffset = EngineTimecode.Rate.AsFrameTime(CachedSettings.BufferSettings.TimecodeClockOffset);
					}

					const FFrameTime FrameOffset = FQualifiedFrameTime(FFrameTime::FromDecimal(CachedSettings.BufferSettings.TimecodeFrameOffset), LastTimecodeFrameRate).ConvertTo(EngineTimecode.Rate);
					const FFrameTime ReadTime = EngineTimecode.Time - (ClockOffset + FrameOffset);
					const FQualifiedFrameTime LookupQFrameTime = FQualifiedFrameTime(ReadTime, EngineTimecode.Rate);
					bSnapshotIsValid = GetFrameAtSceneTime(LookupQFrameTime, FrameSnapshot);
				}
				else
				{
					static const FName NAME_InvalidTime = "LiveLinkSubject_NoCurrentFrameTime";
					FLiveLinkLog::WarningOnce(NAME_InvalidTime, SubjectKey, TEXT("Can't evaluate frame for subject '%s'. The engine doesn't have a timecode value set."), *SubjectKey.SubjectName.ToString());
				}
			}
			break;

			case ELiveLinkSourceMode::EngineTime:
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FAppTime Time = FAppTime::Get_UnsafeNoCheck();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				//Compute the evaluation time using all offsets (user offset + clock offset + smooth offset)
				const double ReadTime = Time.CurrentTime - CachedSettings.BufferSettings.EngineTimeOffset - CachedSettings.BufferSettings.EngineTimeClockOffset - CachedSettings.BufferSettings.SmoothEngineTimeOffset;

				UE_LOGF(LogLiveLink, Verbose, "Subject '%ls' Read Time: %lf    User Offset: %.4lf    Clock Offset: %.4lf    Smooth Offset: %.4lf"
					, *SubjectKey.SubjectName.ToString()
					, ReadTime
					, CachedSettings.BufferSettings.EngineTimeOffset
					, CachedSettings.BufferSettings.EngineTimeClockOffset
					, CachedSettings.BufferSettings.SmoothEngineTimeOffset);

				bSnapshotIsValid = GetFrameAtWorldTime(ReadTime, FrameSnapshot);
				break;
			}

			case ELiveLinkSourceMode::Latest:
			default:
				bSnapshotIsValid = GetLatestFrame(FrameSnapshot);
			}
		}

		if (!bSnapshotIsValid)
		{
			// Invalidate the snapshot
			FrameSnapshot.FrameData.Reset();
		}
		else
		{
			//Extra verbose logging about snapshot just created
			const FTimecode FrameTC = FTimecode::FromFrameNumber(FrameSnapshot.FrameData.GetBaseData()->MetaData.SceneTime.Time.GetFrame(), FrameSnapshot.FrameData.GetBaseData()->MetaData.SceneTime.Rate);
			UE_LOGF(LogLiveLink, Verbose, "Subject '%ls' created snapshot with frame ID %d - Timecode:[%ls.%0.3f] - SourceTime: %0.4f, Offset: %0.6f, CorrectedTime: %0.4f", *SubjectKey.SubjectName.ToString(), FrameSnapshot.FrameData.GetBaseData()->FrameId, *FrameTC.ToString(), FrameSnapshot.FrameData.GetBaseData()->MetaData.SceneTime.Time.GetSubFrame(), FrameSnapshot.FrameData.GetBaseData()->WorldTime.GetSourceTime(), FrameSnapshot.FrameData.GetBaseData()->WorldTime.GetOffset(), FrameSnapshot.FrameData.GetBaseData()->WorldTime.GetOffsettedTime());
		}
	}

	UpdateCachedOldestDataTime();
	UpdateCachedNewestDataTime();
	UdpateCachedDataTimes();

	UpdateState();
}

void FLiveLinkSubject::ClearFrames()
{
	FWriteScopeLock WriteLock(FrameSnapshotRWLock);
	FrameSnapshot.StaticData.Reset();
	FrameSnapshot.FrameData.Reset();
	FrameData.Reset();
	ReceivedOrderedFrames.Empty();
}

bool FLiveLinkSubject::EvaluateFrame(TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	// Protect FrameSnapshot access with read lock to allow concurrent reads from multiple animation threads
	FReadScopeLock ReadLock(FrameSnapshotRWLock);
	return ILiveLinkSubject::EvaluateFrame(InDesiredRole, OutFrame);
}

bool FLiveLinkSubject::HasValidFrameSnapshot() const
{
	FReadScopeLock ReadLock(FrameSnapshotRWLock);
	return FrameSnapshot.StaticData.IsValid() && FrameSnapshot.FrameData.IsValid();
}

FLiveLinkStaticDataStruct& FLiveLinkSubject::GetStaticData(bool bGetOverrideData)
{
	if (bGetOverrideData && OverrideStaticData)
	{
		return *OverrideStaticData;
	}

	return StaticData;
}

const FLiveLinkStaticDataStruct& FLiveLinkSubject::GetStaticData() const
{
	if (OverrideStaticData)
	{
		return *OverrideStaticData;
	}

	return StaticData;
}

TArray<FLiveLinkTime> FLiveLinkSubject::GetFrameTimes() const
{
	TArray<FLiveLinkTime> Result;
	Result.Reset(FrameData.Num());
	for (const FLiveLinkFrameDataStruct& Data : FrameData)
	{
		//When giving out frame times, include the latest Source clock offset
		const double OffsettedWorldTime = Data.GetBaseData()->WorldTime.GetSourceTime() + CachedSettings.BufferSettings.EngineTimeClockOffset + CachedSettings.BufferSettings.SmoothEngineTimeOffset;
		Result.Emplace(OffsettedWorldTime, Data.GetBaseData()->MetaData.SceneTime);
	}
	return Result;
}

bool FLiveLinkSubject::EvaluateFrameAtWorldTime(double InWorldTime, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	if (Role == nullptr)
	{
		static const FName NAME_InvalidRole = "LiveLinkSubject_InvalidRole";
		FLiveLinkLog::ErrorOnce(NAME_InvalidRole, SubjectKey, TEXT("Can't evaluate frame for subject '%s'. No role has been set yet."), *SubjectKey.SubjectName.ToString());
		return false;
	}

	if (InDesiredRole == nullptr)
	{
		static const FName NAME_InvalidDesiredRole = "LiveLinkSubject_InvalidDesiredRole";
		FLiveLinkLog::ErrorOnce(NAME_InvalidDesiredRole, SubjectKey, TEXT("Can't evaluate frame for subject '%s'. Invalid role was received for evaluation."), *SubjectKey.SubjectName.ToString());
		return false;
	}

	if (GetMode() != ELiveLinkSourceMode::EngineTime)
	{
		static const FName NAME_EvalutationWorldTime = "LiveLinkSubject_EvalutationWorldTime";
		FLiveLinkLog::ErrorOnce(NAME_EvalutationWorldTime, SubjectKey, TEXT("Can't evaluate the subject '%s' at world time. The source mode is not set to Engine Time."), *SubjectKey.SubjectName.ToString());
		return false;
	}

	bool bSuccess = false;
	if (FrameData.Num() != 0)
	{
		// Adjust desired read time based on offset settings (User desired offset and continuous clock offset)
		const double ReadTime = InWorldTime - CachedSettings.BufferSettings.EngineTimeOffset - CachedSettings.BufferSettings.EngineTimeClockOffset;
		if (Role == InDesiredRole || Role->IsChildOf(InDesiredRole))
		{
			GetFrameAtWorldTime(ReadTime, OutFrame);
			bSuccess = true;
		}
		else if (SupportsRole(InDesiredRole))
		{
			FLiveLinkSubjectFrameData TmpFrameData;
			GetFrameAtWorldTime(ReadTime, TmpFrameData);
			bSuccess = Translate(this, InDesiredRole, TmpFrameData.StaticData, TmpFrameData.FrameData, OutFrame);
		}
		else
		{
			static const FName NAME_CantTranslate = "LiveLinkSubject_CantTranslate";
			FLiveLinkLog::WarningOnce(NAME_CantTranslate, SubjectKey, TEXT("Can't evaluate frame for subject '%s'. Role '%s' is incompatible with current role '%s'."), *SubjectKey.SubjectName.ToString(), *InDesiredRole->GetName(), *Role->GetName());
		}
	}

	return bSuccess;
}

bool FLiveLinkSubject::EvaluateFrameAtSceneTime(const FQualifiedFrameTime& InSceneTime, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	if (Role == nullptr)
	{
		static const FName NAME_InvalidRole = "LiveLinkSubject_InvalidRole";
		FLiveLinkLog::ErrorOnce(NAME_InvalidRole, SubjectKey, TEXT("Can't evaluate frame for subject '%s'. No role has been set yet."), *SubjectKey.SubjectName.ToString());
		return false;
	}

	if (InDesiredRole == nullptr)
	{
		static const FName NAME_InvalidDesiredRole = "LiveLinkSubject_InvalidDesiredRole";
		FLiveLinkLog::ErrorOnce(NAME_InvalidDesiredRole, SubjectKey, TEXT("Can't evaluate frame for subject '%s'. Invalid role was received for evaluation."), *SubjectKey.SubjectName.ToString());
		return false;
	}

	if (GetMode() != ELiveLinkSourceMode::Timecode)
	{
		static const FName NAME_EvaluationSceneTime = "LiveLinkSubject_EvalutationSceneTime";
		FLiveLinkLog::ErrorOnce(NAME_EvaluationSceneTime, SubjectKey, TEXT("Can't evaluate the subject '%s' at scene time. The source mode is not set to Timecode."), *SubjectKey.SubjectName.ToString());
		return false;
	}

	bool bSuccess = false;
	if (FrameData.Num() != 0)
	{
		// Adjust desired read time based on offset settings and clock offset if any
		FFrameTime ClockOffset;
		if (CachedSettings.BufferSettings.bUseTimecodeSmoothLatest)
		{
			ClockOffset = InSceneTime.Rate.AsFrameTime(CachedSettings.BufferSettings.TimecodeClockOffset);
		}

		const FFrameTime FrameOffset = FQualifiedFrameTime(FFrameTime::FromDecimal(CachedSettings.BufferSettings.TimecodeFrameOffset), LastTimecodeFrameRate).ConvertTo(InSceneTime.Rate);
		const FFrameTime ReadTime = InSceneTime.Time - (ClockOffset + FrameOffset);
		const FQualifiedFrameTime LookupQFrameTime = FQualifiedFrameTime(ReadTime, InSceneTime.Rate);
		if (Role == InDesiredRole || Role->IsChildOf(InDesiredRole))
		{
			GetFrameAtSceneTime(LookupQFrameTime, OutFrame);
			bSuccess = true;
		}
		else if (SupportsRole(InDesiredRole))
		{
			FLiveLinkSubjectFrameData TmpFrameData;
			GetFrameAtSceneTime(LookupQFrameTime, TmpFrameData);
			bSuccess = Translate(this, InDesiredRole, TmpFrameData.StaticData, TmpFrameData.FrameData, OutFrame);
		}
		else
		{
			static const FName NAME_CantTranslate = "LiveLinkSubject_CantTranslate";
			FLiveLinkLog::WarningOnce(NAME_CantTranslate, SubjectKey, TEXT("Can't evaluate frame for subject '%s'. Role '%s' is incompatible with current role '%s'."), *SubjectKey.SubjectName.ToString(), *InDesiredRole->GetName(), *Role->GetName());
		}
	}

	return bSuccess;
}

bool FLiveLinkSubject::HasStaticData() const
{
	return StaticData.IsValid();
}

void FLiveLinkSubject::AddFrameData(FLiveLinkFrameDataStruct&& InFrameData)
{
	if (!ValidateFrameData(InFrameData))
	{
		return;
	}

	// Before adding the new frame, test to see if we are going to increase the buffer size
	const bool bRemoveFrame = FrameData.Num() >= CachedSettings.BufferSettings.MaxNumberOfFrameToBuffered;
	if (bRemoveFrame)
	{
		RemoveFrames(1);
	}

	ELiveLinkSourceMode Mode = CachedSettings.SourceMode;
	ModeOverride.Reset();
	
	// Check if this frame should be force synced, IE go to frame immediately and discard other frames.
	if (const FString* ForceSyncStr = InFrameData.IsValid()
									  ? InFrameData.GetBaseData()->MetaData.StringMetaData.Find(TEXT("ForceSync"))
									  : nullptr)
	{
		Mode = ELiveLinkSourceMode::Latest;
		bool bForceSync = false;
		LexFromString(bForceSync, **ForceSyncStr);

		if (bForceSync)
		{
			ModeOverride = Mode;
			// todo: should we only remove frames if the timestamp is less than the new frame? Can they arrive out of order?
			RemoveFrames(FrameData.Num());
		}
	}

	int32 FrameIndex = INDEX_NONE;
	switch (Mode)
	{
	case ELiveLinkSourceMode::EngineTime:
		FrameIndex = FindNewFrame_WorldTime(InFrameData.GetBaseData()->WorldTime);
		break;
	case ELiveLinkSourceMode::Timecode:
	{
		// Update cached FrameRate based on this new frame data
		if (LastTimecodeFrameRate.IsValid() && InFrameData.GetBaseData()->MetaData.SceneTime.Rate != LastTimecodeFrameRate)
		{
			FLiveLinkLog::Warning(TEXT("Subject '%s' is added a new frame in which the timecode frame rate ('%s') is different than previous frame's frame rate ('%s')."), *SubjectKey.SubjectName.ToString(), *InFrameData.GetBaseData()->MetaData.SceneTime.Rate.ToPrettyText().ToString(), *LastTimecodeFrameRate.ToPrettyText().ToString());

			// Frame Rate changed, clear our buffers to let new frames come in with new frame rate
			FrameData.Reset();
			ReceivedOrderedFrames.Empty();
		}

		// Stamp what frame rate we are currently expecting
		LastTimecodeFrameRate = InFrameData.GetBaseData()->MetaData.SceneTime.Rate;

		//Find an index for the new frame
		FrameIndex = FindNewFrame_SceneTime(InFrameData.GetBaseData()->MetaData.SceneTime, InFrameData.GetBaseData()->WorldTime);

		break;
	}
	
	case ELiveLinkSourceMode::Latest:
	default:
		FrameIndex = FindNewFrame_Latest(InFrameData.GetBaseData()->WorldTime);
		break;
	}

	// Make sure frame should be inserted. 
	if (FrameIndex >= 0)
	{
		for (ULiveLinkFramePreProcessor::FWorkerSharedPtr PreProcessor : FramePreProcessors)
		{
			PreProcessor->PreProcessFrame(GetStaticData(), InFrameData);
		}

		// FrameId should be populated by LiveLinkClient framework before reaching here
		if (FLiveLinkBaseFrameData* BaseData = InFrameData.GetBaseData())
		{
			if (BaseData->FrameId == INDEX_NONE)
			{
				UE_LOGF(LogLiveLink, 
					Error, 
					"FLiveLinkSubject::AddFrameData: Subject '%ls': FrameId was unexpectedly not set at this point in the pipeline, skipping it.", 
					*SubjectKey.SubjectName.ToString()
				);

				IncreaseFrameDroppedStat();

				FrameIndex = -1; // Prevents insertion into FrameData.
			}
			else
			{
				ReceivedOrderedFrames.Enqueue(BaseData->FrameId);
			}
		}

		if (FrameIndex >= 0)
		{
			FrameData.Insert(MoveTemp(InFrameData), FrameIndex);

			if (CachedSettings.BufferSettings.bGenerateSubFrame && CachedSettings.SourceMode == ELiveLinkSourceMode::Timecode)
			{
				AdjustSubFrame_SceneTime(FrameIndex);
			}
		}
	}
	else
	{
		// For some reason, the frame can't be added and is just discarded. Logs will have more info about the why
		IncreaseFrameDroppedStat();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAppTime Time = FAppTime::Get_UnsafeNoCheck();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	LastPushTime = Time.CurrentTime;
}

bool FLiveLinkSubject::IsPaused() const
{
	return bPaused;
}

void FLiveLinkSubject::PauseSubject()
{
	bPaused = true;
}

void FLiveLinkSubject::UnpauseSubject()
{
	bPaused = false;
}

void FLiveLinkSubject::PreprocessFrame(const FLiveLinkStaticDataStruct& InStaticData, FLiveLinkFrameDataStruct& InOutFrameData)
{
	for (ULiveLinkFramePreProcessor::FWorkerSharedPtr PreProcessor : FramePreProcessors)
	{
		PreProcessor->PreProcessFrame(InStaticData, InOutFrameData);
	}
}

bool FLiveLinkSubject::ValidateFrameData(const FLiveLinkFrameDataStruct& InFrameData)
{
	if (!StaticData.IsValid())
	{
		static const FName InvalidStatFrame = "LiveLinkSubject_InvalidStatFrame";
		FLiveLinkLog::WarningOnce(InvalidStatFrame, SubjectKey, TEXT("Can't add frame for subject '%s'. The static frame data is invalid."), *SubjectKey.SubjectName.ToString());
		return false;
	}

	if (Role == nullptr)
	{
		return false;
	}

	if (Role->GetDefaultObject<ULiveLinkRole>()->GetFrameDataStruct() != InFrameData.GetStruct())
	{
		static const FName NAME_IncompatibleRoles = "LiveLinkSubject_IncompatibleRoles";
		FLiveLinkLog::WarningOnce(NAME_IncompatibleRoles, SubjectKey, TEXT("Can't add frame for subject '%s'. The frame data is incompatible with current role '%s'."), *SubjectKey.SubjectName.ToString(), *Role->GetName());
		return false;
	}

	if (!FLiveLinkRoleTrait::Validate(Role, InFrameData))
	{
		static const FName NAME_UnsupportedFrameData = "LiveLinkSubject_UnsupportedFrameData";
		FLiveLinkLog::WarningOnce(NAME_UnsupportedFrameData, SubjectKey, TEXT("Trying to add unsupported frame data type to role '%s'."), *Role->GetName());
		return false;
	}

	return true;
}

void FLiveLinkSubject::ClearOverrideStaticData_AnyThread()
{
	bClearOverrideStaticData = true;
}

int32 FLiveLinkSubject::FindNewFrame_WorldTime(const FLiveLinkWorldTime& WorldTime) const
{
	if (CachedSettings.BufferSettings.bValidEngineTimeEnabled)
	{
		//Offset the reading position with clock offset and use source time directly
		const double ValidEngineTime = FApp::GetCurrentTime() - CachedSettings.BufferSettings.EngineTimeOffset - CachedSettings.BufferSettings.ValidEngineTime - CachedSettings.BufferSettings.EngineTimeClockOffset;
		const double SourceTime = WorldTime.GetSourceTime();
		if (SourceTime < ValidEngineTime)
		{
			static const FName NAME_InvalidWorldTime = "LiveLinkSubject_InvalidWorldTIme";
			FLiveLinkLog::WarningOnce(NAME_InvalidWorldTime, SubjectKey, TEXT("Trying to add a frame in which the world time has a value too low compare to the engine's time. Do you have an invalid offset? The Subject is '%s'."), *SubjectKey.SubjectName.ToString());
		}
	}

	return FindNewFrame_WorldTimeInternal(WorldTime);
}

int32 FLiveLinkSubject::FindNewFrame_WorldTimeInternal(const FLiveLinkWorldTime& WorldTime) const
{
	//When inserting our new frame, use the source time of each packet. 
	int32 FrameIndex = FrameData.Num() - 1;
	const double NewFrameSourceTime = WorldTime.GetSourceTime();
	for (; FrameIndex >= 0; --FrameIndex)
	{
		const double FrameSourceTime = FrameData[FrameIndex].GetBaseData()->WorldTime.GetSourceTime();
		if (FrameSourceTime <= NewFrameSourceTime)
		{
			if (FMath::IsNearlyEqual(FrameSourceTime, NewFrameSourceTime))
			{
				static const FName NAME_SameWorldTime = "LiveLinkSubject_SameWorldTime";
				FLiveLinkLog::WarningOnce(NAME_SameWorldTime, SubjectKey, TEXT("A new frame data for subject '%s' has the same time as a previous frame."), *SubjectKey.SubjectName.ToString());
			}
			break;
		}
	}

	return FrameIndex + 1;
}

int32 FLiveLinkSubject::FindNewFrame_SceneTime(const FQualifiedFrameTime& QualifiedFrameTime, const FLiveLinkWorldTime& WorldTime) const
{
	if (QualifiedFrameTime.Time.FloorToFrame() < 0)
	{
		static const FName NAME_NoSceneTime = "LiveLinkSubject_NoSceneTime";
		FLiveLinkLog::ErrorOnce(NAME_NoSceneTime, SubjectKey, TEXT("Trying to add a frame that does not have a valid scene time (timecode). The Subject is '%s'."), *SubjectKey.SubjectName.ToString());
		return INDEX_NONE;
	}

	// If we do not have a TC set, keep buffering, the TC may be unresponsive for a moment. We do not want to lose data.
	if (FApp::GetCurrentFrameTime().IsSet() && CachedSettings.BufferSettings.bValidTimecodeFrameEnabled)
	{
		FFrameTime TimecodeClockOffsetTime;
		if (CachedSettings.BufferSettings.bUseTimecodeSmoothLatest)
		{
			TimecodeClockOffsetTime = LastTimecodeFrameRate.AsFrameTime(CachedSettings.BufferSettings.TimecodeClockOffset);
		}

		const FQualifiedFrameTime CurrentSyncTime = FApp::GetCurrentFrameTime().GetValue();
		const FFrameTime CurrentFrameTimeInFrameSpace = CurrentSyncTime.ConvertTo(LastTimecodeFrameRate);
		const FFrameTime CurrentOffsetFrameTime = CurrentFrameTimeInFrameSpace - FFrameTime::FromDecimal(CachedSettings.BufferSettings.TimecodeFrameOffset) - CachedSettings.BufferSettings.ValidTimecodeFrame - TimecodeClockOffsetTime;
		if (QualifiedFrameTime.Time.AsDecimal() < CurrentOffsetFrameTime.AsDecimal())
		{
			static const FName NAME_InvalidTC = "LiveLinkSubject_InvalidTC";
			FLiveLinkLog::WarningOnce(NAME_InvalidTC, SubjectKey, TEXT("Trying to add a frame in which the timecode has a value too low compare to the engine's timecode. Do you have an invalid offset?. The Subject is '%s'."), *SubjectKey.SubjectName.ToString());
		}
	}

	if (CachedSettings.BufferSettings.bGenerateSubFrame)
	{
		// match with frame number, then look at the world time

		int32 MinInclusive = FrameData.Num() - 1;
		for (; MinInclusive >= 0; --MinInclusive)
		{
			const FFrameNumber FrameFrameNumber = FrameData[MinInclusive].GetBaseData()->MetaData.SceneTime.Time.GetFrame();
			if (QualifiedFrameTime.Time.GetFrame() > FrameFrameNumber)
			{
				break;
			}
		}
		if (MinInclusive < 0)
		{
			return 0;
		}
		++MinInclusive;
		if (MinInclusive >= FrameData.Num())
		{
			return FrameData.Num();
		}

		int32 MaxInclusive = MinInclusive;
		for (; MaxInclusive < FrameData.Num(); ++MaxInclusive)
		{
			const FFrameNumber FrameFrameNumber = FrameData[MaxInclusive].GetBaseData()->MetaData.SceneTime.Time.GetFrame();
			if (QualifiedFrameTime.Time.GetFrame() != FrameFrameNumber)
			{
				break;
			}
		}
		--MaxInclusive;

		//When ordering sub frames, use source time to see which frame comes first
		const double NewFrameSourceTime = WorldTime.GetSourceTime();
		int32 FrameIndex = MaxInclusive;
		for (; FrameIndex >= MinInclusive; --FrameIndex)
		{
			const double FrameSourceTime = FrameData[FrameIndex].GetBaseData()->WorldTime.GetSourceTime();
			if (FrameSourceTime <= NewFrameSourceTime)
			{
				if (FMath::IsNearlyEqual(FrameSourceTime, NewFrameSourceTime))
				{
					static const FName NAME_SameWorldSceneTime = "LiveLinkSubject_SameWorldSceneTime";
					FLiveLinkLog::WarningOnce(NAME_SameWorldSceneTime, SubjectKey, TEXT("A new frame data for subject '%s' has the same timecode and the same time as a previous frame."), *SubjectKey.SubjectName.ToString());
				}
				break;
			}
		}

		return FrameIndex + 1;
	}
	else
	{
		const double NewFrameQFTSeconds = QualifiedFrameTime.AsSeconds();
		int32 FrameIndex = FrameData.Num() - 1;
		for (; FrameIndex >= 0; --FrameIndex)
		{
			const double FrameQFTSeconds = FrameData[FrameIndex].GetBaseData()->MetaData.SceneTime.AsSeconds();
			if (FrameQFTSeconds <= NewFrameQFTSeconds)
			{
				if (FMath::IsNearlyEqual(FrameQFTSeconds, NewFrameQFTSeconds))
				{
					static const FName NAME_SameSceneTime = "LiveLinkSubject_SameSceneTime";
					FLiveLinkLog::WarningOnce(NAME_SameSceneTime, SubjectKey, TEXT("A new frame data for subject '%s' has the same timecode as a previous frame."), *SubjectKey.SubjectName.ToString());
				}
				break;
			}
		}

		return FrameIndex + 1;
	}
}

int32 FLiveLinkSubject::FindNewFrame_Latest(const FLiveLinkWorldTime& WorldTime) const
{
	return FindNewFrame_WorldTimeInternal(WorldTime);
}

void FLiveLinkSubject::AdjustSubFrame_SceneTime(int32 InFrameIndex)
{
	// We need to generate sub frame after because network timing could affect how the frame come in LiveLink

	const double SourceFrameRate = CachedSettings.BufferSettings.SourceTimecodeFrameRate.AsDecimal(); //ie. 120
	const double TimecodeFrameRateDec = LastTimecodeFrameRate.AsDecimal(); //ie. 30
	float SubFrameIncrement = TimecodeFrameRateDec / SourceFrameRate;

	check(CachedSettings.BufferSettings.bGenerateSubFrame);

	// find max and lower limit for TC with InFrameIndex
	int32 HigherInclusiveLimit = InFrameIndex;
	int32 LowerInclusiveLimit = InFrameIndex;

	const FFrameNumber FrameNumber = FrameData[InFrameIndex].GetBaseData()->MetaData.SceneTime.Time.FrameNumber;
	for (; LowerInclusiveLimit >= 0; --LowerInclusiveLimit)
	{
		FFrameNumber LowerFrameNumber = FrameData[LowerInclusiveLimit].GetBaseData()->MetaData.SceneTime.Time.FrameNumber;
		if (FrameNumber != LowerFrameNumber)
		{
			break;
		}
	}
	LowerInclusiveLimit = FMath::Clamp(LowerInclusiveLimit + 1, 0, FrameData.Num()-1);

	for (; HigherInclusiveLimit < FrameData.Num(); ++HigherInclusiveLimit)
	{
		FFrameNumber HigherFrameNumber = FrameData[HigherInclusiveLimit].GetBaseData()->MetaData.SceneTime.Time.FrameNumber;
		if (FrameNumber != HigherFrameNumber)
		{
			break;
		}
	}
	HigherInclusiveLimit = FMath::Clamp(HigherInclusiveLimit, LowerInclusiveLimit, FrameData.Num()-1);

	// order them by world time
	check(LowerInclusiveLimit <= HigherInclusiveLimit);
	if (LowerInclusiveLimit < HigherInclusiveLimit)
	{
		Algo::SortBy(MakeArrayView(&FrameData[LowerInclusiveLimit], HigherInclusiveLimit - LowerInclusiveLimit + 1), 
			[](const FLiveLinkFrameDataStruct& Data)
			{
				//Use SourceTime to make final granular sorting when generating subframes. 
				//@note Should be updated to have frame numbers associated to each packet to correctly apply subframes
				return Data.GetBaseData()->WorldTime.GetSourceTime();
			});

		// generate sub frame

		if (HigherInclusiveLimit - LowerInclusiveLimit >= static_cast<int32>(1.f / SubFrameIncrement))
		{
			static const FName NAME_TooManyFrameForGenerateSubFrame = "LiveLinkSubject_TooManyFrameForGenerateSubFrame";
			FLiveLinkLog::WarningOnce(NAME_TooManyFrameForGenerateSubFrame, SubjectKey, TEXT("For subject '%s' they are too many frames with the same timecode that exist to create subframe. Check the Frame Rate?"), *SubjectKey.SubjectName.ToString());
			SubFrameIncrement = 1.f / (HigherInclusiveLimit - LowerInclusiveLimit + 1);
		}

		float CurrentIncrement = 0.0;
		for (int32 FrameIndex = LowerInclusiveLimit; FrameIndex <= HigherInclusiveLimit; ++FrameIndex)
		{
			FrameData[FrameIndex].GetBaseData()->MetaData.SceneTime.Time = FFrameTime(FrameNumber, CurrentIncrement);
			CurrentIncrement += SubFrameIncrement;
		}
	}
}

bool FLiveLinkSubject::GetFrameAtWorldTime(const double InSeconds, FLiveLinkSubjectFrameData& OutFrame)
{
	bool bResult = FrameData.Num() != 0;
	if (bResult)
	{
		if (FrameInterpolationProcessor.IsValid())
		{
			bResult = GetFrameAtWorldTime_Interpolated(InSeconds, OutFrame);
		}
		else
		{
			bResult = GetFrameAtWorldTime_Closest(InSeconds, OutFrame);
		}

		if (bResult && !OutFrame.StaticData.IsValid())
		{
			OutFrame.StaticData.InitializeWith(StaticData.GetStruct(), StaticData.GetBaseData());
		}

		//Apply clock offset of our source so outside frame has the current offset built in
		OutFrame.FrameData.GetBaseData()->WorldTime.SetClockOffset(CachedSettings.BufferSettings.EngineTimeClockOffset);
	}
	else if (IsBufferStatsEnabled())
	{
		IncreaseBufferOverFlowStat();
	}
	return bResult;
}

bool FLiveLinkSubject::GetFrameAtWorldTime_Closest(const double InSeconds, FLiveLinkSubjectFrameData& OutFrame)
{
	check(FrameData.Num() != 0);

	bool bOverflowDetected = false;
	bool bUnderflowDetected = false;
	bool bBuiltFrame = false;
	for (int32 FrameIndex = FrameData.Num() - 1; FrameIndex >= 0; --FrameIndex)
	{
		const double FrameTime = FrameData[FrameIndex].GetBaseData()->WorldTime.GetSourceTime();
		if (FrameTime <= InSeconds)
		{
			if (FrameIndex == FrameData.Num() - 1)
			{
				//Copy over the frame directly
				OutFrame.FrameData.InitializeWith(FrameData[FrameIndex]);
				bBuiltFrame = true;

				//If we tried to read above our buffer, stamp an overflow
				bUnderflowDetected = !FMath::IsNearlyEqual(FrameTime, InSeconds);
				break;
			}
			else
			{
				const double NextTime = FrameData[FrameIndex + 1].GetBaseData()->WorldTime.GetSourceTime();
				const float BlendWeight = (InSeconds - NextTime) / (NextTime - FrameTime);
				const int32 CopyIndex = (BlendWeight > 0.5f) ? FrameIndex : FrameIndex + 1;
				OutFrame.FrameData.InitializeWith(FrameData[CopyIndex].GetStruct(), FrameData[CopyIndex].GetBaseData());
				bBuiltFrame = true;
				break;
			}
		}
	}

	if (!bBuiltFrame)
	{
		// Failed to find an interp point so just take oldest frame
		OutFrame.FrameData.InitializeWith(FrameData[0].GetStruct(), FrameData[0].GetBaseData());
		bUnderflowDetected = true;
	}
	
	if (IsBufferStatsEnabled())
	{
		if (bUnderflowDetected)
		{
			IncreaseBufferUnderFlowStat();
		}

		if (bOverflowDetected)
		{
			IncreaseBufferOverFlowStat();
		}

		FTimedDataInputEvaluationData EvaluationData;
		EvaluationData.DistanceToNewestSampleSeconds = FrameData[FrameData.Num()-1].GetBaseData()->WorldTime.GetSourceTime() - InSeconds;
		EvaluationData.DistanceToOldestSampleSeconds = InSeconds - FrameData[0].GetBaseData()->WorldTime.GetSourceTime();
		UpdateEvaluationData(EvaluationData);
	}

	return true;
}

bool FLiveLinkSubject::GetFrameAtWorldTime_Interpolated(const double InSeconds, FLiveLinkSubjectFrameData& OutFrame)
{
	check(FrameData.Num() != 0);

	FLiveLinkInterpolationInfo InterpolationInfo;
	FrameInterpolationProcessor->Interpolate(InSeconds, StaticData, FrameData, OutFrame, InterpolationInfo);

	UE_LOGF(LogLiveLink, Verbose, "Subject '%ls' interpolating between frames %d and %d"
		, *SubjectKey.SubjectName.ToString()
		, InterpolationInfo.FrameIndexA
		, InterpolationInfo.FrameIndexB);

	VerifyInterpolationInfo(InterpolationInfo);

	return true;
}

bool FLiveLinkSubject::GetFrameAtSceneTime(const FQualifiedFrameTime& InTimeInEngineFrameRate, FLiveLinkSubjectFrameData& OutFrame)
{
	bool bResult = FrameData.Num() != 0;
	if (bResult)
	{
		if (FrameInterpolationProcessor.IsValid())
		{
			bResult = GetFrameAtSceneTime_Interpolated(InTimeInEngineFrameRate, OutFrame);
		}
		else
		{
			bResult = GetFrameAtSceneTime_Closest(InTimeInEngineFrameRate, OutFrame);
		}

		if (bResult && !OutFrame.StaticData.IsValid())
		{
			OutFrame.StaticData.InitializeWith(StaticData.GetStruct(), StaticData.GetBaseData());
		}
	}
	else if (IsBufferStatsEnabled())
	{
		IncreaseBufferOverFlowStat();
	}
	return bResult;
}

bool FLiveLinkSubject::GetFrameAtSceneTime_Closest(const FQualifiedFrameTime& InTimeInEngineFrameRate, FLiveLinkSubjectFrameData& OutFrame)
{
	check(FrameData.Num() != 0);

	bool bUnderflowDetected = false;
	bool bOverflowDetected = false;
	bool bBuiltFrame = false;

	const double TimeInSeconds = InTimeInEngineFrameRate.AsSeconds();
	for (int32 FrameIndex = FrameData.Num() - 1; FrameIndex >= 0; --FrameIndex)
	{
		const double FrameASeconds = FrameData[FrameIndex].GetBaseData()->MetaData.SceneTime.AsSeconds();
		if (FrameASeconds <= TimeInSeconds)
		{
			if (FrameIndex == FrameData.Num() - 1)
			{
				//Copy over the frame directly
				OutFrame.FrameData.InitializeWith(FrameData[FrameIndex]);
				bBuiltFrame = true;

				//We were asking for a frame above our newest one?
				bOverflowDetected = !FMath::IsNearlyEqual(FrameASeconds, TimeInSeconds);
				break;
			}
			else
			{
				const double FrameBSeconds = FrameData[FrameIndex+1].GetBaseData()->MetaData.SceneTime.AsSeconds();
				const double BlendWeight = (TimeInSeconds - FrameASeconds) / (FrameBSeconds - FrameASeconds);
				int32 CopyIndex = (BlendWeight > 0.5) ? FrameIndex : FrameIndex + 1;
				OutFrame.FrameData.InitializeWith(FrameData[CopyIndex].GetStruct(), FrameData[CopyIndex].GetBaseData());
				bBuiltFrame = true;
				break;
			}
		}
	}

	if (!bBuiltFrame)
	{
		bUnderflowDetected = true;

		// Failed to find an interp point so just take oldest frame
		OutFrame.FrameData.InitializeWith(FrameData[0].GetStruct(), FrameData[0].GetBaseData());
	}

	if (IsBufferStatsEnabled())
	{
		if (bUnderflowDetected)
		{
			IncreaseBufferUnderFlowStat();
		}

		if (bOverflowDetected)
		{
			IncreaseBufferOverFlowStat();
		}

		FTimedDataInputEvaluationData EvaluationData;
		EvaluationData.DistanceToNewestSampleSeconds = FrameData[FrameData.Num() - 1].GetBaseData()->MetaData.SceneTime.AsSeconds() - TimeInSeconds;
		EvaluationData.DistanceToOldestSampleSeconds = TimeInSeconds - FrameData[0].GetBaseData()->MetaData.SceneTime.AsSeconds();
		UpdateEvaluationData(EvaluationData);
	}

	return true;
}

bool FLiveLinkSubject::GetFrameAtSceneTime_Interpolated(const FQualifiedFrameTime& InTimeInEngineFrameRate, FLiveLinkSubjectFrameData& OutFrame)
{
	check(FrameData.Num() != 0);

	FLiveLinkInterpolationInfo InterpolationInfo;
	FrameInterpolationProcessor->Interpolate(InTimeInEngineFrameRate, StaticData, FrameData, OutFrame, InterpolationInfo);

	if (IsBufferStatsEnabled())
	{
		VerifyInterpolationInfo(InterpolationInfo);
	}

	return true;
}

void FLiveLinkSubject::VerifyInterpolationInfo(const FLiveLinkInterpolationInfo& InterpolationInfo)
{
	if (InterpolationInfo.bOverflowDetected)
	{
		IncreaseBufferOverFlowStat();
	}
	else if (InterpolationInfo.bUnderflowDetected)
	{
		IncreaseBufferUnderFlowStat();
	}

	FTimedDataInputEvaluationData EvaluationData;
	EvaluationData.DistanceToNewestSampleSeconds = InterpolationInfo.ExpectedEvaluationDistanceFromNewestSeconds;
	EvaluationData.DistanceToOldestSampleSeconds = InterpolationInfo.ExpectedEvaluationDistanceFromOldestSeconds;
	UpdateEvaluationData(EvaluationData);
}

bool FLiveLinkSubject::GetLatestFrame(FLiveLinkSubjectFrameData& OutFrame)
{
	bool bResult = FrameData.Num() != 0;
	if (bResult)
	{
		bool bUnderflowDetected = false;
		bool bOverflowDetected = false;

		int32 Index = FrameData.Num() - 1 - CachedSettings.BufferSettings.LatestOffset;
		if (Index >= FrameData.Num())
		{
			Index = FrameData.Num() - 1;
			bOverflowDetected = true;
		}
		else if (Index < 0)
		{
			Index = 0;
			bUnderflowDetected = true;
		}

		check(FrameData.IsValidIndex(Index));

		FLiveLinkFrameDataStruct& LastDataStruct = FrameData[Index];
		OutFrame.FrameData.InitializeWith(LastDataStruct.GetStruct(), LastDataStruct.GetBaseData());
		OutFrame.StaticData.InitializeWith(StaticData.GetStruct(), StaticData.GetBaseData());

		if (IsBufferStatsEnabled())
		{
			if (bUnderflowDetected)
			{
				IncreaseBufferUnderFlowStat();
			}

			if (bOverflowDetected)
			{
				IncreaseBufferOverFlowStat();
			}

			FTimedDataInputEvaluationData EvaluationData;
			EvaluationData.DistanceToNewestSampleSeconds = FrameData[FrameData.Num() - 1].GetBaseData()->WorldTime.GetOffsettedTime() - OutFrame.FrameData.GetBaseData()->WorldTime.GetOffsettedTime();
			EvaluationData.DistanceToOldestSampleSeconds = OutFrame.FrameData.GetBaseData()->WorldTime.GetOffsettedTime() - FrameData[0].GetBaseData()->MetaData.SceneTime.AsSeconds();
			UpdateEvaluationData(EvaluationData);
		}
	}

	return bResult;
}

void FLiveLinkSubject::ResetFrame(FLiveLinkSubjectFrameData& OutFrame) const
{
	//Allocate and copy over our static data for that frame.
	OutFrame.StaticData.InitializeWith(StaticData);

	//Only reset the frame data. Copy will be done later on depending on sampling type
	OutFrame.FrameData.Reset();
}

void FLiveLinkSubject::IncreaseFrameDroppedStat()
{
	++EvaluationStatistics.FrameDrop;
}

void FLiveLinkSubject::IncreaseBufferUnderFlowStat()
{
	++EvaluationStatistics.BufferUnderflow;
}

void FLiveLinkSubject::IncreaseBufferOverFlowStat()
{
	++EvaluationStatistics.BufferOverflow;
}

void FLiveLinkSubject::UpdateEvaluationData(const FTimedDataInputEvaluationData& EvaluationData)
{
	FScopeLock Lock(&StatisticCriticalSection);
	EvaluationStatistics.LastEvaluationData = EvaluationData;
}

void FLiveLinkSubject::RemoveFrames(int32 InCount)
{
	for (int32 i = 0; i < InCount; ++i)
	{
		FLiveLinkFrameIdentifier FrameToRemove = INDEX_NONE;
		ReceivedOrderedFrames.Dequeue(FrameToRemove);
		const int32 FrameToRemoveIndex = FrameData.IndexOfByPredicate([FrameToRemove](const FLiveLinkFrameDataStruct& Other) { return Other.GetBaseData()->FrameId == FrameToRemove; });
		if (FrameToRemoveIndex != INDEX_NONE)
		{
			const int32 Count = 1;
			FrameData.RemoveAt(FrameToRemoveIndex, Count, EAllowShrinking::No);
		}
		else
		{
			FLiveLinkLog::Error(TEXT("Subject '%s': Could not find frame with identifier %d. Buffers are out of sync, resetting them."), *SubjectKey.SubjectName.ToString(), FrameToRemove);
			ReceivedOrderedFrames.Empty();
			FrameData.Reset();
		}
	}
}

void FLiveLinkSubject::UpdateState()
{
	const ELiveLinkSubjectState PreviousState = State;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAppTime Time = FAppTime::Get_UnsafeNoCheck();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const bool bHasValidFrame = (Time.CurrentTime - GetLastPushTime() < GetDefault<ULiveLinkSettings>()->GetTimeWithoutFrameToBeConsiderAsInvalid());
	State = (bHasValidFrame && HasValidFrameSnapshot()) ? ELiveLinkSubjectState::Connected : ELiveLinkSubjectState::Unresponsive;

	if (State != PreviousState)
	{
		OnStateChangedDelegate.ExecuteIfBound(State);
	}
}

void FLiveLinkSubject::SetStaticData(TSubclassOf<ULiveLinkRole> InRole, FLiveLinkStaticDataStruct&& InStaticData)
{
	if (Role == nullptr)
	{
		static const FName NAME_NoRoleForSubject = "LiveLinkSubject_NoRoleForSubject";
		FLiveLinkLog::WarningOnce(NAME_NoRoleForSubject, SubjectKey, TEXT("Setting static data for Subject '%s' before it was initialized."), *SubjectKey.SubjectName.ToString());
		return;
	}

	if (Role == InRole)
	{
		//Set initial blending processor to the role's default one. User will be able to modify it afterwards.
		FrameData.Reset();
		ReceivedOrderedFrames.Empty();
		StaticData = MoveTemp(InStaticData);
		// SetStaticData can be called after caching settings and checking for a dirty remapper. But the remapper needs to be applied
		// to this new static data, so remap again when caching settings.
		bNeedsStaticRemap = OverrideStaticData.IsSet();
	}
	else
	{
		static const FName NAME_DifferentRole = "LiveLinkSubject_DifferentRole";
		FLiveLinkLog::WarningOnce(NAME_DifferentRole, SubjectKey, TEXT("Subject '%s' received data of role %s but was already registered with a different role"), *SubjectKey.SubjectName.ToString(), *InRole->GetName());
	}	
}

void FLiveLinkSubject::CacheSettings(ULiveLinkSourceSettings* SourceSetting, ULiveLinkSubjectSettings* SubjectSetting)
{
	FScopeLock Lock(&SettingsCriticalSection);

	if (SourceSetting)
	{
		const bool bSourceModeChanged = SourceSetting->Mode != CachedSettings.SourceMode;
		const bool bGenerateSubFrameChanged = SourceSetting->Mode == ELiveLinkSourceMode::Timecode && SourceSetting->BufferSettings.bGenerateSubFrame != CachedSettings.BufferSettings.bGenerateSubFrame;
		const bool bTimecodeModeChanged = SourceSetting->Mode == ELiveLinkSourceMode::Timecode && SourceSetting->BufferSettings.bUseTimecodeSmoothLatest != CachedSettings.BufferSettings.bUseTimecodeSmoothLatest;
		if (bSourceModeChanged || bGenerateSubFrameChanged || bTimecodeModeChanged)
		{
			FrameData.Reset();
			ReceivedOrderedFrames.Empty();
		}

		CachedSettings.SourceMode = SourceSetting->Mode;
		CachedSettings.BufferSettings = SourceSetting->BufferSettings;

		// Test and update values
		{
			CachedSettings.BufferSettings.MaxNumberOfFrameToBuffered = FMath::Max(CachedSettings.BufferSettings.MaxNumberOfFrameToBuffered, 1);
			if (CachedSettings.BufferSettings.bGenerateSubFrame)
			{
				const double SourceFrameRate = CachedSettings.BufferSettings.SourceTimecodeFrameRate.AsDecimal(); //ie. 120
				const double TimecodeFrameRateDec = LastTimecodeFrameRate.AsDecimal(); //ie. 30
				if (SourceFrameRate <= TimecodeFrameRateDec)
				{
					CachedSettings.BufferSettings.bGenerateSubFrame = false;

					static const FName NAME_CanGenerateSubFrame = "LiveLinkSubject_CantGenerateSubFrame";
					FLiveLinkLog::WarningOnce(NAME_CanGenerateSubFrame, SubjectKey, TEXT("Can't generate Sub Frame because the 'Subject's Timecode Frame Rate' is bigger or equal to the 'Source Timecode Frame Rate'"));
				}
			}
		}
	}

	if (SubjectSetting)
	{
		//Stamp our frame rate estimation in settings to be visible
		SubjectSetting->FrameRate = LastTimecodeFrameRate;

		// Copy the rebroadcast flag from the incoming settings
		bRebroadcastSubject = SubjectSetting->bRebroadcastSubject;

		// Create a new or fetch the PreProcessors for this frame
		FramePreProcessors.Reset();
		for (ULiveLinkFramePreProcessor* PreProcessor : SubjectSetting->PreProcessors)
		{
			if (PreProcessor)
			{
				ULiveLinkFramePreProcessor::FWorkerSharedPtr NewPreProcessor = PreProcessor->FetchWorker();
				if (NewPreProcessor.IsValid())
				{
					FramePreProcessors.Add(NewPreProcessor);
				}
			}
		}

		SubjectRemapper.Reset();
		
		if (SubjectSetting->Remapper
#if WITH_EDITOR
		/** Remappers may call FinishCompilation for skinned assets, which isn't safe to do if we are running in a different thread. */
		&& (IsInGameThread() || FSkinnedAssetCompilingManager::Get().GetNumRemainingJobs() == 0)
#endif
		)
		{
			// If there wasn't a remapper, then we need to initialize the one we will create.
			bool bRecreateRemapper = SubjectSetting->Remapper->GetWorker() == nullptr || SubjectSetting->Remapper->bDirty || bNeedsStaticRemap;
			bNeedsStaticRemap = false;

			ULiveLinkSubjectRemapper::FWorkerSharedPtr NewRemapper = bRecreateRemapper ? SubjectSetting->Remapper->CreateWorker() : SubjectSetting->Remapper->GetWorker();

			if (NewRemapper.IsValid())
			{
				// If this is a new remapper, then do an initial remapping.
				if (bRecreateRemapper)
				{
					SubjectSetting->Remapper->bDirty = false;

					if (SubjectSetting->Remapper->IsValidRemapper())
					{
						// Since the remapper has changed, we need to update the static data as well. 
						FLiveLinkStaticDataStruct RemappedStaticData;
						RemappedStaticData.InitializeWith(StaticData);
						NewRemapper->RemapStaticData(RemappedStaticData);

						if (NewRemapper->IsRemapperCompatible(StaticData))
						{
							// Protect FrameSnapshot access during remapping
							{
								FWriteScopeLock WriteLock(FrameSnapshotRWLock);
								if (FrameSnapshot.StaticData.IsValid() && FrameSnapshot.FrameData.IsValid())
								{
									NewRemapper->RemapStaticData(FrameSnapshot.StaticData);
									NewRemapper->RemapFrameData(FrameSnapshot.StaticData, FrameSnapshot.FrameData);
								}
							}

							// Important: Because we changed the static data, we have to update the frame data to match the new static data.
							for (FLiveLinkFrameDataStruct& FrameDataStruct : FrameData)
							{
								NewRemapper->RemapFrameData(RemappedStaticData, FrameDataStruct);
							}

							OverrideStaticData = MoveTemp(RemappedStaticData);

							SetStaticDataAsRebroadcasted(false);
						}
						else
						{
							UE_LOGF(LogLiveLink, Warning, "Remapper is not compatible with this static data.");
						}
					}
					else
					{
						// Remapper isn't valid, reset static data.
						ClearFrames();

						OverrideStaticData.Reset();
					}

				}
			}

			SubjectRemapper = MoveTemp(NewRemapper);
		}

		// Create a new or fetch the interpolation for this frame
		FrameInterpolationProcessor.Reset();
		if (SubjectSetting->InterpolationProcessor)
		{
			FrameInterpolationProcessor = SubjectSetting->InterpolationProcessor->FetchWorker();
		}

		// Create a new or fetch the translators for this frame
		FrameTranslators.Reset();
		for (ULiveLinkFrameTranslator* Translator : SubjectSetting->Translators)
		{
			if (Translator)
			{
				ULiveLinkFrameTranslator::FWorkerSharedPtr NewTranslator = Translator->FetchWorker();
				if (NewTranslator.IsValid())
				{
					FrameTranslators.Add(NewTranslator);
				}
			}
		}
	}
}

ELiveLinkSourceMode FLiveLinkSubject::GetMode() const
{
	return ModeOverride.Get(CachedSettings.SourceMode);
}

FLiveLinkSubjectTimeSyncData FLiveLinkSubject::GetTimeSyncData()
{
	FLiveLinkSubjectTimeSyncData SyncData;
	SyncData.bIsValid = FrameData.Num() > 0;

	if (SyncData.bIsValid)
	{
		SyncData.NewestSampleTime = FrameData.Last().GetBaseData()->MetaData.SceneTime.Time;
		SyncData.OldestSampleTime = FrameData[0].GetBaseData()->MetaData.SceneTime.Time;
		SyncData.SampleFrameRate = FrameData[0].GetBaseData()->MetaData.SceneTime.Rate;
	}

	return SyncData;
}

bool FLiveLinkSubject::IsTimeSynchronized() const
{
	if (GetMode() == ELiveLinkSourceMode::Timecode)
	{
		const FLiveLinkSubjectFrameData& Snapshot = GetFrameSnapshot();
		if (Snapshot.StaticData.IsValid() && Snapshot.FrameData.IsValid() && Snapshot.FrameData.GetBaseData() && FApp::GetCurrentFrameTime().IsSet())
		{
			const FQualifiedFrameTime CurrentQualifiedFrameTime = FApp::GetCurrentFrameTime().GetValue();
			const FFrameNumber FrameDataInEngineFrameNumber = Snapshot.FrameData.GetBaseData()->MetaData.SceneTime.ConvertTo(CurrentQualifiedFrameTime.Rate).GetFrame();
			const FFrameNumber CurrentEngineFrameNumber = CurrentQualifiedFrameTime.Time.GetFrame();
			return FrameDataInEngineFrameNumber == CurrentEngineFrameNumber;
		}
	}
	return false;
}

/**
 * ITimedDataInput interface
 */
FText FLiveLinkSubject::GetDisplayName() const
{
	return FText::FromName(SubjectKey.SubjectName);
}

ETimedDataInputState FLiveLinkSubject::GetState() const
{
	using enum ELiveLinkSubjectState;
	switch (State)
	{
		case Connected: return ETimedDataInputState::Connected;
		case Unresponsive: return ETimedDataInputState::Unresponsive;
		case Disconnected: return ETimedDataInputState::Disconnected;
		default:
			ensureMsgf(false, TEXT("Unhandled ELiveLinkSubjectState::%d"), EnumToUnderlyingType(State));
			return ETimedDataInputState::Disconnected;
	}
}

FTimedDataChannelSampleTime FLiveLinkSubject::GetOldestDataTime() const
{
	FScopeLock Lock(&CachedTimedDataMonitorInfoCS);
	return CachedTimeDataMonitorInfo.OldestDataTime;
}

void FLiveLinkSubject::UpdateCachedOldestDataTime()
{
	FScopeLock Lock(&CachedTimedDataMonitorInfoCS);

	CachedTimeDataMonitorInfo.OldestDataTime = FTimedDataChannelSampleTime();

	if (FrameData.Num() > 0)
	{
		//For timed data, adjust frame timings to match with clock offset to have valid timing diagrams
		FFrameTime TimecodeClockOffsetTime;
		if (CachedSettings.BufferSettings.bUseTimecodeSmoothLatest)
		{
			TimecodeClockOffsetTime = FrameData[0].GetBaseData()->MetaData.SceneTime.Rate.AsFrameTime(CachedSettings.BufferSettings.TimecodeClockOffset);
		}

		const double OffsettedOldestTime = FrameData[0].GetBaseData()->WorldTime.GetSourceTime() + CachedSettings.BufferSettings.EngineTimeClockOffset + CachedSettings.BufferSettings.SmoothEngineTimeOffset;
		const FQualifiedFrameTime OldestSceneTime = FQualifiedFrameTime(FrameData[0].GetBaseData()->MetaData.SceneTime.Time + TimecodeClockOffsetTime, FrameData[0].GetBaseData()->MetaData.SceneTime.Rate);

		CachedTimeDataMonitorInfo.OldestDataTime = FTimedDataChannelSampleTime(OffsettedOldestTime, OldestSceneTime);
	}
}

FTimedDataChannelSampleTime FLiveLinkSubject::GetNewestDataTime() const
{
	FScopeLock Lock(&CachedTimedDataMonitorInfoCS);
	return CachedTimeDataMonitorInfo.NewestDataTime;
}

void FLiveLinkSubject::UpdateCachedNewestDataTime()
{
	FScopeLock Lock(&CachedTimedDataMonitorInfoCS);

	CachedTimeDataMonitorInfo.NewestDataTime = FTimedDataChannelSampleTime();

	if (FrameData.Num() > 0)
	{
		//For timed data, adjust frame timings to match with clock offset to have valid timing diagrams
		FFrameTime TimecodeClockOffsetTime;
		if (CachedSettings.BufferSettings.bUseTimecodeSmoothLatest)
		{
			TimecodeClockOffsetTime = FrameData.Last().GetBaseData()->MetaData.SceneTime.Rate.AsFrameTime(CachedSettings.BufferSettings.TimecodeClockOffset);
		}

		const double OffsettedNewestTime = FrameData.Last().GetBaseData()->WorldTime.GetSourceTime() + CachedSettings.BufferSettings.EngineTimeClockOffset + CachedSettings.BufferSettings.SmoothEngineTimeOffset;
		const FQualifiedFrameTime NewestSceneTime = FQualifiedFrameTime(FrameData.Last().GetBaseData()->MetaData.SceneTime.Time + TimecodeClockOffsetTime, FrameData.Last().GetBaseData()->MetaData.SceneTime.Rate);
		CachedTimeDataMonitorInfo.NewestDataTime = FTimedDataChannelSampleTime(OffsettedNewestTime, NewestSceneTime);
	}
}

TArray<FTimedDataChannelSampleTime> FLiveLinkSubject::GetDataTimes() const
{
	FScopeLock Lock(&CachedTimedDataMonitorInfoCS);
	return CachedTimeDataMonitorInfo.DataTimes;
}

void FLiveLinkSubject::UdpateCachedDataTimes()
{
	FScopeLock Lock(&CachedTimedDataMonitorInfoCS);

	CachedTimeDataMonitorInfo.DataTimes.Reset();

	if (FrameData.Num() > 0)
	{
		//For timed data, adjust frame timings to match with clock offset to have valid timing diagrams
		FFrameTime TimecodeClockOffsetTime;
		if (CachedSettings.BufferSettings.bUseTimecodeSmoothLatest)
		{
			TimecodeClockOffsetTime = FrameData[0].GetBaseData()->MetaData.SceneTime.Rate.AsFrameTime(CachedSettings.BufferSettings.TimecodeClockOffset);
		}

		CachedTimeDataMonitorInfo.DataTimes.Reserve(FrameData.Num());
		for (const FLiveLinkFrameDataStruct& Data : FrameData)
		{
			const double OffsettedEngineTime = Data.GetBaseData()->WorldTime.GetSourceTime() + CachedSettings.BufferSettings.EngineTimeClockOffset + CachedSettings.BufferSettings.SmoothEngineTimeOffset;
			const FQualifiedFrameTime AdjustedSceneTime = FQualifiedFrameTime(Data.GetBaseData()->MetaData.SceneTime.Time + TimecodeClockOffsetTime, Data.GetBaseData()->MetaData.SceneTime.Rate);
			CachedTimeDataMonitorInfo.DataTimes.Emplace(OffsettedEngineTime, AdjustedSceneTime);
		}
	}
}

int32 FLiveLinkSubject::GetNumberOfSamples() const
{
	return FrameData.Num();
}

bool FLiveLinkSubject::IsBufferStatsEnabled() const
{
	return bIsStatLoggingEnabled;
}

void FLiveLinkSubject::SetBufferStatsEnabled(bool bEnable)
{
	if (bEnable && !bIsStatLoggingEnabled)
	{
		//When enabling stat tracking, start clean
		ResetBufferStats();
	}

	bIsStatLoggingEnabled = bEnable;
}

int32 FLiveLinkSubject::GetBufferUnderflowStat() const
{
	return EvaluationStatistics.BufferUnderflow;
}

int32 FLiveLinkSubject::GetBufferOverflowStat() const
{
	return EvaluationStatistics.BufferOverflow;
}

int32 FLiveLinkSubject::GetFrameDroppedStat() const
{
	return EvaluationStatistics.FrameDrop;
}

void FLiveLinkSubject::GetLastEvaluationData(FTimedDataInputEvaluationData& OutEvaluationData) const
{
	FScopeLock Lock(&StatisticCriticalSection);
	OutEvaluationData = EvaluationStatistics.LastEvaluationData;
}

void FLiveLinkSubject::ResetBufferStats()
{
	FScopeLock Lock(&StatisticCriticalSection);
	EvaluationStatistics.BufferUnderflow = 0;
	EvaluationStatistics.BufferOverflow = 0;
	EvaluationStatistics.FrameDrop = 0;
	EvaluationStatistics.LastEvaluationData = FTimedDataInputEvaluationData();
}
