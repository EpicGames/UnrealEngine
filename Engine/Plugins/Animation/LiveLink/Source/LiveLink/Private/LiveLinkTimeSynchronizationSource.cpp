// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkTimeSynchronizationSource.h"
#include "LiveLinkLog.h"
#include "Features/IModularFeatures.h"
#include "LiveLinkSourceSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkTimeSynchronizationSource)

ULiveLinkTimeSynchronizationSource::ULiveLinkTimeSynchronizationSource()
{
	if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.OnModularFeatureRegistered().AddUObject(this, &ThisClass::OnModularFeatureRegistered);
		ModularFeatures.OnModularFeatureUnregistered().AddUObject(this, &ThisClass::OnModularFeatureUnregistered);

		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			LiveLinkClient = &ModularFeatures.GetModularFeature<FLiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		}
	}
}

FFrameTime ULiveLinkTimeSynchronizationSource::GetNewestSampleTime() const
{
	UpdateCachedState();
	return CachedData.NewestSampleTime + FrameOffset;
}

FFrameTime ULiveLinkTimeSynchronizationSource::GetOldestSampleTime() const
{
	UpdateCachedState();
	return CachedData.OldestSampleTime + FrameOffset;
}

FFrameRate ULiveLinkTimeSynchronizationSource::GetFrameRate() const
{
	UpdateCachedState();
	return CachedData.SampleFrameRate;
}

bool ULiveLinkTimeSynchronizationSource::IsReady() const
{
	UpdateCachedState();
	return LiveLinkClient && CachedData.bIsValid && IsCurrentStateValid();
}

bool ULiveLinkTimeSynchronizationSource::Open(const FTimeSynchronizationOpenData& OpenData)
{
	State = ESyncState::NotSynced;
	SubjectKey = FLiveLinkSubjectKey();

	if (LiveLinkClient == nullptr)
	{
		return false;
	}

	TArray<FLiveLinkSubjectKey> AllSubjects = LiveLinkClient->GetSubjects(false, false);
	FLiveLinkSubjectKey* SubjectKeyPtr = AllSubjects.FindByPredicate([this](const FLiveLinkSubjectKey& InSubjectKey) { return InSubjectKey.SubjectName == SubjectName; });
	if (SubjectKeyPtr == nullptr)
	{
		FLiveLinkLog::Error(TEXT("The subject '%s' is not valid"), *SubjectName.ToString());
		return false;
	}

	SubjectKey = *SubjectKeyPtr;

	bool bResult = IsCurrentStateValid();

	if (bResult)
	{
		State = ESyncState::Opened;
	}
	return bResult;
}

void ULiveLinkTimeSynchronizationSource::Start(const FTimeSynchronizationStartData& StartData)
{
}

void ULiveLinkTimeSynchronizationSource::Close()
{
	State = ESyncState::NotSynced;
	SubjectKey = FLiveLinkSubjectKey();
}

FString ULiveLinkTimeSynchronizationSource::GetDisplayName() const
{
	return SubjectName.ToString();
}

bool ULiveLinkTimeSynchronizationSource::IsCurrentStateValid() const
{
	ensure(LiveLinkClient != nullptr);
	if (LiveLinkClient == nullptr)
	{
		return false;
	}

	if (!LiveLinkClient->IsSubjectEnabled(SubjectKey, false))
	{
		static const FName NAME_DisabledSubject = "ULiveLinkTimeSynchronizationSource_DisabledSubject";
		FLiveLinkLog::ErrorOnce(NAME_DisabledSubject, SubjectKey, TEXT("The subject '%s' is not enabled."), *SubjectName.ToString());
		return false;
	}

	if (LiveLinkClient->IsVirtualSubject(SubjectKey))
	{
		static const FName NAME_VirtualSubject = "ULiveLinkTimeSynchronizationSource_VirtualSubject";
		FLiveLinkLog::ErrorOnce(NAME_VirtualSubject, SubjectKey, TEXT("The subject '%s' can't be a virtual subject."), *SubjectName.ToString());
		return false;
	}

	ULiveLinkSourceSettings* SourceSettings = LiveLinkClient->GetSourceSettings(SubjectKey.Source);
	if (SourceSettings == nullptr)
	{
		static const FName NAME_InvalidSettings = "ULiveLinkTimeSynchronizationSource_InvalidSettings";
		FLiveLinkLog::ErrorOnce(NAME_InvalidSettings, SubjectKey, TEXT("The subject '%s' source does not have a source settings."), *SubjectName.ToString());
		return false;
	}

	if (SourceSettings->Mode != ELiveLinkSourceMode::Timecode)
	{
		static const FName NAME_InvalidTimeMode = "ULiveLinkTimeSynchronizationSource_InvalidTimeMode";
		FLiveLinkLog::ErrorOnce(NAME_InvalidTimeMode, SubjectKey, TEXT("The subject '%s' source is not in Timecode mode."), *SubjectName.ToString());
		return false;
	}

	return true;
}

void ULiveLinkTimeSynchronizationSource::OnModularFeatureRegistered(const FName& FeatureName, class IModularFeature* Feature)
{
	if (FeatureName == ILiveLinkClient::ModularFeatureName)
	{
		LiveLinkClient = static_cast<FLiveLinkClient*>(Feature);
	}
}

void ULiveLinkTimeSynchronizationSource::OnModularFeatureUnregistered(const FName& FeatureName, class IModularFeature* Feature)
{
	if (FeatureName == ILiveLinkClient::ModularFeatureName && (LiveLinkClient != nullptr) && ensure(Feature == LiveLinkClient))
	{
		LiveLinkClient = nullptr;
	}
}

void ULiveLinkTimeSynchronizationSource::UpdateCachedState() const
{
	if (LastUpdateFrame != GFrameCounter && LiveLinkClient != nullptr)
	{
		LastUpdateFrame = GFrameCounter;
		CachedData = LiveLinkClient->GetTimeSyncData(SubjectName);
	}
}
