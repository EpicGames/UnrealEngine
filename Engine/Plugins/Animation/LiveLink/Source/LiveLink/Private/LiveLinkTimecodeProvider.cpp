// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkTimecodeProvider.h"

#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkClient.h"
#include "LiveLinkCustomTimeStep.h"
#include "LiveLinkTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkTimecodeProvider)


namespace UE::LiveLink::Private
{
static const ULiveLinkCustomTimeStep* GetLiveLinkGenlock()
{
	if (!GEngine)
	{
		return nullptr;
	}

	return Cast<ULiveLinkCustomTimeStep>(GEngine->GetCustomTimeStep());
}
}


ULiveLinkTimecodeProvider::ULiveLinkTimecodeProvider()
	: Evaluation(ELiveLinkTimecodeProviderEvaluationType::Latest)
	, bOverrideFrameRate(false)
	, OverrideFrameRate(24, 1)
	, BufferSize(4)
	, LiveLinkClient(nullptr)
{
}

FQualifiedFrameTime ULiveLinkTimecodeProvider::GetQualifiedFrameTime() const
{
	FScopeLock ScopeLock(&SubjectFrameLock);

	if (SubjectFrameTimes.Num() > 0)
	{
		if (Evaluation == ELiveLinkTimecodeProviderEvaluationType::Latest || SubjectFrameTimes.Num() == 1)
		{
			// When using Latest mode, prefer genlocked timecode unless sourcing from sequencer
			const bool bIsSequencer = bIsSequencerSourceAnyThread.load(std::memory_order_relaxed);

			if (Evaluation == ELiveLinkTimecodeProviderEvaluationType::Latest && !bIsSequencer)
			{
				if (const ULiveLinkCustomTimeStep* LiveLinkCTS = UE::LiveLink::Private::GetLiveLinkGenlock())
				{
					const FQualifiedFrameTime GenlockTime = LiveLinkCTS->GetGenlockTriggerSceneTime();
					return ConvertTo(GenlockTime);
				}
			}

			// Fall back to buffered latest frame
			return ConvertTo(SubjectFrameTimes.Last().SceneTime);
		}
		else
		{
			// Find the frame that is the closest to the current world time.
			const double Seconds = FPlatformTime::Seconds();
			int32 FoundIndex = SubjectFrameTimes.Num() - 1;
			for (; FoundIndex >= 0; --FoundIndex)
			{
				if (SubjectFrameTimes[FoundIndex].WorldTime <= Seconds)
				{
					break;
				}
			}

			check(SubjectFrameTimes.Num() >= 2);
			int32 IndexA = INDEX_NONE;
			int32 IndexB = INDEX_NONE;
			if (FoundIndex < 0)
			{
				// find a time before the frames that we have buffered
				IndexA = 0;
				IndexB = 1;
			}
			else if (FoundIndex >= SubjectFrameTimes.Num()-1)
			{
				// find a time before the frames that we have buffered
				IndexA = SubjectFrameTimes.Num()-2;
				IndexB = SubjectFrameTimes.Num()-1;
			}
			else
			{
				IndexA = FoundIndex;
				IndexB = FoundIndex+1;
			}

			check(IndexA != INDEX_NONE && IndexB != INDEX_NONE);
			// Using the 2 closest frames
			return LerpBetweenFrames(Seconds, IndexA, IndexB);
		}
	}

	return FQualifiedFrameTime(0, OverrideFrameRate);
}


void ULiveLinkTimecodeProvider::ClearFrameHistory()
{
	FScopeLock ScopeLock(&SubjectFrameLock);
	SubjectFrameTimes.Reset();
}

FQualifiedFrameTime ULiveLinkTimecodeProvider::ConvertTo(FQualifiedFrameTime Value) const
{
	if (bOverrideFrameRate)
	{
		return FQualifiedFrameTime(Value.ConvertTo(OverrideFrameRate), OverrideFrameRate);
	}
	return Value;
}


FQualifiedFrameTime ULiveLinkTimecodeProvider::LerpBetweenFrames(double Seconds, int32 IndexA, int32 IndexB) const
{
	const double TimeA = SubjectFrameTimes[IndexA].WorldTime;
	const double TimeB = SubjectFrameTimes[IndexB].WorldTime;
	const double Alpha = (Seconds - TimeA) / (TimeB - TimeA);

	if (Evaluation == ELiveLinkTimecodeProviderEvaluationType::Lerp && !FMath::IsNearlyEqual(Alpha, 1.0) && !FMath::IsNearlyZero(Alpha))
	{
		const FFrameRate Rate = SubjectFrameTimes[IndexA].SceneTime.Rate;
		const double SceneTimeA = SubjectFrameTimes[IndexA].SceneTime.Time.AsDecimal();
		const double SceneTimeB = SubjectFrameTimes[IndexB].SceneTime.ConvertTo(Rate).AsDecimal();
		const double SceneTimeLerp = FMath::Lerp(SceneTimeA, SceneTimeB, Alpha);
		return ConvertTo(FQualifiedFrameTime(FFrameTime::FromDecimal(SceneTimeLerp), Rate));
	}

	const int32 ReturnedIndex = (Alpha < 0.5) ? IndexA : IndexB;
	return ConvertTo(SubjectFrameTimes[ReturnedIndex].SceneTime);
}


bool ULiveLinkTimecodeProvider::Initialize(class UEngine* InEngine)
{
	State = ETimecodeProviderSynchronizationState::Synchronizing;

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.OnModularFeatureRegistered().AddUObject(this, &ULiveLinkTimecodeProvider::OnLiveLinkClientRegistered);
	ModularFeatures.OnModularFeatureUnregistered().AddUObject(this, &ULiveLinkTimecodeProvider::OnLiveLinkClientUnregistered);

	InitClient();

	UnregisterSubject();
	RegisterSubject();
	
	return true;
}


void ULiveLinkTimecodeProvider::Shutdown(class UEngine* InEngine)
{
	UninitClient();
	State = ETimecodeProviderSynchronizationState::Closed;
}


void ULiveLinkTimecodeProvider::BeginDestroy()
{
	UninitClient();
	Super::BeginDestroy();
}


#if WITH_EDITOR
void ULiveLinkTimecodeProvider::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkTimecodeProvider, SubjectKey))
	{
		// If was already registered
		if (RegisterForFrameDataReceivedHandle.IsValid() && RegisteredSubjectKey.SubjectName != SubjectKey.SubjectName)
		{
			UnregisterSubject();
			RegisterSubject();
		}
		// If was waiting for the subject to be added to the client
		else if (LiveLinkClient && LiveLinkClient->IsSubjectValid(SubjectKey.SubjectName))
		{
			OnLiveLinkSubjectAdded(SubjectKey);
		}
	}
}
#endif

void ULiveLinkTimecodeProvider::InitClient()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		LiveLinkClient->OnLiveLinkSubjectAdded().AddUObject(this, &ULiveLinkTimecodeProvider::OnLiveLinkSubjectAdded);
		LiveLinkClient->OnLiveLinkSubjectRemoved().AddUObject(this, &ULiveLinkTimecodeProvider::OnLiveLinkSubjectRemoved);

		// if the subject already exist
		if (LiveLinkClient->IsSubjectValid(SubjectKey.SubjectName))
		{
			OnLiveLinkSubjectAdded(SubjectKey);
		}
	}
}


void ULiveLinkTimecodeProvider::UninitClient()
{
	if (LiveLinkClient)
	{
		UnregisterSubject();

		LiveLinkClient->OnLiveLinkSubjectAdded().RemoveAll(this);
		LiveLinkClient->OnLiveLinkSubjectRemoved().RemoveAll(this);
		LiveLinkClient = nullptr;
		State = ETimecodeProviderSynchronizationState::Closed;
	}
}


void ULiveLinkTimecodeProvider::RegisterSubject()
{
	if (!LiveLinkClient)
	{
		return;
	}

	{
		FScopeLock ScopeLock(&SubjectFrameLock);
		SubjectFrameTimes.Reset();
	}

	RegisteredSubjectKey = SubjectKey;

	// Check if this is a sequencer source and cache the result
	FText SourceType = LiveLinkClient->GetSourceType(SubjectKey.Source);
	FString SourceTypeStr = SourceType.ToString();
	bool bIsSequencer = SourceTypeStr.Contains(UE::LiveLink::SourceTypes::SequencerSourcePrefix, ESearchCase::IgnoreCase);
	bIsSequencerSourceAnyThread.store(bIsSequencer, std::memory_order_relaxed);

	FDelegateHandle Tmp;
	LiveLinkClient->RegisterForFrameDataReceived(
		RegisteredSubjectKey
		, FOnLiveLinkSubjectStaticDataReceived::FDelegate()
		, FOnLiveLinkSubjectFrameDataReceived::FDelegate::CreateUObject(this, &ULiveLinkTimecodeProvider::OnLiveLinkFrameDataReceived_AnyThread)
		, Tmp
		, RegisterForFrameDataReceivedHandle);
}


void ULiveLinkTimecodeProvider::UnregisterSubject()
{
	if (!LiveLinkClient)
	{
		return;
	}

	if (RegisterForFrameDataReceivedHandle.IsValid())
	{
		LiveLinkClient->UnregisterForFrameDataReceived(RegisteredSubjectKey, FDelegateHandle(), RegisterForFrameDataReceivedHandle);
		RegisterForFrameDataReceivedHandle.Reset();
	}

	// Also unregister from external subject if we registered to it
	if (ExternalSubjectFrameDataReceivedHandle.IsValid() && OriginalExternalSubjectKey.Source.IsValid())
	{
		LiveLinkClient->UnregisterForFrameDataReceived(OriginalExternalSubjectKey, FDelegateHandle(), ExternalSubjectFrameDataReceivedHandle);
		ExternalSubjectFrameDataReceivedHandle.Reset();
	}

	bIsSequencerSourceAnyThread.store(false, std::memory_order_relaxed);
	RegisteredSubjectKey = FLiveLinkSubjectKey();

	{
		FScopeLock ScopeLock(&SubjectFrameLock);
		SubjectFrameTimes.Reset();
	}
}


void ULiveLinkTimecodeProvider::OnLiveLinkClientRegistered(const FName& Type, class IModularFeature* ModularFeature)
{
	if (Type == ILiveLinkClient::ModularFeatureName && !LiveLinkClient)
	{
		InitClient();
	}
}


void ULiveLinkTimecodeProvider::OnLiveLinkClientUnregistered(const FName& Type, class IModularFeature* ModularFeature)
{
	if (Type == ILiveLinkClient::ModularFeatureName && ModularFeature == LiveLinkClient)
	{
		UninitClient();
		InitClient();
	}
}


void ULiveLinkTimecodeProvider::OnLiveLinkSubjectAdded(FLiveLinkSubjectKey InSubjectKey)
{
	if (!LiveLinkClient || InSubjectKey.SubjectName != SubjectKey.SubjectName)
	{
		return;
	}

	// Check if already registered to a subject with same name - unregister first
	if (RegisteredSubjectKey.SubjectName == InSubjectKey.SubjectName && RegisteredSubjectKey.Source.IsValid())
	{
		UnregisterSubject();
	}

	// Check if this is a sequencer source before registering
	FText SourceType = LiveLinkClient->GetSourceType(InSubjectKey.Source);
	FString SourceTypeStr = SourceType.ToString();
	bool bIsSequencer = SourceTypeStr.Contains(UE::LiveLink::SourceTypes::SequencerSourcePrefix, ESearchCase::IgnoreCase);

	if (bIsSequencer)
	{
		// Store the original external key if this is the first sequencer takeover
		if (!OriginalExternalSubjectKey.Source.IsValid())
		{
			OriginalExternalSubjectKey = SubjectKey;
		}

		// Register to the sequencer source without updating SubjectKey (keeps UI showing external source)
		{
			FScopeLock ScopeLock(&SubjectFrameLock);
			SubjectFrameTimes.Reset();
		}

		RegisteredSubjectKey = InSubjectKey;
		bIsSequencerSourceAnyThread.store(true, std::memory_order_relaxed);

		FDelegateHandle Tmp;
		LiveLinkClient->RegisterForFrameDataReceived(
			RegisteredSubjectKey,
			FOnLiveLinkSubjectStaticDataReceived::FDelegate(),
			FOnLiveLinkSubjectFrameDataReceived::FDelegate::CreateUObject(this, &ULiveLinkTimecodeProvider::OnLiveLinkFrameDataReceived_AnyThread),
			Tmp,
			RegisterForFrameDataReceivedHandle);

		// Also register to the disabled external subject to read its live timecode
		if (OriginalExternalSubjectKey.Source.IsValid() && LiveLinkClient->GetSubjectSettings(OriginalExternalSubjectKey))
		{
			FDelegateHandle TmpExternal;
			LiveLinkClient->RegisterForFrameDataReceived(
				OriginalExternalSubjectKey,
				FOnLiveLinkSubjectStaticDataReceived::FDelegate(),
				FOnLiveLinkSubjectFrameDataReceived::FDelegate::CreateUObject(this, &ULiveLinkTimecodeProvider::OnLiveLinkFrameDataReceived_AnyThread),
				TmpExternal,
				ExternalSubjectFrameDataReceivedHandle);

			UE_LOGF(LogLiveLink, Log, "LiveLinkTimecodeProvider: Registered to both sequencer and external subject for timecode");
		}
	}
	else
	{
		// External source - update SubjectKey and register normally
		OriginalExternalSubjectKey = InSubjectKey;
		SubjectKey = InSubjectKey;
		RegisterSubject();
	}
}


void ULiveLinkTimecodeProvider::OnLiveLinkSubjectRemoved(FLiveLinkSubjectKey InSubjectKey)
{
	if (InSubjectKey.SubjectName == RegisteredSubjectKey.SubjectName)
	{
		const bool bWasSequencerSource = bIsSequencerSourceAnyThread.load(std::memory_order_relaxed);
		UnregisterSubject();

		// If sequencer source was removed, try to re-register to the original external subject
		if (bWasSequencerSource && LiveLinkClient && OriginalExternalSubjectKey.Source.IsValid()
			&& LiveLinkClient->GetSubjectSettings(OriginalExternalSubjectKey))
		{
			SubjectKey = OriginalExternalSubjectKey;
			RegisterSubject();

			if (State == ETimecodeProviderSynchronizationState::Error)
			{
				State = ETimecodeProviderSynchronizationState::Synchronizing;
			}
		}
		else
		{
			State = ETimecodeProviderSynchronizationState::Error;
		}
	}
}


void ULiveLinkTimecodeProvider::OnLiveLinkFrameDataReceived_AnyThread(const FLiveLinkFrameDataStruct& InFrameData)
{
	const FLiveLinkTime NewLiveLinkTime = InFrameData.GetBaseData()->GetLiveLinkTime();

	FScopeLock ScopeLock(&SubjectFrameLock);

	if (LiveLinkClient)
	{
		// find where to insert the frame
		int32 InsertIndex = SubjectFrameTimes.Num() == 0 ? 0 : INDEX_NONE;
		for (int32 Index = SubjectFrameTimes.Num() - 1; Index >= 0; --Index)
		{
			if (SubjectFrameTimes[Index].WorldTime < NewLiveLinkTime.WorldTime)
			{
				InsertIndex = Index + 1;
				break;
			}
		}

		// if the new frame is newer than any of the other frames
		if (InsertIndex != INDEX_NONE)
		{
			SubjectFrameTimes.Insert(NewLiveLinkTime, InsertIndex);
			// is the buffer exceed the capacity
			if (SubjectFrameTimes.Num() > BufferSize)
			{
				State = ETimecodeProviderSynchronizationState::Synchronized;
				SubjectFrameTimes.RemoveAt(0);
			}
		}
	}
}

