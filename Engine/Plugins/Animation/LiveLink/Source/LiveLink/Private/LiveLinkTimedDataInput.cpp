// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkTimedDataInput.h"

#include "ILiveLinkModule.h"
#include "ITimeManagementModule.h"
#include "LiveLinkClient.h"
#include "LiveLinkSettings.h"
#include "LiveLinkSourceSettings.h"
#include "TimedDataInputCollection.h"

#if WITH_EDITOR
#include "Styling/SlateStyle.h"
#endif

#define LOCTEXT_NAMESPACE "LiveLinkTimedDataInput"


namespace LiveLinkTimedDataInput
{
	TAutoConsoleVariable<int32> CVarLiveLinkMinBufferSize(
		TEXT("LiveLink.TimedDataInput.MinBufferSize"),
		5,
		TEXT("The min size the timed data input is allowed to set the buffer size."),
		ECVF_Default
	);

	TAutoConsoleVariable<int32> CVarLiveLinkMaxBufferSize(
		TEXT("LiveLink.TimedDataInput.MaxBufferSize"),
		200,
		TEXT("The max size the timed data input is allowed to set the buffer size."),
		ECVF_Default
	);

	TAutoConsoleVariable<bool> CVarLiveLinkUpdateContinuousClockOffset(
		TEXT("LiveLink.TimedDataInput.UpdateClockOffset"),
		true,
		TEXT("By default, clock offset is continuously updated for each source. You can pause it if desired with this cvar and offset will be fixed to its value."),
		ECVF_Default
	);

	TAutoConsoleVariable<float> CVarLiveLinkNumFramesForSmoothOffset(
		TEXT("LiveLink.TimedDataInput.NumFramesForSmoothOffset"),
		1.5,
		TEXT("The fractional number of source frames used an offset to achieve smooth evaluation time."),
		ECVF_Default
	);

	ETimedDataInputEvaluationType ToTimedDataInputEvaluationType(ELiveLinkSourceMode SourceMode)
	{
		switch (SourceMode)
		{
		case ELiveLinkSourceMode::EngineTime:
			return ETimedDataInputEvaluationType::PlatformTime;
		case ELiveLinkSourceMode::Timecode:
			return ETimedDataInputEvaluationType::Timecode;
		case ELiveLinkSourceMode::Latest:
		default:
			return ETimedDataInputEvaluationType::None;
		}
	}

	ELiveLinkSourceMode ToLiveLinkSourceMode(ETimedDataInputEvaluationType EvaluationType)
	{
		switch (EvaluationType)
		{
		case ETimedDataInputEvaluationType::PlatformTime:
			return ELiveLinkSourceMode::EngineTime;
		case ETimedDataInputEvaluationType::Timecode:
			return ELiveLinkSourceMode::Timecode;
		case ETimedDataInputEvaluationType::None:
		default:
			return ELiveLinkSourceMode::Latest;
		}
	}
}


FLiveLinkTimedDataInput::FLiveLinkTimedDataInput(FLiveLinkClient* InClient, FGuid InSource)
	: LiveLinkClient(InClient)
	, Source(InSource)
{
	ITimeManagementModule::Get().GetTimedDataInputCollection().Add(this);

	EngineClockOffset.SetCorrectionStep(GetDefault<ULiveLinkSettings>()->ClockOffsetCorrectionStep);
	TimecodeClockOffset.SetCorrectionStep(GetDefault<ULiveLinkSettings>()->ClockOffsetCorrectionStep);

	FrameTimes.Reserve(FrameTimeBufferSize);
}

FLiveLinkTimedDataInput::~FLiveLinkTimedDataInput()
{
	ITimeManagementModule* TimeManagementModule = FModuleManager::GetModulePtr<ITimeManagementModule>("TimeManagement");
	if (TimeManagementModule)
	{
		TimeManagementModule->GetTimedDataInputCollection().Remove(this);
	}
}

FText FLiveLinkTimedDataInput::GetDisplayName() const
{
	return LiveLinkClient->GetSourceType(Source);
}

TArray<ITimedDataInputChannel*> FLiveLinkTimedDataInput::GetChannels() const
{
	return Channels;
}

ETimedDataInputEvaluationType FLiveLinkTimedDataInput::GetEvaluationType() const
{
	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		return LiveLinkTimedDataInput::ToTimedDataInputEvaluationType(Settings->Mode);
	}
	return ETimedDataInputEvaluationType::None;
}

void FLiveLinkTimedDataInput::SetEvaluationType(ETimedDataInputEvaluationType NewSourceMode)
{
	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		ELiveLinkSourceMode SourceMode = LiveLinkTimedDataInput::ToLiveLinkSourceMode(NewSourceMode);
		if (Settings->Mode != SourceMode)
		{
			Settings->Mode = SourceMode;
		}
	}
}

double FLiveLinkTimedDataInput::GetEvaluationOffsetInSeconds() const
{
	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		switch (Settings->Mode)
		{
		case ELiveLinkSourceMode::EngineTime:
			return Settings->BufferSettings.EngineTimeOffset;
		case ELiveLinkSourceMode::Timecode:
			return ITimedDataInput::ConvertFrameOffsetInSecondOffset(Settings->BufferSettings.TimecodeFrameOffset, Settings->BufferSettings.DetectedFrameRate);
		case ELiveLinkSourceMode::Latest:
		default:
			return Settings->BufferSettings.LatestOffset;
		}
	}
	return 0.f;
}

void FLiveLinkTimedDataInput::SetEvaluationOffsetInSeconds(double OffsetInSeconds)
{
	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		switch (Settings->Mode)
		{
		case ELiveLinkSourceMode::Latest:
			Settings->BufferSettings.LatestOffset = (float)OffsetInSeconds;
			break;
		case ELiveLinkSourceMode::EngineTime:
			Settings->BufferSettings.EngineTimeOffset = (float)OffsetInSeconds;
			break;
		case ELiveLinkSourceMode::Timecode:
			float OffsetInFrame = (float)ITimedDataInput::ConvertSecondOffsetInFrameOffset(OffsetInSeconds, Settings->BufferSettings.DetectedFrameRate);
			Settings->BufferSettings.TimecodeFrameOffset = OffsetInFrame;
			break;
		}
	}
}

FFrameRate FLiveLinkTimedDataInput::GetFrameRate() const
{
	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		if (Settings->Mode == ELiveLinkSourceMode::Timecode)
		{
			return Settings->BufferSettings.DetectedFrameRate;
		}
	}
	return ITimedDataInput::UnknownFrameRate;
}

int32 FLiveLinkTimedDataInput::GetDataBufferSize() const
{
	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		return Settings->BufferSettings.MaxNumberOfFrameToBuffered;
	}
	return 0;
}

void FLiveLinkTimedDataInput::SetDataBufferSize(int32 BufferSize)
{
	const int32 NewSize = FMath::Clamp(BufferSize, LiveLinkTimedDataInput::CVarLiveLinkMinBufferSize.GetValueOnAnyThread(), LiveLinkTimedDataInput::CVarLiveLinkMaxBufferSize.GetValueOnAnyThread());
	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		Settings->BufferSettings.MaxNumberOfFrameToBuffered = NewSize;
	}
}

UObject* FLiveLinkTimedDataInput::GetSettingsObject() const
{
	return LiveLinkClient->GetSourceSettings(Source);
}

#if WITH_EDITOR
const FSlateBrush* FLiveLinkTimedDataInput::GetDisplayIcon() const
{
	return ILiveLinkModule::Get().GetStyle()->GetBrush("LiveLinkIcon");
}
#endif

void FLiveLinkTimedDataInput::ProcessNewFrameTimingInfo(FLiveLinkBaseFrameData& NewFrameData)
{
	const bool bUpdateContinuousClockOffset = LiveLinkTimedDataInput::CVarLiveLinkUpdateContinuousClockOffset.GetValueOnAnyThread();

	//Update both clock offsets for each frame received for our subjects
	//We mark the last source/frame time that we used to update our offset to only update once per source frame
	if (!FMath::IsNearlyEqual(NewFrameData.WorldTime.GetSourceTime(), LastWorldSourceTime))
	{
		LastWorldSourceTime = NewFrameData.WorldTime.GetSourceTime();

		if (bUpdateContinuousClockOffset)
		{
			EngineClockOffset.UpdateEstimation(NewFrameData.WorldTime.GetSourceTime(), NewFrameData.ArrivalTime.WorldTime);
		}

		UpdateSmoothEngineTimeOffset(NewFrameData);
	}

	const double NewFrameSceneTime = NewFrameData.MetaData.SceneTime.AsSeconds();
	if (!FMath::IsNearlyEqual(NewFrameSceneTime, LastSceneTime))
	{
		LastSceneTime = NewFrameSceneTime;

		if (bUpdateContinuousClockOffset)
		{
			TimecodeClockOffset.UpdateEstimation(NewFrameSceneTime, NewFrameData.ArrivalTime.SceneTime.AsSeconds());
		}
	}

	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		Settings->BufferSettings.EngineTimeClockOffset = EngineClockOffset.GetEstimatedOffset();
		Settings->BufferSettings.TimecodeClockOffset = TimecodeClockOffset.GetEstimatedOffset();
	}

	//Update frame world time offset based on our latest clock offset
	NewFrameData.WorldTime.SetClockOffset(EngineClockOffset.GetEstimatedOffset());
}

void FLiveLinkTimedDataInput::UpdateSmoothEngineTimeOffset(const FLiveLinkBaseFrameData& NewFrameData)
{
	//Remove the oldest frame from the buffer if it is full
	if (FrameTimes.Num() >= FrameTimeBufferSize)
	{
		constexpr int32 Count = 1;
		FrameTimes.RemoveAt(0, Count, EAllowShrinking::No);
	}

	//Add the newest frame time to the buffer
	FrameTimes.Add(NewFrameData.WorldTime.GetSourceTime());

	if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Source))
	{
		//Early-out if no frame offset is needed
		const float NumFramesForSmoothOffset = LiveLinkTimedDataInput::CVarLiveLinkNumFramesForSmoothOffset.GetValueOnAnyThread();

		if (NumFramesForSmoothOffset <= 0)
		{
			Settings->BufferSettings.SmoothEngineTimeOffset = 0.0;
			return;
		}

		//In order to compute a frame interval, at least 2 frames are needed
		const int32 NumFramesInBuffer = FrameTimes.Num();
		if (NumFramesInBuffer > 1)
		{
			//Compute the time interval between the two most recent frames (N+1 - N)
			const double LatestFrameInterval = FrameTimes[NumFramesInBuffer - 1] - FrameTimes[NumFramesInBuffer - 2];

			// Early-out if a very large amount of time has passed since the last frame was received (indicating that the device was previously idle)
			if (LatestFrameInterval > VeryLargeFrameIntervalThreshold)
			{
				FrameTimes.RemoveAt(0, NumFramesInBuffer - 1, EAllowShrinking::No);
				Settings->BufferSettings.SmoothEngineTimeOffset = 0.0;
				NumFramesToConsiderForAverage = 1;
				return;
			}

			//Recover the average frame interval computed last frame
			const double PreviousAverageFrameInterval = Settings->BufferSettings.SmoothEngineTimeOffset / NumFramesForSmoothOffset;

			//Detect if the new frame interval is very different from the current moving average
			if (FMath::Abs(LatestFrameInterval - PreviousAverageFrameInterval) > FrameIntervalThreshold)
			{
				++FrameIntervalChangeCount;

				if (FrameIntervalChangeCount >= FrameIntervalSnapCount)
				{
					//Reduce the number of frames to consider to only those consecutive recent frames that differed so much from the previous average
					NumFramesToConsiderForAverage = FrameIntervalSnapCount - 1;
					FrameIntervalChangeCount = 0;
				}
			}
			else
			{
				FrameIntervalChangeCount = 0;
			}

			//Slowly increase the number of frames to consider until it reaches the size of the frame buffer
			if (NumFramesToConsiderForAverage < NumFramesInBuffer)
			{
				++NumFramesToConsiderForAverage;
			}
			else
			{
				NumFramesToConsiderForAverage = NumFramesInBuffer;
			}

			check(NumFramesToConsiderForAverage > 1);

			const int32 OldestFrameToConsider = NumFramesInBuffer - NumFramesToConsiderForAverage;
			const double AverageFrameInterval = (FrameTimes[NumFramesInBuffer - 1] - FrameTimes[OldestFrameToConsider]) / (NumFramesToConsiderForAverage - 1);

			Settings->BufferSettings.SmoothEngineTimeOffset = AverageFrameInterval * NumFramesForSmoothOffset;
		}
	}
}


#undef LOCTEXT_NAMESPACE
