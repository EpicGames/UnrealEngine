// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkVirtualSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkVirtualSource)


#define LOCTEXT_NAMESPACE "LiveLinkVirtualSubjectSource"



bool FLiveLinkVirtualSubjectSource::CanBeDisplayedInUI() const 
{
	return true; 
}

void FLiveLinkVirtualSubjectSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) 
{

}

void FLiveLinkVirtualSubjectSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	ULiveLinkVirtualSubjectSourceSettings* VirtualSettings = Cast<ULiveLinkVirtualSubjectSourceSettings>(Settings);
	check(VirtualSettings);
	
	SourceName = VirtualSettings->SourceName;
}

bool FLiveLinkVirtualSubjectSource::IsSourceStillValid() const
{
	return true; 
}

bool FLiveLinkVirtualSubjectSource::RequestSourceShutdown()
{
	return true; 
}

FText FLiveLinkVirtualSubjectSource::GetSourceType() const
{
	return FText::FromName(SourceName);
}

FText FLiveLinkVirtualSubjectSource::GetSourceMachineName() const
{
	return FText();
}

FText FLiveLinkVirtualSubjectSource::GetSourceStatus() const
{
	return LOCTEXT("VirtualSubjectSourceStatus", "Active");
}

#undef LOCTEXT_NAMESPACE


