// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"


#include "LiveLinkVirtualSource.generated.h"

class ILiveLinkClient;


/**
 * Completely empty "source" that virtual subjects can hang off
 */
class FLiveLinkVirtualSubjectSource : public ILiveLinkSource
{
public:
	FLiveLinkVirtualSubjectSource() = default;
	virtual ~FLiveLinkVirtualSubjectSource() = default;

	//~ Begin ILiveLinkSource interface
	virtual bool CanBeDisplayedInUI() const override;
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;
	virtual bool IsSourceStillValid() const override;
	virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override;
	virtual FText GetSourceMachineName() const override;
	virtual FText GetSourceStatus() const override;
	//~ End ILiveLinkSource interface

protected:
	FName SourceName;
};

/** VirtualSubjectSource Settings to be able to differentiate from live sources and keep a name associated to the source */
UCLASS()
class ULiveLinkVirtualSubjectSourceSettings : public ULiveLinkSourceSettings
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FName SourceName;
};
