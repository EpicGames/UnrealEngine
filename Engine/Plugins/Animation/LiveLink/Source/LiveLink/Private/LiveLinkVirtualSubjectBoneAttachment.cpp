// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkVirtualSubjectBoneAttachment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkVirtualSubjectBoneAttachment)

#define LOCTEXT_NAMESPACE "VirtualSubjectBoneAttachment"

bool FLiveLinkVirtualSubjectBoneAttachment::IsValid(const TArray<FLiveLinkSubjectName>& ActiveSubjects) const
{
	if (ParentSubject.IsNone() || ChildSubject.IsNone())
	{
		LastError = LOCTEXT("InvalidSubjectError", "One or more subject name is not specified.");
	}
	else if (ParentBone.IsNone() || ChildBone.IsNone())
	{
		LastError = LOCTEXT("InvalidBoneError", "One or more bone name is not specified.");
	}
	else if (!ActiveSubjects.Contains(ParentSubject) || !ActiveSubjects.Contains(ChildSubject))
	{
		LastError = LOCTEXT("DisabledSubject", "One or more subject is not enabled.");
	}
	else
	{
		LastError = FText::GetEmpty();
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
