// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreProcessor/LiveLinkAxisSwitchPreProcessor.h"

#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "Roles/LiveLinkLocatorRole.h"
#include "Roles/LiveLinkLocatorTypes.h"
#include "UObject/ReleaseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkAxisSwitchPreProcessor)


namespace LiveLinkAxisSwitchPreProcessor
{
	float LiveLinkAxisToVectorMember(ELiveLinkAxis InAxis, const FVector& Origin)
	{
		const uint8 Value = static_cast<uint8>(InAxis) % 3;
		return Origin[Value];
	}

	float AxisSign(ELiveLinkAxis InAxis)
	{
		return static_cast<uint8>(InAxis) < 3 ? 1.f : -1.f;
	}

	ELiveLinkAxis AxisAbs(ELiveLinkAxis InAxis)
	{
		const uint8 Value = static_cast<uint8>(InAxis) % 3;
		return static_cast<ELiveLinkAxis>(Value);
	}

	void SwitchTransform(FTransform& Transform, ELiveLinkAxis FrontAxis, ELiveLinkAxis RightAxis, ELiveLinkAxis UpAxis, bool bUseOffsetPosition, FVector OffsetPosition, bool bUseOffsetOrientation, FRotator OffsetOrientation)
	{
		FVector Location = Transform.GetLocation();
		FVector NewLocation;
		NewLocation[0] = LiveLinkAxisToVectorMember(FrontAxis, Location) * AxisSign(FrontAxis);
		NewLocation[1] = LiveLinkAxisToVectorMember(RightAxis, Location) * AxisSign(RightAxis);
		NewLocation[2] = LiveLinkAxisToVectorMember(UpAxis, Location) * AxisSign(UpAxis);

		FVector Scale = Transform.GetScale3D();
		FVector NewScale;
		NewScale[0] = LiveLinkAxisToVectorMember(FrontAxis, Scale);
		NewScale[1] = LiveLinkAxisToVectorMember(RightAxis, Scale);
		NewScale[2] = LiveLinkAxisToVectorMember(UpAxis, Scale);

		FQuat Orientation = Transform.GetRotation();
		FQuat NewOrientation;
		switch (FrontAxis)
		{
			case ELiveLinkAxis::X:		NewOrientation.X = Orientation.X;	break;
			case ELiveLinkAxis::XNeg:	NewOrientation.X = -Orientation.X;	break;
			case ELiveLinkAxis::Y:		NewOrientation.X = Orientation.Y;	break;
			case ELiveLinkAxis::YNeg:	NewOrientation.X = -Orientation.Y;	break;
			case ELiveLinkAxis::Z:		NewOrientation.X = Orientation.Z;	break;
			case ELiveLinkAxis::ZNeg:	NewOrientation.X = -Orientation.Z;	break;
		}
		switch (RightAxis)
		{
			case ELiveLinkAxis::X:		NewOrientation.Y = Orientation.X;	break;
			case ELiveLinkAxis::XNeg:	NewOrientation.Y = -Orientation.X;	break;
			case ELiveLinkAxis::Y:		NewOrientation.Y = Orientation.Y;	break;
			case ELiveLinkAxis::YNeg:	NewOrientation.Y = -Orientation.Y;	break;
			case ELiveLinkAxis::Z:		NewOrientation.Y = Orientation.Z;	break;
			case ELiveLinkAxis::ZNeg:	NewOrientation.Y = -Orientation.Z;	break;
		}
		switch (UpAxis)
		{
			case ELiveLinkAxis::X:		NewOrientation.Z = Orientation.X;	break;
			case ELiveLinkAxis::XNeg:	NewOrientation.Z = -Orientation.X;	break;
			case ELiveLinkAxis::Y:		NewOrientation.Z = Orientation.Y;	break;
			case ELiveLinkAxis::YNeg:	NewOrientation.Z = -Orientation.Y;	break;
			case ELiveLinkAxis::Z:		NewOrientation.Z = Orientation.Z;	break;
			case ELiveLinkAxis::ZNeg:	NewOrientation.Z = -Orientation.Z;	break;
		}

		// Calculate the Levi-Civita symbol for W scaling according to the following table
		// +1 xyz yzx zxy
		// -1 xzy yxz zyx
		// 0 if any axes are duplicated (x == y, y == z, or z == x)
		float LCSymbol = 0.0f;
		ELiveLinkAxis FrontAxisAbs = AxisAbs(FrontAxis);
		ELiveLinkAxis RightAxisAbs = AxisAbs(RightAxis);
		ELiveLinkAxis UpAxisAbs = AxisAbs(UpAxis);
		if (((FrontAxisAbs == ELiveLinkAxis::X) && (RightAxisAbs == ELiveLinkAxis::Y) && (UpAxisAbs == ELiveLinkAxis::Z)) ||
			((FrontAxisAbs == ELiveLinkAxis::Y) && (RightAxisAbs == ELiveLinkAxis::Z) && (UpAxisAbs == ELiveLinkAxis::X)) ||
			((FrontAxisAbs == ELiveLinkAxis::Z) && (RightAxisAbs == ELiveLinkAxis::X) && (UpAxisAbs == ELiveLinkAxis::Y)))
		{
			LCSymbol = 1.0f;
		}
		else if (((FrontAxisAbs == ELiveLinkAxis::X) && (RightAxisAbs == ELiveLinkAxis::Z) && (UpAxisAbs == ELiveLinkAxis::Y)) ||
			((FrontAxisAbs == ELiveLinkAxis::Y) && (RightAxisAbs == ELiveLinkAxis::X) && (UpAxisAbs == ELiveLinkAxis::Z)) ||
			((FrontAxisAbs == ELiveLinkAxis::Z) && (RightAxisAbs == ELiveLinkAxis::Y) && (UpAxisAbs == ELiveLinkAxis::X)))
		{
			LCSymbol = -1.0f;
		}

		// Make sure the handedness is corrected to match the remapping
		const float FlipSign = AxisSign(FrontAxis) * AxisSign(RightAxis) * AxisSign(UpAxis);

		NewOrientation.W = Orientation.W * LCSymbol * FlipSign;

		// Apply any offsets after the remapping
		if (bUseOffsetPosition)
		{
			NewLocation += OffsetPosition;
		}
		if (bUseOffsetOrientation)
		{
			NewOrientation *= OffsetOrientation.Quaternion();
		}

		Transform.SetComponents(NewOrientation.GetNormalized(), NewLocation, NewScale);
	}
}

/**
 * 
 */
namespace LiveLinkLocationSwitchPreprocessor
{
	float LiveLinkAxisToVectorMember(ELiveLinkAxis InAxis, const FVector& Origin)
	{
		const uint8 Value = static_cast<uint8>(InAxis) % 3;
		return Origin[Value];
	}

	float AxisSign(ELiveLinkAxis InAxis)
	{
		return static_cast<uint8>(InAxis) < 3 ? 1.f : -1.f;
	}

	ELiveLinkAxis AxisAbs(ELiveLinkAxis InAxis)
	{
		const uint8 Value = static_cast<uint8>(InAxis) % 3;
		return static_cast<ELiveLinkAxis>(Value);
	} 

	void SwitchTranslation(FVector& Translation, ELiveLinkAxis FrontAxis, ELiveLinkAxis RightAxis, ELiveLinkAxis UpAxis, bool bUseOffsetPosition, FVector OffsetPosition, bool bUseOffsetOrientation, FRotator OffsetOrientation)
	{
		FVector Location = Translation;
		FVector NewLocation;
		NewLocation[0] = LiveLinkAxisToVectorMember(FrontAxis, Location) * AxisSign(FrontAxis);
		NewLocation[1] = LiveLinkAxisToVectorMember(RightAxis, Location) * AxisSign(RightAxis);
		NewLocation[2] = LiveLinkAxisToVectorMember(UpAxis, Location) * AxisSign(UpAxis);

		// Apply any offsets after the remapping
		if (bUseOffsetPosition)
		{
			NewLocation += OffsetPosition;
		}
		if (bUseOffsetOrientation)
		{
			NewLocation = OffsetOrientation.RotateVector(NewLocation); //Note this is executed after the axes are flipped/swapped
		}

		Translation = NewLocation;
	}
}

/**
 * ULiveLinkTransformAxisSwitchPreProcessor::FLiveLinkTransformAxisSwitchPreProcessorWorker
 */
TSubclassOf<ULiveLinkRole> ULiveLinkTransformAxisSwitchPreProcessor::FLiveLinkTransformAxisSwitchPreProcessorWorker::GetRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

bool ULiveLinkTransformAxisSwitchPreProcessor::FLiveLinkTransformAxisSwitchPreProcessorWorker::PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const
{
	FLiveLinkTransformFrameData& TransformData = *InOutFrame.Cast<FLiveLinkTransformFrameData>();
	LiveLinkAxisSwitchPreProcessor::SwitchTransform(TransformData.Transform, FrontAxis, RightAxis, UpAxis, bUseOffsetPosition, OffsetPosition, bUseOffsetOrientation, OffsetOrientation);
	return true;
}

/**
 * ULiveLinkTransformAxisSwitchPreProcessor
 */
TSubclassOf<ULiveLinkRole> ULiveLinkTransformAxisSwitchPreProcessor::GetRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

ULiveLinkFramePreProcessor::FWorkerSharedPtr ULiveLinkTransformAxisSwitchPreProcessor::FetchWorker()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShared<FLiveLinkTransformAxisSwitchPreProcessorWorker, ESPMode::ThreadSafe>();
		Instance->FrontAxis = FrontAxis;
		Instance->RightAxis = RightAxis;
		Instance->UpAxis = UpAxis;
		Instance->bUseOffsetPosition = bUseOffsetPosition;
		Instance->bUseOffsetOrientation = bUseOffsetOrientation;
		Instance->OffsetPosition = OffsetPosition;
		Instance->OffsetOrientation = OffsetOrientation;
	}

	return Instance;
}

#if WITH_EDITOR
void ULiveLinkTransformAxisSwitchPreProcessor::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static const FName NAME_FrontAxis = GET_MEMBER_NAME_CHECKED(ThisClass, FrontAxis);
	static const FName NAME_RightAxis = GET_MEMBER_NAME_CHECKED(ThisClass, RightAxis);
	static const FName NAME_UpAxis = GET_MEMBER_NAME_CHECKED(ThisClass, UpAxis);
	static const FName NAME_OffsetPosition = GET_MEMBER_NAME_CHECKED(ThisClass, OffsetPosition);
	static const FName NAME_OffsetOrientation = GET_MEMBER_NAME_CHECKED(ThisClass, OffsetOrientation);

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FEditPropertyChain::TDoubleLinkedListNode* ActiveMemberNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode();
	const FProperty* MemberProperty = ActiveMemberNode ? ActiveMemberNode->GetValue() : nullptr;
	const FName StructName = MemberProperty ? MemberProperty->GetFName() : NAME_None;

	if (!PropertyChangedEvent.Property || !MemberProperty)
	{
		Instance.Reset();
	}
	else if ((PropertyName == NAME_FrontAxis) || (PropertyName == NAME_RightAxis) || (PropertyName == NAME_UpAxis))
	{
		Instance.Reset();
	}
	else if ((StructName == NAME_OffsetPosition) || (StructName == NAME_OffsetOrientation))
	{
		Instance.Reset();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR


/**
 * ULiveLinkAnimationAxisSwitchPreProcessor::FLiveLinkAnimationAxisSwitchPreProcessorWorker
 */
TSubclassOf<ULiveLinkRole> ULiveLinkAnimationAxisSwitchPreProcessor::FLiveLinkAnimationAxisSwitchPreProcessorWorker::GetRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

bool ULiveLinkAnimationAxisSwitchPreProcessor::FLiveLinkAnimationAxisSwitchPreProcessorWorker::PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const
{
	FLiveLinkAnimationFrameData& AnimationData = *InOutFrame.Cast<FLiveLinkAnimationFrameData>();
	
	for (FTransform& Transform : AnimationData.Transforms)
	{
		LiveLinkAxisSwitchPreProcessor::SwitchTransform(Transform, FrontAxis, RightAxis, UpAxis, bUseOffsetPosition, OffsetPosition, bUseOffsetOrientation, OffsetOrientation);
	}
	
	return true;
}

/**
 * ULiveLinkAnimationAxisSwitchPreProcessor
 */
TSubclassOf<ULiveLinkRole> ULiveLinkAnimationAxisSwitchPreProcessor::GetRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

ULiveLinkFramePreProcessor::FWorkerSharedPtr ULiveLinkAnimationAxisSwitchPreProcessor::FetchWorker()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShared<FLiveLinkAnimationAxisSwitchPreProcessorWorker, ESPMode::ThreadSafe>();
		Instance->FrontAxis = FrontAxis;
		Instance->RightAxis = RightAxis;
		Instance->UpAxis = UpAxis;
		Instance->bUseOffsetPosition = bUseOffsetPosition;
		Instance->bUseOffsetOrientation = bUseOffsetOrientation;
		Instance->OffsetPosition = OffsetPosition;
		Instance->OffsetOrientation = OffsetOrientation;
	}

	return Instance;
}

void ULiveLinkTransformAxisSwitchPreProcessor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::AddedFrontRightUpAxesToLiveLinkPreProcessor)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FrontAxis = OrientationAxisX_DEPRECATED;
			RightAxis = OrientationAxisY_DEPRECATED;
			UpAxis = OrientationAxisZ_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif
}

/**
 * ULiveLinkTransformAxisSwitchPreProcessor::FLiveLinkLocatorAxisSwitchPreProcessorWorker
 */

TSubclassOf<ULiveLinkRole> ULiveLinkLocatorAxisSwitchPreProcessor::FLiveLinkLocatorAxisSwitchPreProcessorWorker::GetRole() const
	{
		return ULiveLinkLocatorRole::StaticClass();
	}

bool ULiveLinkLocatorAxisSwitchPreProcessor::FLiveLinkLocatorAxisSwitchPreProcessorWorker::PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const
{
	FLiveLinkLocatorFrameData& LocatorData = *InOutFrame.Cast<FLiveLinkLocatorFrameData>();

	for (FVector& Location : LocatorData.Locators)
	{
		LiveLinkLocationSwitchPreprocessor::SwitchTranslation(Location, FrontAxis, RightAxis, UpAxis, bUseOffsetPosition, OffsetPosition, bUseOffsetOrientation, OffsetOrientation);
	}
	return true;
}

ULiveLinkFramePreProcessor::FWorkerSharedPtr ULiveLinkLocatorAxisSwitchPreProcessor::FetchWorker()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShared<FLiveLinkLocatorAxisSwitchPreProcessorWorker, ESPMode::ThreadSafe>();
		Instance->FrontAxis = FrontAxis;
		Instance->RightAxis = RightAxis;
		Instance->UpAxis = UpAxis;
		Instance->bUseOffsetPosition = bUseOffsetPosition;
		Instance->bUseOffsetOrientation = bUseOffsetOrientation;
		Instance->OffsetPosition = OffsetPosition;
		Instance->OffsetOrientation = OffsetOrientation;
	}
	return Instance;
}

TSubclassOf<ULiveLinkRole> ULiveLinkLocatorAxisSwitchPreProcessor::GetRole() const
{
	return ULiveLinkLocatorRole::StaticClass();
}
