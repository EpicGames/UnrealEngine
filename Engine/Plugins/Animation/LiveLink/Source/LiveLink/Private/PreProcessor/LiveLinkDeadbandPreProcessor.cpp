// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreProcessor/LiveLinkDeadbandPreProcessor.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "UObject/ReleaseObjectVersion.h"

// WORKER

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkDeadbandPreProcessor)

TSubclassOf<ULiveLinkRole> ULiveLinkTransformDeadbandPreProcessor::FLiveLinkTransformDeadbandPreProcessorWorker::GetRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

bool ULiveLinkTransformDeadbandPreProcessor::FLiveLinkTransformDeadbandPreProcessorWorker::PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const
{
	if (!bEnableDeadband)
	{
		return true;
	}

	FLiveLinkTransformFrameData& TransformData = *InOutFrame.Cast<FLiveLinkTransformFrameData>();

	// location
	{
		const double DeltaTranslation = (StableTransform.GetLocation() - TransformData.Transform.GetLocation()).Length();

		if (DeltaTranslation < TranslationDeadband)
		{
			// Keep stable location
			TransformData.Transform.SetLocation(StableTransform.GetLocation());
		}
	}

	// rotation
	{
		const double DeltaRotationInDegrees = FMath::RadiansToDegrees(StableTransform.GetRotation().AngularDistance(TransformData.Transform.GetRotation()));

		if (DeltaRotationInDegrees < RotationDeadbandInDegrees)
		{
			// Keep stable rotation
			TransformData.Transform.SetRotation(StableTransform.GetRotation());
		}
	}

	StableTransform = TransformData.Transform;

	return true;
}

// PREPROCESSOR

TSubclassOf<ULiveLinkRole> ULiveLinkTransformDeadbandPreProcessor::GetRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

ULiveLinkTransformDeadbandPreProcessor::FWorkerSharedPtr ULiveLinkTransformDeadbandPreProcessor::FetchWorker()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShared<FLiveLinkTransformDeadbandPreProcessorWorker, ESPMode::ThreadSafe>();
		Instance->TranslationDeadband = TranslationDeadband;
		Instance->RotationDeadbandInDegrees = RotationDeadbandInDegrees;
		Instance->bEnableDeadband = bEnableDeadband;
	}

	return Instance;
}

#if WITH_EDITOR
void ULiveLinkTransformDeadbandPreProcessor::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static const FName NAME_TranslationDeadband = GET_MEMBER_NAME_CHECKED(ThisClass, TranslationDeadband);
	static const FName NAME_RotationDegreesDeadband = GET_MEMBER_NAME_CHECKED(ThisClass, RotationDeadbandInDegrees);
	static const FName NAME_bEnableDeadband = GET_MEMBER_NAME_CHECKED(ThisClass, bEnableDeadband);

	const FName PropertyName = PropertyChangedEvent.Property->GetFName();

	if ((PropertyName == NAME_TranslationDeadband) 
		|| (PropertyName == NAME_RotationDegreesDeadband)
		|| (PropertyName == NAME_bEnableDeadband)
	)
	{
		Instance.Reset();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

