// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkMessageBusSourceFactory.h"

#include "Framework/Views/TableViewMetadata.h"
#include "ILiveLinkModule.h"
#include "Widgets/Views/SListView.h"
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
#include "LiveLinkMessageBusDiscoveryManager.h"
#endif

#define LOCTEXT_NAMESPACE "LiveLinkMessageBusSourceEditor"

namespace ProviderPollUI
{
	const FName TypeColumnName(TEXT("Type"));
	const FName MachineColumnName(TEXT("Machine"));
};

namespace
{
	bool operator==(const FProviderPollResult& LHS, const FProviderPollResult& RHS)
	{
		return LHS.Name == RHS.Name && LHS.MachineName == RHS.MachineName;
	}
}

class SProviderPollRow : public SMultiColumnTableRow<FProviderPollResultPtr>
{
public:
	SLATE_BEGIN_ARGS(SProviderPollRow) {}

		/** The list item for this row */
		SLATE_ARGUMENT(FProviderPollResultPtr, PollResultPtr)

	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		PollResultPtr = Args._PollResultPtr;

		SMultiColumnTableRow<FProviderPollResultPtr>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			OwnerTableView
		);

		if (!PollResultPtr->bIsValidProvider)
		{
			SetEnabled(false);
			SetToolTipText(LOCTEXT("InvalidProvider", 
				"This provider is invalid because it was built with an older version of Unreal Engine. You must upgrade your provider to UE5 to continue using it."));
		}
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ProviderPollUI::TypeColumnName)
		{
			return	SNew(STextBlock)
				.Text(FText::FromString(PollResultPtr->Name));
		}
		else if (ColumnName == ProviderPollUI::MachineColumnName)
		{
			return	SNew(STextBlock)
				.Text(FText::FromString(PollResultPtr->MachineName));
		}

		return SNullWidget::NullWidget;
	}

private:
	FProviderPollResultPtr PollResultPtr;
};


SLiveLinkMessageBusSourceFactory::~SLiveLinkMessageBusSourceFactory()
{
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	if (ILiveLinkModule* ModulePtr = FModuleManager::GetModulePtr<ILiveLinkModule>("LiveLink"))
	{
		ModulePtr->GetMessageBusDiscoveryManager().RemoveDiscoveryMessageRequest();
	}
#endif
}

void SLiveLinkMessageBusSourceFactory::Construct(const FArguments& Args)
{
	OnSourceSelected = Args._OnSourceSelected;
	FactoryClass = Args._FactoryClass;

#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	ILiveLinkModule::Get().GetMessageBusDiscoveryManager().AddDiscoveryMessageRequest();
#endif

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SBox)
			.HeightOverride(200)
			.WidthOverride(200)
			[
				SAssignNew(ListView, SListView<TSharedPtr<FLiveLinkSource>>)
				.ListItemsSource(&Sources)
				.SelectionMode(ESelectionMode::SingleToggle)
				.OnGenerateRow(this, &SLiveLinkMessageBusSourceFactory::MakeSourceListViewWidget)
				.OnSelectionChanged(this, &SLiveLinkMessageBusSourceFactory::OnSourceListSelectionChanged)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(ProviderPollUI::TypeColumnName)
					.FillWidth(43.0f)
					.DefaultLabel(LOCTEXT("TypeColumnHeaderName", "Source Type"))
					+ SHeaderRow::Column(ProviderPollUI::MachineColumnName)
					.FillWidth(43.0f)
					.DefaultLabel(LOCTEXT("MachineColumnHeaderName", "Source Machine"))
				)
			]
		]
	];
}

void SLiveLinkMessageBusSourceFactory::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (FApp::GetCurrentTime() - LastUIUpdateSeconds > 0.5)
	{
		PollData.Reset();
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
		PollData.Append(ILiveLinkModule::Get().GetMessageBusDiscoveryManager().GetDiscoveryResults());
#endif
		
		Sources.RemoveAllSwap([this](TSharedPtr<FLiveLinkSource> Source)
		{
			return FApp::GetCurrentTime() - Source->LastTimeSincePong > SecondsBeforeSourcesDisappear;
		});


		const TMap<FDelegateHandle, FOnLiveLinkShouldDisplaySource> RegisteredFilters = ILiveLinkModule::Get().GetSourceFilters();
		
		for (const FProviderPollResultPtr& PollResult : PollData)
		{
			TSharedPtr<FLiveLinkSource>* FoundSource = Sources.FindByPredicate([&PollResult](const TSharedPtr<FLiveLinkSource> Source){ return Source && PollResult && *Source->PollResult == *PollResult;});
			if (FoundSource && *FoundSource)
			{
				(*FoundSource)->LastTimeSincePong = FApp::GetCurrentTime();
			}
			else
			{
				bool bAllowResult = true;

				for (const TPair<FDelegateHandle, FOnLiveLinkShouldDisplaySource>& Filter : RegisteredFilters)
				{
					if (!Filter.Value.Execute(FactoryClass, PollResult))
					{
						bAllowResult = false;
						break;
					}
				}

				if (bAllowResult)
				{
					Sources.Add(MakeShared<FLiveLinkSource>(PollResult));
				}
			}
		}

		Sources.StableSort([](const TSharedPtr<FLiveLinkSource>& LHS, const TSharedPtr<FLiveLinkSource>& RHS)
		{
			if (LHS.IsValid() && RHS.IsValid() && LHS->PollResult.IsValid() && RHS->PollResult.IsValid())
			{
				//List by SourceName first and then by machine name
				const int32 Result = LHS->PollResult->Name.Compare(RHS->PollResult->Name);
				if (Result == 0)
				{
					return LHS->PollResult->MachineName.Compare(RHS->PollResult->MachineName) <= 0;
				}
				else
				{
					return Result <= 0;
				}
			}
			return true;
		});

		ListView->RequestListRefresh();
		LastUIUpdateSeconds = FApp::GetCurrentTime();
	}
}

TSharedRef<ITableRow> SLiveLinkMessageBusSourceFactory::MakeSourceListViewWidget(TSharedPtr<FLiveLinkSource> Source, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SProviderPollRow, OwnerTable).PollResultPtr(Source->PollResult);
}

void SLiveLinkMessageBusSourceFactory::OnSourceListSelectionChanged(TSharedPtr<FLiveLinkSource> Source, ESelectInfo::Type SelectionType)
{
	if (Source)
	{
		SelectedResult = Source->PollResult;
		OnSourceSelected.ExecuteIfBound(SelectedResult);
	}
}

#undef LOCTEXT_NAMESPACE
