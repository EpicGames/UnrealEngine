// Copyright Epic Games, Inc. All Rights Reserved.

#include "Translator/LiveLinkAnimationRoleToTransform.h"

#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkAnimationRoleToTransform)


/**
 * ULiveLinkAnimationRoleToTransform::FLiveLinkAnimationRoleToTransformWorker
 */
TSubclassOf<ULiveLinkRole> ULiveLinkAnimationRoleToTransform::FLiveLinkAnimationRoleToTransformWorker::GetFromRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

TSubclassOf<ULiveLinkRole> ULiveLinkAnimationRoleToTransform::FLiveLinkAnimationRoleToTransformWorker::GetToRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

bool ULiveLinkAnimationRoleToTransform::FLiveLinkAnimationRoleToTransformWorker::Translate(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, FLiveLinkSubjectFrameData& OutTranslatedFrame) const
{
	if (!InStaticData.IsValid() || !InFrameData.IsValid())
	{
		return false;
	}

	const FLiveLinkSkeletonStaticData* SkeletonData = InStaticData.Cast<FLiveLinkSkeletonStaticData>();
	const FLiveLinkAnimationFrameData* FrameData = InFrameData.Cast<FLiveLinkAnimationFrameData>();
	if (SkeletonData == nullptr || FrameData == nullptr)
	{
		return false;
	}

	//Allocate memory for the output translated frame with the desired type
	OutTranslatedFrame.StaticData.InitializeWith(FLiveLinkTransformStaticData::StaticStruct(), nullptr);
	OutTranslatedFrame.FrameData.InitializeWith(FLiveLinkTransformFrameData::StaticStruct(), nullptr);

	FLiveLinkTransformStaticData* TransformStaticData = OutTranslatedFrame.StaticData.Cast<FLiveLinkTransformStaticData>();
	FLiveLinkTransformFrameData* TransformFrameData = OutTranslatedFrame.FrameData.Cast<FLiveLinkTransformFrameData>();
	check(TransformStaticData && TransformFrameData);

	const int32 BoneIndex = SkeletonData->BoneNames.IndexOfByKey(BoneName);
	if (!FrameData->Transforms.IsValidIndex(BoneIndex))
	{
		return false;
	}

	//Time to translate
	TransformStaticData->PropertyNames.Append(SkeletonData->PropertyNames);

	TransformFrameData->MetaData = FrameData->MetaData;
	TransformFrameData->PropertyValues = FrameData->PropertyValues;
	TransformFrameData->WorldTime = FrameData->WorldTime;
	TransformFrameData->Transform = FrameData->Transforms[BoneIndex];
	return true;
}


/**
 * ULiveLinkAnimationRoleToTransform
 */
TSubclassOf<ULiveLinkRole> ULiveLinkAnimationRoleToTransform::GetFromRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

TSubclassOf<ULiveLinkRole> ULiveLinkAnimationRoleToTransform::GetToRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

ULiveLinkFrameTranslator::FWorkerSharedPtr ULiveLinkAnimationRoleToTransform::FetchWorker()
{
	if (BoneName.IsNone())
	{
		Instance.Reset();
	}
	else if (!Instance.IsValid())
	{
		Instance = MakeShared<FLiveLinkAnimationRoleToTransformWorker, ESPMode::ThreadSafe>();
		Instance->BoneName = BoneName;
	}

	return Instance;
}

#if WITH_EDITOR
void ULiveLinkAnimationRoleToTransform::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ThisClass, BoneName))
	{
		Instance.Reset();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

