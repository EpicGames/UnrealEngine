// Copyright Epic Games, Inc. All Rights Reserved.

#include "Translator/LiveLinkTransformRoleToAnimation.h"

#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkTransformRoleToAnimation)


/**
 * ULiveLinkTransformRoleToAnimation::FLiveLinkTransformRoleToAnimationWorker
 */
TSubclassOf<ULiveLinkRole> ULiveLinkTransformRoleToAnimation::FLiveLinkTransformRoleToAnimationWorker::GetFromRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

TSubclassOf<ULiveLinkRole> ULiveLinkTransformRoleToAnimation::FLiveLinkTransformRoleToAnimationWorker::GetToRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

bool ULiveLinkTransformRoleToAnimation::FLiveLinkTransformRoleToAnimationWorker::Translate(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, FLiveLinkSubjectFrameData& OutTranslatedFrame) const
{
	if (!InStaticData.IsValid() || !InFrameData.IsValid())
	{
		return false;
	}


	const FLiveLinkTransformStaticData* TransformStaticData = InStaticData.Cast<FLiveLinkTransformStaticData>();
	const FLiveLinkTransformFrameData* FrameData = InFrameData.Cast<FLiveLinkTransformFrameData>();

	if (TransformStaticData == nullptr || FrameData == nullptr)
	{
		return false;
	}

	//Allocate memory for the output translated frame with the desired type
	OutTranslatedFrame.StaticData.InitializeWith(FLiveLinkSkeletonStaticData::StaticStruct(), nullptr);
	OutTranslatedFrame.FrameData.InitializeWith(FLiveLinkAnimationFrameData::StaticStruct(), nullptr);

	FLiveLinkSkeletonStaticData* AnimationStaticData = OutTranslatedFrame.StaticData.Cast<FLiveLinkSkeletonStaticData>();
	FLiveLinkAnimationFrameData* AnimationFrameData = OutTranslatedFrame.FrameData.Cast<FLiveLinkAnimationFrameData>();
	check(AnimationStaticData && AnimationFrameData);

	AnimationStaticData->BoneNames.Add(OutputBoneName);
	AnimationStaticData->BoneParents.Add(-1);

	AnimationFrameData->MetaData = FrameData->MetaData;
	AnimationFrameData->WorldTime = FrameData->WorldTime;
	AnimationFrameData->Transforms.Reset(1);
	AnimationFrameData->Transforms.Add(FrameData->Transform);

	return true;
}

/**
 * ULiveLinkTransformRoleToAnimation
 */
TSubclassOf<ULiveLinkRole> ULiveLinkTransformRoleToAnimation::GetFromRole() const
{
	return ULiveLinkTransformRole::StaticClass();
}

TSubclassOf<ULiveLinkRole> ULiveLinkTransformRoleToAnimation::GetToRole() const
{
	return ULiveLinkAnimationRole::StaticClass();
}

ULiveLinkFrameTranslator::FWorkerSharedPtr ULiveLinkTransformRoleToAnimation::FetchWorker()
{
	if (OutputBoneName.IsNone())
	{
		Instance.Reset();
	}
	else if (!Instance.IsValid())
	{
		Instance = MakeShared<FLiveLinkTransformRoleToAnimationWorker>();
		Instance->OutputBoneName = OutputBoneName;
	}

	return Instance;
}

#if WITH_EDITOR
void ULiveLinkTransformRoleToAnimation::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ThisClass, OutputBoneName))
	{
		Instance.Reset();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

