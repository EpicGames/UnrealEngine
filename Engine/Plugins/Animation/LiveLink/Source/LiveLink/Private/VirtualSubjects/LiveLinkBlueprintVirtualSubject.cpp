// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualSubjects/LiveLinkBlueprintVirtualSubject.h"

#include "LiveLinkRole.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkBlueprintVirtualSubject)

void ULiveLinkBlueprintVirtualSubject::Initialize(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, ILiveLinkClient* InLiveLinkClient)
{
	Super::Initialize(InSubjectKey, Role, LiveLinkClient);

	CachedStaticData.InitializeWith(GetRoleStaticStruct(), nullptr);

	FLiveLinkStaticDataStruct NewStaticData;
	NewStaticData.InitializeWith(GetRoleStaticStruct(), CachedStaticData.GetBaseData());
	UpdateStaticDataSnapshot(MoveTemp(NewStaticData));

	FLiveLinkFrameDataStruct NewFrameData;
	NewFrameData.InitializeWith(GetRoleFrameStruct(), nullptr);
	UpdateFrameDataSnapshot(MoveTemp(NewFrameData));

	{
		FEditorScriptExecutionGuard ScriptGuard;

		OnInitialize();
	}
}

void ULiveLinkBlueprintVirtualSubject::Update()
{
	Super::Update();

	{
		FEditorScriptExecutionGuard ScriptGuard;

		OnUpdate();
	}

	if (HasValidFrameData() && !HasValidStaticData() && CachedStaticData.IsValid())
	{
		FLiveLinkStaticDataStruct NewStaticData;
		NewStaticData.InitializeWith(GetRoleStaticStruct(), CachedStaticData.GetBaseData());
		UpdateStaticDataSnapshot(MoveTemp(NewStaticData));
	}
}

void ULiveLinkBlueprintVirtualSubject::UpdateVirtualSubjectStaticData(const FLiveLinkBaseStaticData* InStaticData)
{
	CachedStaticData.InitializeWith(GetRoleStaticStruct(), InStaticData);

	FLiveLinkStaticDataStruct NewStaticData;
	NewStaticData.InitializeWith(GetRoleStaticStruct(), InStaticData);
	UpdateStaticDataSnapshot(MoveTemp(NewStaticData));
	
	// Invalidate any existing Frame Data
	InvalidateFrameData();
}

void ULiveLinkBlueprintVirtualSubject::UpdateVirtualSubjectFrameData(const FLiveLinkBaseFrameData* InFrameData, bool bInShouldStampCurrentTime)
{
	FLiveLinkFrameDataStruct NewFrameData;
	NewFrameData.InitializeWith(GetRoleFrameStruct(), InFrameData);

	// Stamp the current time into the frame if desired
	if (bInShouldStampCurrentTime)
	{
		if (FLiveLinkBaseFrameData* BaseFrameData = NewFrameData.GetBaseData())
		{
			// Stamp the current world time
			BaseFrameData->WorldTime = FLiveLinkWorldTime(FApp::GetCurrentTime());

			// If we have a valid Frame Time then stamp it to the meta data
			const TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();
			if (CurrentFrameTime.IsSet())
			{
				BaseFrameData->MetaData.SceneTime = CurrentFrameTime.GetValue();
			}
		}
	}

	UpdateFrameDataSnapshot(MoveTemp(NewFrameData));
}

bool ULiveLinkBlueprintVirtualSubject::UpdateVirtualSubjectStaticData_Internal(const FLiveLinkBaseStaticData& InStruct)
{
	// We should never hit this!  stubs to avoid NoExport on the class.
	check(0);
	return false;
}

bool ULiveLinkBlueprintVirtualSubject::UpdateVirtualSubjectFrameData_Internal(const FLiveLinkBaseFrameData& InStruct, bool bInShouldStampCurrentTime)
{
	// We should never hit this!  stubs to avoid NoExport on the class.
	check(0);
	return false;
}

UScriptStruct* ULiveLinkBlueprintVirtualSubject::GetRoleStaticStruct()
{
	check(Role);
	return Role->GetDefaultObject<ULiveLinkRole>()->GetStaticDataStruct();
}

UScriptStruct* ULiveLinkBlueprintVirtualSubject::GetRoleFrameStruct()
{
	check(Role);
	return Role->GetDefaultObject<ULiveLinkRole>()->GetFrameDataStruct();
}

DEFINE_FUNCTION(ULiveLinkBlueprintVirtualSubject::execUpdateVirtualSubjectStaticData_Internal)
{
	Stack.StepCompiledIn<FStructProperty>(NULL);
	void* PropertyAddress = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	bool bSuccess = false;

	if (StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FLiveLinkBaseStaticData::StaticStruct()))
	{
		FLiveLinkBaseStaticData* StaticData = reinterpret_cast<FLiveLinkBaseStaticData*>(PropertyAddress);

		P_THIS->UpdateVirtualSubjectStaticData(StaticData);

		bSuccess = true;
	}

	P_NATIVE_END;

	*(bool*)RESULT_PARAM = true;
}

DEFINE_FUNCTION(ULiveLinkBlueprintVirtualSubject::execUpdateVirtualSubjectFrameData_Internal)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* PropertyAddress = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_GET_PROPERTY(FBoolProperty, bStampTime);

	P_FINISH;

	P_NATIVE_BEGIN;

	bool bSuccess = false;

	if (StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FLiveLinkBaseFrameData::StaticStruct()))
	{
		FLiveLinkBaseFrameData* FrameData = reinterpret_cast<FLiveLinkBaseFrameData*>(PropertyAddress);

		P_THIS->UpdateVirtualSubjectFrameData(FrameData, bStampTime);

		bSuccess = true;
	}

	P_NATIVE_END;

	*(bool*)RESULT_PARAM = true;
}
