// Copyright Epic Games, Inc. All Rights Reserved.


#include "Visualizers/LiveLinkDataPreview.h"

#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkCameraRole.h"
#include "Roles/LiveLinkLocatorRole.h"
#include "Roles/LiveLinkTransformRole.h"
#include "ILiveLinkClient.h"
#include "Features/IModularFeatures.h"
#include "Components/BillboardComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "Visualizers/LiveLinkDataPreviewComponent.h"


// Sets default values

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkDataPreview)
ALiveLinkDataPreview::ALiveLinkDataPreview()
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;
	bEvaluateLiveLink = true;

	BoneMesh = FSoftObjectPath("/LiveLink/Visualizers/SM_Bone.SM_Bone");
	JointMesh = FSoftObjectPath("/LiveLink/Visualizers/SM_Joint.SM_Joint");
	AxisMesh = FSoftObjectPath("/LiveLink/Visualizers/SM_JointAxis.SM_JointAxis");
	TransformMesh = FSoftObjectPath("/LiveLink/Visualizers/SM_TransformAxis.SM_TransformAxis");
	LocatorMesh = FSoftObjectPath("/LiveLink/Visualizers/SM_MarkerSphere.SM_MarkerSphere");
	CameraMesh = FSoftObjectPath("/LiveLink/Visualizers/SM_LiveLinkCamera.SM_LiveLinkCamera");
	SpriteTexture = FSoftObjectPath("/LiveLink/Starship/Location_256.Location_256");

	USceneComponent* SceneRoot = CreateDefaultSubobject<USceneComponent>("Root");
	SetRootComponent(SceneRoot);
	BillboardComponent = CreateDefaultSubobject<UBillboardComponent>("Sprite");
	BillboardComponent->SetupAttachment(SceneRoot);

	if (!IsTemplate())
	{
		UTexture2D* SpriteTex = SpriteTexture.LoadSynchronous();
		if (SpriteTex)
		{
			BillboardComponent->SetSprite(SpriteTex);
			BillboardComponent->bIsScreenSizeScaled = true;
			BillboardComponent->ScreenSize = 0.0006f;
			BillboardComponent->SetRelativeLocation(FVector(0, 0, 10));
		}
	}
}

// Called every frame
void ALiveLinkDataPreview::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	
		TArray<FLiveLinkSubjectKey> SubjectKeys = LiveLinkClient.GetSubjects(false, false);

		//If the number of Live Link subjects changes, force reinitialization. 
		if(SubjectKeys.Num() != CachedSubjects.Num())
		{
			InitializeSubjects();
		}
		CachedSubjects = SubjectKeys;
	}
}

void ALiveLinkDataPreview::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	InitializeSubjects();
}

#if WITH_EDITOR
void ALiveLinkDataPreview::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property != nullptr)
	{
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ALiveLinkDataPreview, bEvaluateLiveLink))
		{
			TArray<ULiveLinkDataPreviewComponent*> DebugComponents;
			GetComponents(DebugComponents);

			for (ULiveLinkDataPreviewComponent* DebugComponent : DebugComponents)
			{
				DebugComponent->SetEvaluateLiveLinkData(bEvaluateLiveLink);
			}
		}
	}
}
#endif

void ALiveLinkDataPreview::SetEnableLiveLinkData(bool bNewEvaluate)
{
	bEvaluateLiveLink = bNewEvaluate;

	//Loop through each subject's preview component and set it's bEvaluateLiveLink member.

	TArray<ULiveLinkDataPreviewComponent*> DebugComponents;
	GetComponents(DebugComponents);

	for (ULiveLinkDataPreviewComponent* DebugComponent : DebugComponents)
	{
		DebugComponent->SetEvaluateLiveLinkData(bEvaluateLiveLink);
	}
}

//Initialize each subject's preview component and depending on the role type, add a component with the mesh of the correct type.
void ALiveLinkDataPreview::InitializeSubjects()
{
	if((GetWorld() && !GetWorld()->IsPreviewWorld())) //Do not spawn the preview components unless we are a non-preview world
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

			TArray<FLiveLinkSubjectKey> SubjectKeys = LiveLinkClient.GetSubjects(false, false);

			Subjects.Empty();

			for (FLiveLinkSubjectKey Subjectkey : SubjectKeys)
			{
				FLiveLinkSubjectName Subject = Subjectkey.SubjectName;
				Subjects.Emplace(Subject);
			}

			TArray<ULiveLinkDataPreviewComponent*> DataPreviewComponents;
			GetComponents(DataPreviewComponents);

			for (ULiveLinkDataPreviewComponent* DataPreviewComponent : DataPreviewComponents)
			{
				DataPreviewComponent->DestroyComponent();
			}
			
			for (FLiveLinkSubjectName Subject : Subjects)
			{
				if(LiveLinkClient.IsSubjectEnabled(Subject))
				{
					ULiveLinkDataPreviewComponent* DataPreviewComponent;
					DataPreviewComponent = Cast<ULiveLinkDataPreviewComponent>(AddComponentByClass(ULiveLinkDataPreviewComponent::StaticClass(), false, FTransform::Identity, false));
					
					DataPreviewComponent->SubjectName = Subject;
					DataPreviewComponent->bEvaluateLiveLink = bEvaluateLiveLink;
					DataPreviewComponent->bDrawLabels = bDrawLabels;

					if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(Subject, ULiveLinkAnimationRole::StaticClass()))
					{
						if(UStaticMesh* StaticMesh = BoneMesh.LoadSynchronous())
						{
							DataPreviewComponent->SetStaticMesh(StaticMesh);
							SetMaterialInstance(DataPreviewComponent);
						}
						DataPreviewComponent->BoneVisualType = ELiveLinkVisualBoneType::Bone;
					}
					else if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(Subject, ULiveLinkLocatorRole::StaticClass()))
					{
						if(UStaticMesh* StaticMesh = LocatorMesh.LoadSynchronous())
						{
							DataPreviewComponent->SetStaticMesh(StaticMesh);
							SetMaterialInstance(DataPreviewComponent);
						}
					}
					else if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(Subject, ULiveLinkCameraRole::StaticClass()))
					{
						
						if(UStaticMesh* StaticMesh = CameraMesh.LoadSynchronous())
						{
							DataPreviewComponent->SetStaticMesh(StaticMesh);
							SetMaterialInstance(DataPreviewComponent);
						}
					}
					else if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(Subject, ULiveLinkTransformRole::StaticClass()))
					{
						;
						if(UStaticMesh* StaticMesh = TransformMesh.LoadSynchronous())
						{
							DataPreviewComponent->SetStaticMesh(StaticMesh);
						}
					}
				}
			}
		}
	}
}

//Set the material instance and set a nice random color. 
void ALiveLinkDataPreview::SetMaterialInstance(ULiveLinkDataPreviewComponent* InDataPreviewComponent)
{
	UMaterialInterface* DebugMaterial = InDataPreviewComponent->GetMaterial(0);
	if(DebugMaterial)
	{
		UMaterialInstanceDynamic* DynamicMaterial = InDataPreviewComponent->CreateDynamicMaterialInstance(0, DebugMaterial);
		InDataPreviewComponent->SetMaterial(0, DynamicMaterial);
		DynamicMaterial->ClearParameterValues();
		/*Generate a random color that is weighted toward 1 or 2 primary colors.*/
		float Red01 = FMath::FRandRange(0.33,0.8);
		float Green01 = FMath::FRandRange(0.33,0.8);
		float Blue01 = FMath::FRandRange(0.33,0.8);
		float Red02 = Red01 - Green01;
		float Green02 = Green01 - Blue01;
		float Blue02 = Green01 - Red01;
		DynamicMaterial->SetVectorParameterValue(FName(TEXT("Color")), FLinearColor(FMath::RandBool() ? Red01 : Red02, FMath::RandBool() ? Green01 : Green02, FMath::RandBool() ? Blue01 : Blue02, 1.0) );
	}
}
