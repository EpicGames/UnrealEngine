// Copyright Epic Games, Inc. All Rights Reserved.


#include "Visualizers/LiveLinkDataPreviewComponent.h"

#include "AnimationCoreLibrary.h"
#include "CanvasTypes.h"
#include "ILiveLinkClient.h"
#include "LiveLinkClient.h"
#include "Features/IModularFeatures.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkLocatorRole.h"
#include "Roles/LiveLinkTransformRole.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkDataPreviewComponent)

ULiveLinkDataPreviewComponent::ULiveLinkDataPreviewComponent(const FObjectInitializer& ObjectInitializer)
	: bEvaluateLiveLink(true)
	, bDrawLabels(false)
	, bIsDirty(true)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true; //make this component tick in editor
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_PostUpdateWork;
}

void ULiveLinkDataPreviewComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	//TODO: add drawing labels for bones and transforms. 
	
	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient.GetSubjectRole_AnyThread(SubjectName);
	
	if (SubjectRole && !bIsDirty)
	{
		//Process Animation Data. Animation Role handling.
		if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkAnimationRole::StaticClass()))
		{
			if(LiveLinkClient.IsSubjectValid(SubjectName)) //Check there is valid data on the subject before evaluate it. 
			{
				FLiveLinkSubjectFrameData SubjectFrameData;
				LiveLinkClient.EvaluateFrame_AnyThread(SubjectName, LiveLinkClient.GetSubjectRole_AnyThread(SubjectName), SubjectFrameData);

				FLiveLinkSkeletonStaticData* SkeletonStaticData = SubjectFrameData.StaticData.Cast<FLiveLinkSkeletonStaticData>();
				FLiveLinkAnimationFrameData* AnimationFrameData = SubjectFrameData.FrameData.Cast<FLiveLinkAnimationFrameData>();
				check(SkeletonStaticData);
				check(AnimationFrameData);

				CacheSkeletalAnimationData(SkeletonStaticData, AnimationFrameData);

				if(GetInstanceCount()<1)
				{
					CreateInstances();
				}

				if(bEvaluateLiveLink)
				{
					switch (BoneVisualType)
					{
					case ELiveLinkVisualBoneType::Joint:
						{
							BatchUpdateInstancesTransforms(0, GetJointTransforms(), true, true, false);
							break;
						}
					case ELiveLinkVisualBoneType::Bone:
						{
							BatchUpdateInstancesTransforms(0, GetBoneTransforms(), true, true, false);
							break;
						}
					}
				}
			}
		}
		//Locator role handling
		else if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkLocatorRole::StaticClass()) && bEvaluateLiveLink)
		{
			
			TArray<FTransform> Transforms;
			TArray<FVector> Locators = GetLocatorPositions();

			for (int32 i = 0; i < Locators.Num(); i++)
			{
				FVector Locator = Locators[i];
				FTransform Transform = FTransform::Identity;
				Transform.SetTranslation(Locator);
				Transforms.Emplace(Transform * GetComponentTransform());
			}
			BatchUpdateInstancesTransforms(0, Transforms, true, true, false);
		}
		//Transform role handling.
		else if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkTransformRole::StaticClass()) && bEvaluateLiveLink)
		{
			BatchUpdateInstancesTransform(0,1,GetSingleTransform() * GetComponentTransform(),true,true,false);

			if (bDrawLabels)
			{
				FVector Location = (GetSingleTransform() * GetComponentTransform()).GetTranslation();
				float Time = 1.0;
				UWorld* World = GetWorld();
				DrawDebugString(World, Location, TEXT("this is some text"), GetOwner(), FColor::Red, Time, false, 1.0);
			}
		}
	}
	if (bIsDirty)
	{
		CreateInstances();
	}
}

#if WITH_EDITOR
void ULiveLinkDataPreviewComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property != nullptr)
	{
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkDataPreviewComponent, SubjectName))
		{
			ClearInstances();
			bIsDirty = true;
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkDataPreviewComponent, bEvaluateLiveLink))
		{
			ClearInstances();
			bIsDirty = true;
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkDataPreviewComponent, BoneVisualType))
		{
			ClearInstances();
			bIsDirty = true;
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkDataPreviewComponent, bDrawLabels))
		{
			ClearInstances();
			bIsDirty = true;
		}
	}
	
}
#endif

void ULiveLinkDataPreviewComponent::OnRegister()
{
	Super::OnRegister();
	
	ClearInstances();

	bIsDirty = true;
}

void ULiveLinkDataPreviewComponent::OnUnregister()
{
	Super::OnUnregister();

	ClearInstances();
	
}

void ULiveLinkDataPreviewComponent::GetTransformRootSpace(const int32 InTransformIndex, FTransform& OutTransform) const
{
	// Case: Root joint or invalid
	OutTransform = FTransform::Identity;
	
	if (IsValidTransformIndex(InTransformIndex, CachedAnimationData))
	{
		TPair<bool, FTransform>& RootSpaceCache = CachedRootSpaceTransforms[InTransformIndex];
		bool bHasValidCache = RootSpaceCache.Key;
		// Case: Have Cached Value
		if (bHasValidCache)
		{
			OutTransform = RootSpaceCache.Value;
		}
		// Case: Need to generate Cache
		else
		{
			const TArray<int32>& BoneParents = CachedSkeletonData.GetBoneParents();
			int32 ParentIndex = BoneParents[InTransformIndex];

			const FTransform& LocalSpaceTransform = CachedAnimationData.Transforms[InTransformIndex];

			FTransform ParentRootSpaceTransform;

			if(InTransformIndex == 0 && CachedSkeletonData.BoneParents[0] == 0) //special case handling for virtual subjects where bone[0] has parent bone[0]
			{
				ParentRootSpaceTransform = FTransform::Identity;
			}
			else
			{
				GetTransformRootSpace(ParentIndex, ParentRootSpaceTransform);
			}

			OutTransform = LocalSpaceTransform * ParentRootSpaceTransform;

			// Save cached results
			RootSpaceCache.Key = true;
			RootSpaceCache.Value = OutTransform;
		}
	}
}

//Add the static mesh instances to represent the Live Link data. 
void ULiveLinkDataPreviewComponent::CreateInstances()
{
	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient.GetSubjectRole_AnyThread(SubjectName);
	if (SubjectRole)
	{
		//Process Animation Data
		if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkAnimationRole::StaticClass()))
		{
			switch (BoneVisualType)
			{
			case ELiveLinkVisualBoneType::Joint:
				{
					AddInstances(GetJointTransforms(), false, true, false);
					break;
				}
			case ELiveLinkVisualBoneType::Bone:
				{
					AddInstances(GetBoneTransforms(), false, true, false);
					break;
				}
			}
			UE_LOGF(LogLiveLink, Display, "Created Instances %d", GetInstanceCount());
			UE_LOGF(LogLiveLink, Display, "Created Instances %ls", bIsDirty ? TEXT("true") : TEXT("false") );
			bIsDirty=false;
		}
		else if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkLocatorRole::StaticClass()))
		{
			TArray<FTransform> Transforms;
			TArray<FVector> Locators = GetLocatorPositions();

			for (int32 i = 0; i < Locators.Num(); i++)
			{
				FVector Locator = Locators[i];
				FTransform Transform = FTransform::Identity;
				Transform.SetTranslation(Locator);
				Transforms.Emplace(Transform * GetComponentTransform());
			}
			AddInstances(Transforms, false, true, false);
			bIsDirty=false;
		}
		else if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkTransformRole::StaticClass()))
		{
			AddInstance(GetSingleTransform() * GetComponentTransform(),true);
			bIsDirty=false;
		}
	}
}

//Loop through a bone's parent's to get it's transform in actor space. Live Link bone transforms are expressed in parent bone space so we need to do this to convert to actor space. 
TArray<FTransform> ULiveLinkDataPreviewComponent::GetBoneTransforms() const
{
	TArray<FTransform> JointTransforms;
		
		FTransform BoneTransform;
		FTransform ParentBoneTransform;
		double BoneLength;
		FVector Aim = FVector( 0, 0, 1);

		const TArray<FTransform> BoneTransforms = CachedAnimationData.Transforms;
		const TArray<int32> BoneParents = CachedSkeletonData.BoneParents;

					
		for (int32 i=0; i < CachedSkeletonData.BoneNames.Num(); i++)
		{
			GetTransformRootSpace(i, BoneTransform);
			BoneTransform = BoneTransform * GetComponentTransform();
			GetTransformRootSpace(CachedSkeletonData.BoneParents[i],ParentBoneTransform);
			ParentBoneTransform = ParentBoneTransform * GetComponentTransform();
			
			BoneLength = FVector::Distance(BoneTransform.GetTranslation(), ParentBoneTransform.GetTranslation());
			BoneTransform.SetScale3D(FVector(1, 1, BoneLength));
			FQuat DiffRotation = AnimationCore::SolveAim(BoneTransform, ParentBoneTransform.GetLocation(), Aim.GetSafeNormal(), false, FVector(1, 1, 1), float(0));
			BoneTransform.SetRotation(DiffRotation);
			JointTransforms.Add(BoneTransform);
		}
	return JointTransforms;
}

//Get bone transforms from the cached skeleton data.
TArray<FTransform> ULiveLinkDataPreviewComponent::GetJointTransforms() const
{
	TArray<FTransform> BoneTransforms;
	
				
		int32 i = 0;
		FTransform BoneTransform;

		for (i=0 ; i < CachedSkeletonData.BoneNames.Num() ; i++)
		{
			GetTransformRootSpace(i, BoneTransform);
			BoneTransform = BoneTransform * GetComponentTransform();
			BoneTransforms.Add(BoneTransform);
		}
	return BoneTransforms;
}

//Get the location of each locator in the locator role and apply it to the instanced static meshes.
TArray<FVector> ULiveLinkDataPreviewComponent::GetLocatorPositions() const
{
	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	
	TArray<FVector> MarkerPostions;

	TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient.GetSubjectRole_AnyThread(SubjectName);
	if (SubjectRole)
	{
		if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkLocatorRole::StaticClass()))
		{
			if(LiveLinkClient.IsSubjectValid(SubjectName))
			{
				FLiveLinkSubjectFrameData SubjectFrameData;
				LiveLinkClient.EvaluateFrame_AnyThread(SubjectName, LiveLinkClient.GetSubjectRole_AnyThread(SubjectName), SubjectFrameData);
							
				const FLiveLinkLocatorStaticData* MarkerStaticData = SubjectFrameData.StaticData.Cast<FLiveLinkLocatorStaticData>();
				const FLiveLinkLocatorFrameData* MarkerFrameData = SubjectFrameData.FrameData.Cast<FLiveLinkLocatorFrameData>();
				check(MarkerStaticData);
				check(MarkerFrameData);

				if (MarkerFrameData->Locators.Num() > 0)
				{
					TArray<FVector> MarkerPositions = MarkerFrameData->Locators;
					return MarkerFrameData->Locators;
				}
			}

		}
	}
	return MarkerPostions;
}

//Get a tranform role's data and applying it to the instance static mesh in actor space.
FTransform ULiveLinkDataPreviewComponent::GetSingleTransform() const
{
	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	FTransform Transform;
	
	if(LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectName, ULiveLinkTransformRole::StaticClass()))
	{
		if(LiveLinkClient.IsSubjectValid(SubjectName))
		{
			FLiveLinkSubjectFrameData SubjectFrameData;
			LiveLinkClient.EvaluateFrame_AnyThread(SubjectName, LiveLinkClient.GetSubjectRole_AnyThread(SubjectName), SubjectFrameData);
			const FLiveLinkTransformFrameData* TransformFrameData = SubjectFrameData.FrameData.Cast<FLiveLinkTransformFrameData>();
			Transform = TransformFrameData->Transform;
		}

	}

	return Transform;
}

//Cached a frame's Live Link Animation role bone data for use later in the frame, updating in the instance static mesh components. 
void ULiveLinkDataPreviewComponent::CacheSkeletalAnimationData(const FLiveLinkSkeletonStaticData* InStaticData, const FLiveLinkAnimationFrameData* InFrameData)
{
	{
		CachedSkeletonData = *InStaticData;
		CachedAnimationData = *InFrameData;

		const int32 NumTransforms = InFrameData->Transforms.Num();
		check(InStaticData->BoneNames.Num() == NumTransforms);
		check(InStaticData->BoneParents.Num() == NumTransforms);
		check(InStaticData->PropertyNames.Num() == InFrameData->PropertyValues.Num());
		CachedRootSpaceTransforms.SetNum(NumTransforms);
		CachedChildTransformIndices.SetNum(NumTransforms);
		for (int32 i = 0; i < NumTransforms; ++i)
		{
			CachedRootSpaceTransforms[i].Key = false;
			CachedChildTransformIndices[i].Key = false;
		}
	}
}

//Check a transform is valid, based on the given index. 
bool ULiveLinkDataPreviewComponent::IsValidTransformIndex(int32 InTransformIndex, FLiveLinkAnimationFrameData InAnimData)
{
	return (InTransformIndex >= 0) && (InTransformIndex < InAnimData.Transforms.Num());
}
