// Copyright Epic Games, Inc. All Rights Reserved.


#include "Visualizers/LiveLinkMarkerVisualizer.h"

#include "Materials/MaterialInstanceDynamic.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "ILiveLinkClient.h"
#include "Features/IModularFeatures.h"
#include "LiveLinkClient.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkTransformRole.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkMarkerVisualizer)

ULiveLinkMarkerVisualizer::ULiveLinkMarkerVisualizer(const FObjectInitializer& ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true; //make this component tick in editor
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_PostUpdateWork;
	bDrawLabels = false;
	XAxisSign = EAxisSign::Positive;
	YAxisSign = EAxisSign::Positive;
	bEvaluateLiveLink = true;
	
}

ULiveLinkMarkerVisualizer::~ULiveLinkMarkerVisualizer()
{
	//Placeholder 
}

void ULiveLinkMarkerVisualizer::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	//If there are no instances, add them. 
	if(GetInstanceCount() == 0)
	{
		AddInstances(GetMarkerTransforms(), true,true,false);
	}
	
	if(bEvaluateLiveLink)
	{
		BatchUpdateInstancesTransforms(0, GetMarkerTransforms(), true,true,false);
	}
	
	if(bDrawLabels)
	{
		DrawLabels(MarkerLabels);
	}
}

TArray<FTransform> ULiveLinkMarkerVisualizer::GetMarkerTransforms()
{
	FLiveLinkSubjectFrameData SubjectFrameData;
	
	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	LiveLinkClient.EvaluateFrame_AnyThread(LiveLinkSubject, LiveLinkClient.GetSubjectRole_AnyThread(LiveLinkSubject), SubjectFrameData);

	const FLiveLinkBaseFrameData* MarkerData = SubjectFrameData.FrameData.GetBaseData();
	const FLiveLinkBaseStaticData* MarkerNames = SubjectFrameData.StaticData.GetBaseData();

	//TODO : special handling for world attached skeletal mesh actor (animation role)

	if(MarkerData)
	{
		//Early out if the two array's aren't the same length
		if(MarkerNames->PropertyNames.Num() != MarkerData->PropertyValues.Num())
		{
			UE_LOGF(LogLiveLink, Warning, "LiveLnk data not valid - length of property values does not match length of property names")
			return MarkerLocations;
		}

		MarkerLocations.Empty();
	
		const TArray<float>& LocationValues = MarkerData->PropertyValues;
		float XScale = GetAxisSign(XAxisSign);
		float YScale = GetAxisSign(YAxisSign);
		FTransform MarkerTransform;
		AActor* OwnerActor = GetOwner();
		AActor* AttachedParent = GetOwner()->GetAttachParentActor();

		if(LiveLinkClient.GetSubjectRole_AnyThread(LiveLinkSubject) == ULiveLinkAnimationRole::StaticClass())
		{
			for ( int i=0; i < MarkerData->PropertyValues.Num();  i = i + 3)
			{
				MarkerTransform = FTransform(FVector(LocationValues[i], LocationValues[i+1], LocationValues[i+2])) * FTransform(FRotator(0,0,0), FVector(0,0,0),FVector(XScale,YScale,1));
				MarkerTransform = MarkerTransform * OwnerActor->GetActorTransform();
				//MarkerLocations.Emplace(MarkerTransform);
				MarkerLocations.Add(MarkerTransform);
			}
		}
		else //If not an animation role assume a transform or a basic role
		{
			for ( int i=0; i < MarkerData->PropertyValues.Num();  i = i + 3)
			{
				if(AttachedParent!=nullptr)
				{
					MarkerTransform = FTransform(FVector(LocationValues[i], LocationValues[i+1], LocationValues[i+2])) * FTransform(FRotator(0,0,0), FVector(0,0,0),FVector(XScale,YScale,1));
					MarkerTransform = MarkerTransform * AttachedParent->GetActorTransform();
					MarkerLocations.Add(MarkerTransform);
				}
				else
				{
					MarkerTransform = FTransform(FVector(LocationValues[i], LocationValues[i+1], LocationValues[i+2])) * FTransform(FRotator(0,0,0), FVector(0,0,0),FVector(XScale,YScale,1));
					MarkerLocations.Add(MarkerTransform);
				}
			}
		}
		
	}
	return MarkerLocations;
}

void ULiveLinkMarkerVisualizer::DrawLabels(TArray<FName> Labels)
{
	//ToDo
}

void ULiveLinkMarkerVisualizer::OnRegister()
{
	Super::OnRegister();

	//ClearInstances();
	
	AddInstances(GetMarkerTransforms(), true,true,false);
}

void ULiveLinkMarkerVisualizer::OnUnregister()
{
	Super::OnUnregister();

	ClearInstances();
}
