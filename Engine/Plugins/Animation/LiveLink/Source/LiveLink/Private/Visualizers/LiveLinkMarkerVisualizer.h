// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "LiveLinkTypes.h"
#include "ILiveLinkClient.h"
#include "LiveLinkMarkerVisualizer.generated.h"

#define UE_API LIVELINK_API

class UMaterialInstanceDynamic;

/**
 * Sign of the axis, to handle data coming from non-left handed systems.
 */
UENUM(BlueprintType)
enum class EAxisSign : uint8
{
	Positive = 0	UMETA(DisplayName = "Positive"),
	Negative = 1	UMETA(DisplayName = "Negative")
};

/**
 * An Instanced Static Mesh Component to represent Motion Capture marker data locations
 */
UCLASS(MinimalAPI, ClassGroup=(PerformanceCapture), meta=(BlueprintSpawnableComponent),Transient, DisplayName = "MocapMarkerVisualizer")
class ULiveLinkMarkerVisualizer : public UInstancedStaticMeshComponent
{
	GENERATED_BODY()

public:
	//Sets default values for this component's properties
	UE_API ULiveLinkMarkerVisualizer(const FObjectInitializer& ObjectInitializer);

	//Destructor
	UE_API ~ULiveLinkMarkerVisualizer();

	//Called every frame
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	/** Get marker transforms defined in mocap tracking space */
	UE_API TArray<FTransform> GetMarkerTransforms();

	static UE_API void DrawLabels(TArray<FName> Labels);

	/** Dynamic material to use on the instance static meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="PerformanceCapture|Visualization")
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;

	/** Live Link subject to drive the instanced static mesh transforms. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="PerformanceCapture|Visualization")
	FLiveLinkSubjectName LiveLinkSubject;

	/** Bool to control evaluation of Live Link data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="PerformanceCapture|Visualization")
	bool bEvaluateLiveLink;

	//To Do - add text labels to drawn markers
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="PerformanceCapture|Visualization")
	bool bDrawLabels;

	/** Sign for the x axis of the marker. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="PerformanceCapture|Visualization")
	EAxisSign XAxisSign;

	/** Sign for the y axis of the marker. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="PerformanceCapture|Visualization")
	EAxisSign YAxisSign;

	/** Array of Transforms for the markers.  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="PerformanceCapture|Visualization")
	TArray<FTransform> MarkerLocations;

protected:
	
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

private:
	
	TArray<FName> MarkerLabels;

	static float GetAxisSign(EAxisSign AxisSign)
	{
		switch (AxisSign)
		{case EAxisSign::Positive:return 1.0f; case EAxisSign::Negative:return -1.0f; default: return 1.0f;	}
	}
};

#undef UE_API
