// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/**
 * Continually estimates the time offset between two live time sources.
 *
 * Samples of clock times corresponding to the same instant are provided,
 * and it will update its estimation of the time difference between the clocks.
 */
class FClockOffsetEstimatorRamp
{
public:

	/** Get current estimated offset between the two tracked clocks */
	double GetEstimatedOffset() const;

	/** Update the current time offset estimation with a new data sample */
	void UpdateEstimation(double Atime, double Btime);

	/** Set required consecutive big errors to clamp */
	void SetRequiredConsecutiveBigError(int32 NewCount);

	/** Set maximum allowed estimation error before looking to clamp */
	void SetMaxAllowedEstimationError(double NewValue);

	/** Sets incremental correction step */
	void SetCorrectionStep(double NewValue);

	/** Puts back the estimator in its initial state */
	void Reset();

protected:

	/** Current estimated offset between two clocks */
	double EstimatedOffset = 0.0;
	

	/** Maximum error tolerated before considering clamping to current offset */
	double MaxAllowedEstimationError = 0.25;

	/** Correction step to apply, positive or negative depending on error sign */
	double CorrectionStep = 100e-6;

	/** Consecutive time a too big estimation error is required before clamping */
	int32 ConsecutiveTooBigErrorRequired = 5;
	
	/** Current consecutive big error count. Initialized at required count so first pass snaps */
	int32 ConsecutiveBigErrorCount = 5;
};
