// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

#ifndef WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
#define WITH_LIVELINK_DISCOVERY_MANAGER_THREAD 1
#endif

class FDelegateHandle;
class FLiveLinkHeartbeatEmitter;
class FLiveLinkMessageBusDiscoveryManager;
struct FLiveLinkSubjectKey;
struct FProviderPollResult;
class FSlateStyleSet;

using FProviderPollResultPtr = TSharedPtr<FProviderPollResult, ESPMode::ThreadSafe>;


DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnLiveLinkShouldDisplaySource, UClass*/*FactoryClass*/, FProviderPollResultPtr/*PollResultPtr*/);

/**
 * Interface for messaging modules.
 */
class ILiveLinkModule
	: public IModuleInterface
{
public:

	/**
	 * Gets a reference to the live link module instance.
	 *
	 * @return A reference to the live link module.
	 */
	static ILiveLinkModule& Get()
	{
#if PLATFORM_IOS
		static ILiveLinkModule& LiveLinkModule = FModuleManager::LoadModuleChecked<ILiveLinkModule>("LiveLink");
		return LiveLinkModule;
#else
		return FModuleManager::LoadModuleChecked<ILiveLinkModule>("LiveLink");
#endif
	}

	virtual TSharedPtr<FSlateStyleSet> GetStyle() = 0;
	virtual FLiveLinkHeartbeatEmitter& GetHeartbeatEmitter() = 0;
#if WITH_LIVELINK_DISCOVERY_MANAGER_THREAD
	virtual FLiveLinkMessageBusDiscoveryManager& GetMessageBusDiscoveryManager() = 0;
#endif

	/**
     * Register a message bus source filter invoked by the MessageBus Source Factory to filter out discovered sources.
     * The delegate should return true if the poll result should be included in the discovered sources list, and false otherwise.
     * @param SourceFilter The filter invoked by the source factory.
     * @return A handle to the delegate, used for unregistering the filter.
     */
	virtual FDelegateHandle RegisterMessageBusSourceFilter(const FOnLiveLinkShouldDisplaySource& SourceFilter) = 0;

	/**
	 * Unregister a previously registered message bus source filter.
	 * @param Handle A handle to the filter delegate that was registered.
	 */
	virtual void UnregisterMessageBusSourceFilter(FDelegateHandle Handle) = 0;

	/**
	 * @return The list of source filters registered with this module.
	 */
	virtual const TMap<FDelegateHandle, FOnLiveLinkShouldDisplaySource>& GetSourceFilters() const = 0;

	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnSubjectOutboundNameModified, const FLiveLinkSubjectKey& /*Key*/, const FString& /*PreviousOutboundName*/, const FString& /*OutboundName*/);
	/** (Internal use) Delegate called by LiveLinkHubSubjectSettings to notify. */
	virtual FOnSubjectOutboundNameModified& OnSubjectOutboundNameModified() = 0;
public:

	/** Virtual destructor. */
	virtual ~ILiveLinkModule() = default;
};
