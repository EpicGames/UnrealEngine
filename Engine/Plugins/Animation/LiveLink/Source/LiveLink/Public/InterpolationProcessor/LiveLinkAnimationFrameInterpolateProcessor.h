// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterpolationProcessor/LiveLinkBasicFrameInterpolateProcessor.h"
#include "LiveLinkAnimationFrameInterpolateProcessor.generated.h"

#define UE_API LIVELINK_API


/**
 * Default blending method for animation frames
 */
UCLASS(MinimalAPI, meta=(DisplayName="Animation Interpolation"))
class ULiveLinkAnimationFrameInterpolationProcessor : public ULiveLinkBasicFrameInterpolationProcessor
{
	GENERATED_BODY()

public:
	class FLiveLinkAnimationFrameInterpolationProcessorWorker : public ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker
	{
	public:
		FLiveLinkAnimationFrameInterpolationProcessorWorker(bool bInterpolatePropertyValues);

		virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
		virtual void Interpolate(double InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo) override;
		virtual void Interpolate(const FQualifiedFrameTime& InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo) override;
	};

public:
	UE_API virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
	UE_API virtual ULiveLinkFrameInterpolationProcessor::FWorkerSharedPtr FetchWorker() override;

private:
	TSharedPtr<FLiveLinkAnimationFrameInterpolationProcessorWorker, ESPMode::ThreadSafe> Instance;
};

#undef UE_API
