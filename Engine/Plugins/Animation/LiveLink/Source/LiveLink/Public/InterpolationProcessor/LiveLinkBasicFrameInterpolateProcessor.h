// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkFrameInterpolationProcessor.h"
#include "LiveLinkBasicFrameInterpolateProcessor.generated.h"

#define UE_API LIVELINK_API

/**
 * Default blending method for any type of frames.
 * It will interpolate numerical properties that are mark with "Interp".
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Base Interpolation"))
class ULiveLinkBasicFrameInterpolationProcessor : public ULiveLinkFrameInterpolationProcessor
{
	GENERATED_BODY()

public:
	class FLiveLinkBasicFrameInterpolationProcessorWorker : public ILiveLinkFrameInterpolationProcessorWorker
	{
	public:
		UE_API FLiveLinkBasicFrameInterpolationProcessorWorker(bool bInterpolatePropertyValues);

		UE_API virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
		
		UE_API virtual void Interpolate(double InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo) override;
		UE_API virtual void Interpolate(const FQualifiedFrameTime& InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo) override;

		struct FGenericInterpolateOptions
		{
			bool bInterpolatePropertyValues = true;
			bool bCopyClosestFrame = true;
			bool bCopyClosestMetaData = true; // only used if bCopyClosestFrame is false. Does NOT apply to SceneTime, it will always be interpolated.
			bool bInterpolateInterpProperties = true;
		};

		static UE_API void GenericInterpolate(double InBlendFactor, const FGenericInterpolateOptions& Options, const FLiveLinkFrameDataStruct& FrameDataA, const FLiveLinkFrameDataStruct& FrameDataB, FLiveLinkFrameDataStruct& OutBlendedFrame);
		static UE_API double GetBlendFactor(double InTime, const FLiveLinkFrameDataStruct& FrameDataA, const FLiveLinkFrameDataStruct& FrameDataB);
		static UE_API double GetBlendFactor(FQualifiedFrameTime InTime, const FLiveLinkFrameDataStruct& FrameDataA, const FLiveLinkFrameDataStruct& FrameDataB);
		static UE_API bool FindInterpolateIndex(double InTime, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, int32& OutFrameIndexA, int32& OutFrameIndexB, FLiveLinkInterpolationInfo& OutInterpolationInfo);
		static UE_API bool FindInterpolateIndex(FQualifiedFrameTime InTime, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, int32& OutFrameIndexA, int32& OutFrameIndexB, FLiveLinkInterpolationInfo& OutInterpolationInfo);

	protected:
		bool bInterpolatePropertyValues = true;
	};

public:
	UE_API virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
	UE_API virtual ULiveLinkFrameInterpolationProcessor::FWorkerSharedPtr FetchWorker() override;

public:
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	bool bInterpolatePropertyValues = true;

private:
	TSharedPtr<FLiveLinkBasicFrameInterpolationProcessorWorker, ESPMode::ThreadSafe> BaseInstance;
};

#undef UE_API
