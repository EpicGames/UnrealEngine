// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkVirtualSubject.h"
#include "LiveLinkVirtualSubjectBoneAttachment.h"


#include "LiveLinkAnimationVirtualSubject.generated.h"

#define UE_API LIVELINK_API

struct FLiveLinkSubjectFrameData;
struct FLiveLinkSubjectKey;

/** What action should be taken when a bone name conflict happens between a parent and a child subject. */
UENUM()
enum class EBoneTransformResolution
{
	/** Keep parent bone transform. */
	KeepParent,
	/** Keep child bone transform. */
	KeepChild,
	/** Combine the child and parent's bone transforms. */
	Combine
};

/** A Skeleton virtual subject is an assembly of different subjects supporting the animation or basic role */
UCLASS(MinimalAPI, meta=(DisplayName="Animation Virtual Subject"))
class ULiveLinkAnimationVirtualSubject : public ULiveLinkVirtualSubject
{
	GENERATED_BODY()

public:
	UE_API ULiveLinkAnimationVirtualSubject();

	UE_API virtual void Update() override;

	//~ Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

protected:
	/** Validates current source subjects */
	UE_API bool AreSubjectsValid(const TArray<FLiveLinkSubjectKey>& InActiveSubjects) const;

	UE_API bool BuildSubjectSnapshot(TArray<FLiveLinkSubjectFrameData>& OutSnapshot);

	/** Builds a new ref skeleton based on the current subject state(can early out if ref skeleton is already up to date) */
	UE_API virtual void BuildSkeleton(const TArray<FLiveLinkSubjectFrameData>& InSubjectSnapshots);

	UE_API virtual void BuildFrame(const TArray<FLiveLinkSubjectFrameData>& InSubjectSnapshots);

	/** Called after the update method. */
	virtual void PostSkeletonRebuild() {}

	/** Tests to see if current ref skeleton is up to data */
	UE_API bool DoesSkeletonNeedRebuilding() const;
	bool bInvalidate;

	/** Sort the subjects according to the dependencies introduced by the attachments. */
	UE_API void SortSubjects();

	/** Build the ChildBonesInfo list from the specified attachments. */
	UE_API void ProcessAttachmentsForStaticData(TArray<int32>& InOutBoneParents);

	/**  Apply transform offsets generated by the attachments to the frame data. */
	UE_API void ProcessAttachmentsForFrameData(struct FLiveLinkAnimationFrameData* SnapshotFrameData);

	//~ LiveLinkVirtualSubject Interface
	ELiveLinkSubjectState GetState() const override
	{
		if (IsPaused())
		{
			return ELiveLinkSubjectState::Paused;
		}
		else if (!bLastSnapshotIsComplete)
		{
			return ELiveLinkSubjectState::InvalidOrDisabled;
		}
		else if (HasValidFrameSnapshot())
		{
			return ELiveLinkSubjectState::Connected;
		}
		else
		{
			return ELiveLinkSubjectState::InvalidOrDisabled;
		}
	}

public:
	/** List of bone attachments between subjects. */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	TArray<FLiveLinkVirtualSubjectBoneAttachment> Attachments;

	/** What should happen to the location of a bone when there's a conflict between the child and parent subject. */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	EBoneTransformResolution LocationBehavior = EBoneTransformResolution::Combine;

	/** What should happen to the rotation of a bone when there's a conflict between the child and parent subject. */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	EBoneTransformResolution RotationBehavior = EBoneTransformResolution::Combine;

	/** Whether to append SubjectName to each bones part of the virtual hierarchy */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bAppendSubjectNameToBones;

private:
	/** Information about a bone that needs to be attached to a parent subject. */
	struct FChildBoneInfo
	{
		/** Parent bone global index. */
		int32 ParentBone = INDEX_NONE;
		/** Offset of the child bone to its parent bone. */
		FTransform Offset = FTransform::Identity;
	};

	/** Map of global child indices to their respective bone info structure.  */
	TMap<int32, FChildBoneInfo> ChildBonesInfo;

	/** Lookup table of Subject + bone name to index in the concatenated list of bone names. */
	TMap<TPair<FName, FName>, int32> BoneNameToIndex;

	/** Cached snapshots, used when the subjects are invalid in order to maintain tranmission of data. */
	TArray<FLiveLinkSubjectFrameData> LastFrameSnapshots;

	/** 
	 * Returns whether all snapshots were successfully created. 
	 * Will be false if no snapshot was created, or if one of the subject was invalid, causing the subject to use the previous frame's data. 
	 */
	bool bLastSnapshotIsComplete = false;

	/** Whether the subjects need to be sorted because an attachment has been modified. */
	bool bSubjectsNeedSorting = false;
};

#undef UE_API
