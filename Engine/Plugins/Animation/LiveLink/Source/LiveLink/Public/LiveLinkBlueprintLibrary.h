// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Kismet/BlueprintFunctionLibrary.h"
#include "ILiveLinkClient.h"

#include "LiveLinkBlueprintLibrary.generated.h"

#define UE_API LIVELINK_API

class ULiveLinkRole;
struct FLiveLinkAnimationFrameData;
struct FLiveLinkBaseBlueprintData;
struct FLiveLinkBasicBlueprintData;
struct FLiveLinkSkeletonStaticData;
struct FLiveLinkSourceHandle;
struct FLiveLinkSubjectKey;
struct FLiveLinkSubjectName;
struct FLiveLinkSubjectRepresentation;
struct FLiveLinkTransform;
struct FSubjectFrameHandle;
struct FSubjectMetadata;
struct FTimecode;
template <typename T> class TSubclassOf;

UCLASS(MinimalAPI)
class ULiveLinkBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

// FLiveLinkBasicBlueprintData

	// Returns the value of a property stored in the Subject Frame
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static bool GetPropertyValue(UPARAM(ref) FLiveLinkBasicBlueprintData& BasicData, FName PropertyName, float& Value);

// FSubjectFrameHandle 

	// Returns the float curves stored in the Subject Frame as a map
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static void GetCurves(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, TMap<FName, float>& Curves);

	// Returns the number of Transforms stored in the Subject Frame
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static int NumberOfTransforms(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle);

	// Returns an array of Transform Names stored in the Subject Frame
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static void TransformNames(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, TArray<FName>& TransformNames);

	// Returns the Root Transform for the Subject Frame as a LiveLink Transform or the Identity if there are no transforms.
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static void GetRootTransform(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkTransform& LiveLinkTransform);

	// Returns the LiveLink Transform stored in a Subject Frame at a given index. Returns an Identity transform if Transform Index is invalid.
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static void GetTransformByIndex(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, int TransformIndex, FLiveLinkTransform& LiveLinkTransform);

	// Returns the LiveLink Transform stored in a Subject Frame with a given name. Returns an Identity transform if Transform Name is invalid.
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static void GetTransformByName(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FName TransformName, FLiveLinkTransform& LiveLinkTransform);

	// Returns the Subject Metadata structure stored in the Subject Frame
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static void GetMetadata(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FSubjectMetadata& Metadata);

	// Returns the Subject base structure stored in the Subject Frame
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static void GetBasicData(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkBasicBlueprintData& BasicBlueprintData);

	// Returns the Subject's static data stored in the Subject Frame. Returns false if no valid data found.
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static bool GetAnimationStaticData(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkSkeletonStaticData& AnimationStaticData);

	// Returns the Subject's frame data stored in the Subject Frame. Returns false if no valid data found.
	UFUNCTION(BlueprintPure, Category = "LiveLink|Animation")
	static bool GetAnimationFrameData(UPARAM(ref) FSubjectFrameHandle& SubjectFrameHandle, FLiveLinkAnimationFrameData& AnimationFrameData);

// FLiveLinkTransform

	// Returns the Name of a given LiveLink Transform
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static void TransformName(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FName& Name);

	// Returns the Transform value in Parent Space for a given LiveLink Transform
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static void ParentBoneSpaceTransform(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FTransform& Transform);

	// Returns the Transform value in Root Space for a given LiveLink Transform
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static void ComponentSpaceTransform(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FTransform& Transform);

	// Returns whether a given LiveLink Transform has a parent transform
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static bool HasParent(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform);

	// Returns the Parent LiveLink Transform if one exists or an Identity transform if no parent exists
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static void GetParent(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, FLiveLinkTransform& Parent);

	// Returns the number of Children for a given LiveLink Transform
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static int ChildCount(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform);

	// Returns an array of Child LiveLink Transforms for a given LiveLink Transform
	UFUNCTION(BlueprintPure, Category = "LiveLink")
	static void GetChildren(UPARAM(ref) FLiveLinkTransform& LiveLinkTransform, TArray<FLiveLinkTransform>& Children);

// FLiveLinkSourceHandle

	// Checks whether the LiveLink Source is valid via its handle
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static bool IsSourceStillValid(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle);

	// Requests the given LiveLink Source to shut down via its handle
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static bool RemoveSource(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle);

	// Get the text status of a LiveLink Source via its handle
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static FText GetSourceStatus(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle);

	// Get the type of a LiveLink Source via its handle
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static FText GetSourceType(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle);

	// Get the machine name of a LiveLink Source via its handle
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static FText GetSourceMachineName(UPARAM(ref) FLiveLinkSourceHandle& SourceHandle);

public:
	/**
	 * Get the type of a source from the given GUID
	 * @param SourceGuid the GUID identifying the LiveLink Source
	 * @return The type of the Source as Text
	 */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API FText GetSourceTypeFromGuid(FGuid SourceGuid);
	
	/** Get a list of all enabled subject names */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API TArray<FLiveLinkSubjectName> GetLiveLinkEnabledSubjectNames(bool bIncludeVirtualSubject);

	/** Get a list of all subjects */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API TArray<FLiveLinkSubjectKey> GetLiveLinkSubjects(bool bIncludeDisabledSubject, bool bIncludeVirtualSubject);

	/**
	 * Whether or not a subject from the specific source is the enabled subject.
	 * Only 1 subject with the same name can be enabled.
	 * At the start of the frame, a snapshot of the enabled subjects will be made.
	 * That snapshot dictate which subject will be used for the duration of that frame.
	 */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API bool IsSpecificLiveLinkSubjectEnabled(const FLiveLinkSubjectKey SubjectKey, bool bForThisFrame);

	/**
	 * Whether or not the client has a subject with this name enabled
	 * Only 1 subject with the same name can be enabled.
	 * At the start of the frame, a snapshot of the enabled subjects will be made.
	 * That snapshot dictate which subject will be used for the duration of that frame.
	 */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API bool IsLiveLinkSubjectEnabled(const FLiveLinkSubjectName SubjectName);

	/**
	 * Returns the state of the subject with the given name.
	 */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API ELiveLinkSubjectState GetLiveLinkSubjectState(const FLiveLinkSubjectName SubjectName);

	/** 
	 * Set the subject's from a specific source to enabled, disabling the other in the process.
	 * Only 1 subject with the same name can be enabled.
	 * At the start of the frame, a snapshot of the enabled subjects will be made.
	 * That snapshot dictate which subject will be used for the duration of that frame.
	 * SetSubjectEnabled will take effect on the next frame.
	 */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API void SetLiveLinkSubjectEnabled(const FLiveLinkSubjectKey SubjectKey, bool bEnabled);

	/** Get the role of a subject from a specific source */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API TSubclassOf<ULiveLinkRole> GetSpecificLiveLinkSubjectRole(const FLiveLinkSubjectKey SubjectKey);

	/** Get the role of the subject with this name */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API TSubclassOf<ULiveLinkRole> GetLiveLinkSubjectRole(const FLiveLinkSubjectName SubjectName);

	/** Pause a subject, freezing its evaluation to the last data it received. */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API void PauseSubject(const FLiveLinkSubjectName SubjectName);
	
	/** Unpause a subject, resuming its normal operation. */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API void UnpauseSubject(const FLiveLinkSubjectName SubjectName);

	UE_DEPRECATED(4.23, "EvaluateLiveLinkFrame with Subject Represention is deprecated, recreate the node.")
	UFUNCTION(BlueprintCallable, CustomThunk, Category = LiveLink, meta = (DeprecatedFunction, CustomStructureParam = "OutBlueprintData", BlueprintInternalUseOnly = "true", AllowAbstract = "false"))
	static UE_API bool EvaluateLiveLinkFrame(FLiveLinkSubjectRepresentation SubjectRepresentation, FLiveLinkBaseBlueprintData& OutBlueprintData);

	/** Fetches a frame on a subject for a specific role. Output is evaluated based on the role */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = LiveLink, meta = (DisplayName="EvaluateLiveLinkFrame", CustomStructureParam = "OutBlueprintData", BlueprintInternalUseOnly = "true", AllowAbstract = "false"))
	static UE_API bool EvaluateLiveLinkFrameWithSpecificRole(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkBaseBlueprintData& OutBlueprintData);

	/** Fetches a frame on a subject for a specific role at an offset from the application current time. Output is evaluated based on the role */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = LiveLink, meta = (CustomStructureParam = "OutBlueprintData", BlueprintInternalUseOnly = "true", AllowAbstract = "false"))
	static UE_API bool EvaluateLiveLinkFrameAtWorldTimeOffset(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, float WorldTimeOffset, FLiveLinkBaseBlueprintData& OutBlueprintData);

	/**
	 * Fetches a frame on a subject for a specific role at a specified scene time (timecode).
	 * The Timecode should be at the frame rate as the engine timecode.
	 * Output is evaluated based on the role */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = LiveLink, meta = (CustomStructureParam = "OutBlueprintData", BlueprintInternalUseOnly = "true", AllowAbstract = "false"))
	static UE_API bool EvaluateLiveLinkFrameAtSceneTime(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, FTimecode SceneTime, FLiveLinkBaseBlueprintData& OutBlueprintData);

private:
	DECLARE_FUNCTION(execEvaluateLiveLinkFrame);
	DECLARE_FUNCTION(execEvaluateLiveLinkFrameWithSpecificRole);
	DECLARE_FUNCTION(execEvaluateLiveLinkFrameAtWorldTimeOffset);
	DECLARE_FUNCTION(execEvaluateLiveLinkFrameAtSceneTime);
};

#undef UE_API
