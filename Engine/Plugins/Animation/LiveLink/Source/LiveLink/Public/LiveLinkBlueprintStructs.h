// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "Roles/LiveLinkAnimationBlueprintStructs.h"

UE_DEPRECATED(4.23, "Including LiveLinkBlueprintStructs.h has been deprecated. Please include Roles/LiveLinkAnimationBlueprintStructs.h instead.")
inline void LiveLinkBlueprintStructsHeaderDeprecatedWarning()
{
}

inline void TriggerLiveLinkBlueprintStructsHeaderDeprecatedWarning()
{
	LiveLinkBlueprintStructsHeaderDeprecatedWarning();
}
