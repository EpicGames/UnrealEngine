// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILiveLinkSource.h"
#include "LiveLinkPresetTypes.h"
#include "Components/ActorComponent.h"
#include "Templates/SharedPointer.h"

#include "LiveLinkBroadcastComponent.generated.h"

class FLiveLinkBroadcastSource;

#define UE_API LIVELINK_API

UCLASS(BlueprintType, DisplayName="Live Link Broadcast", ClassGroup=("LiveLink"), meta=(BlueprintSpawnableComponent), HideCategories=(Navigation, Tags, AssetUserData, Activation), MinimalAPI)
class ULiveLinkBroadcastComponent : public UActorComponent 
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	UE_API ULiveLinkBroadcastComponent();

	/** Whether this component should broadcast data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Live Link Broadcast")
	bool bEnable = true;

	/**
	 * The name of the Live Link subject. Will default to the actor name if left blank
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Live Link Broadcast")
	FName SubjectName;

	/** 
	 * What role should be used for evaluating the data.
	 * Choosing Transform Role will only send the transform of the chosen component, while picking Animation will send the full skeleton transforms.
	 **/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Live Link Broadcast")
	TSubclassOf<ULiveLinkRole> Role;

	/**
	 * The source mesh to broadcast
	 * todo: Use customization to restrict the allowed classes metadata
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Live Link Broadcast", meta=(UseComponentPicker))
	FComponentReference SourceMesh;

	/**
	 * Optional list of allowed bone names (useful for Metahumans and other complex assets).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Live Link Broadcast",  meta=(ToolTip="Use this array to provide an inclusive list of bones to broadcast"))
	TArray<FName> AllowedBoneNames;

	/**
	 * Optional list of allowed curve names (useful for Metahumans and other complex assets).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Live Link Broadcast", meta=(ToolTip="Curve data will be added to the broadcast data if listed here"))
	TArray<FName> AllowedCurveNames;

	/** Whether the livelink subject should be removed when broadcasting is disabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Live Link Broadcast")
	bool bRemoveSubjectWhenDisabled = false;

	//~ UActorComponent interface
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	/** Trigger retransmission of static data on the next tick. */
	void MarkDirty()
	{
		bIsDirty = true;
	}

	/** Sets whether this component should broadcast live link data. */
	UE_API void SetEnabled(bool bInEnabled);

private:
	/** Creates the rebroadcast source if it doesn't exist. */
	void InitiateLiveLink();

	//~ Populates a static data struct from the chosen mesh
	FLiveLinkStaticDataStruct GetSkeletonStaticData() const;
	FLiveLinkStaticDataStruct GetCharacterStaticData() const;
	FLiveLinkStaticDataStruct GetCameraStaticData() const;
	FLiveLinkStaticDataStruct GetTransformStaticData() const;

	//~ Populates a frame data struct from the chosen mesh
	FLiveLinkFrameDataStruct GetSkeletonFrameData() const;
	FLiveLinkFrameDataStruct GetCharacterFrameData() const;
	FLiveLinkFrameDataStruct GetCameraFrameData() const;
	FLiveLinkFrameDataStruct GetTransformFrameData() const;

	/** Get bone names from the skekelal mesh. */
	TArray<FName> GetSkeletalBoneNames() const;

	/** Get bone parents from the skekelal mesh. */
	TArray<int32> GetSkeletalBoneParents() const;

	/** Get the bone transforms from the mesh. */
	TArray<FTransform> GetBoneTransforms() const;

	/** Get the curve names from the mesh. */
	TArray<FName> GetCurveNames() const;

	/** Get the curve values from the mesh. */
	TArray<float> GetCurveValues() const;

	/** Send the static data to the broadcast subsystem. */
	void BroadcastStaticData() const;

	/** Send the frame data to the broadcast subsystem. */
	void BroadcastFrameData() const;
	
protected:
	//~ Begin UActorComponent interface
	UE_API virtual void OnComponentCreated() override;
	UE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	//~ End UActorComponent interface
	
#if WITH_EDITOR
	//~ Begin UObject interface 
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface 

	/** Object modified callback used to trigger the dirty flag if one of the properties on the tracked source object is modified. */
	void OnObjectModified(UObject* ModifiedObject);
#endif

private:
	/** Cached resolved object, updated when bIsDirty gets triggered. */
	class UActorComponent* CachedResolvedObject = nullptr;
	/** Dirty flag used to trigger updates on the broadcast subsystem. */
	bool bIsDirty;
	/** Cached subject key for this component. */
	FLiveLinkSubjectKey SubjectKey;
};

#undef UE_API