// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkClient.h"
#include "Containers/Set.h"

#define UE_API LIVELINK_API

class ILiveLinkSubject;
struct FPropertyChangedEvent;
struct ILiveLinkProvider;

class ULiveLinkPreset;
class ULiveLinkSourceSettings;
class ULiveLinkSubjectBase;
class FLiveLinkSourceCollection;

// Live Link Log Category
DECLARE_LOG_CATEGORY_EXTERN(LogLiveLink, Log, All);

DECLARE_STATS_GROUP(TEXT("Live Link"), STATGROUP_LiveLink, STATCAT_Advanced);

struct FLiveLinkSubjectTimeSyncData
{
	bool bIsValid = false;
	FFrameTime OldestSampleTime;
	FFrameTime NewestSampleTime;
	FFrameRate SampleFrameRate;
};

/** Describes what triggered the LiveLinkClient Tick. */
enum class ELiveLinkTickType
{
	Default, // Default tick path, used in UE. 
	Scheduled, // Scheduled tick that's triggered by the LiveLinkHub ticker.
	Triggered  // Tick triggered by frame data being received outside of the Game Thread.
};

namespace UE::LiveLink::Private
{
	/** Name token  used to register to all subject updates. */
	static const FName ALL_SUBJECTS_DELEGATE_TOKEN = "__Internal_AllSubjects_Update";
}

class FLiveLinkClient : public ILiveLinkClient
{
public:
	/** Default constructor that setups LiveLink to use the SamplingInput delegate to tick. */
	UE_API FLiveLinkClient();

	//~ Constructor that allow providing a custom delegate for ticking LiveLink.  
	UE_API FLiveLinkClient(FSimpleMulticastDelegate& InTickingDelegate);
	UE_API FLiveLinkClient(FTSSimpleMulticastDelegate& InTickingDelegate);

	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FLiveLinkTickDelegate, ELiveLinkTickType)
	UE_API FLiveLinkClient(FLiveLinkTickDelegate& InTickingDelegate);

	UE_API virtual ~FLiveLinkClient();

	//~ Begin ILiveLinkClient implementation
	UE_API virtual FGuid AddSource(TSharedPtr<ILiveLinkSource> Source) override;
	UE_API virtual FGuid AddVirtualSubjectSource(FName SourceName) override;
	UE_API virtual bool CreateSource(const FLiveLinkSourcePreset& SourcePreset) override;
	UE_API virtual void RemoveSource(TSharedPtr<ILiveLinkSource> Source) override;
	UE_API virtual void RemoveSource(FGuid InEntryGuid) override;
	UE_API virtual bool HasSourceBeenAdded(TSharedPtr<ILiveLinkSource> Source) const override;
	UE_API virtual TArray<FGuid> GetSources(bool bEvenIfPendingKill = false) const override;
	UE_API virtual TArray<FGuid> GetVirtualSources(bool bEvenIfPendingKill = false) const override;
	UE_API virtual FLiveLinkSourcePreset GetSourcePreset(FGuid SourceGuid, UObject* DuplicatedObjectOuter) const override;
	UE_API virtual FText GetSourceType(FGuid EntryGuid) const override;
	UE_API virtual FText GetSourceStatus(FGuid EntryGuid) const override;
	UE_API virtual FText GetSourceToolTip(FGuid EntryGuid) const override;
	UE_API virtual FText GetSourceMachineName(FGuid EntryGuid) const override;
	UE_API virtual bool IsSourceStillValid(FGuid EntryGuid) const override;

	UE_API virtual void PushSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData) override;
	UE_API virtual void PushSubjectFrameData_AnyThread(const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct&& FrameData) override;

	UE_API virtual bool CreateSubject(const FLiveLinkSubjectPreset& SubjectPreset) override;
	UE_API virtual bool AddVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey, TSubclassOf<ULiveLinkVirtualSubject> VirtualSubjectClass) override;
	UE_API virtual void RemoveVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey) override;
	UE_API virtual void RemoveSubject_AnyThread(const FLiveLinkSubjectKey& SubjectKey) override;
	UE_API virtual void PauseSubject_AnyThread(FLiveLinkSubjectName SubjectName) override;
	UE_API virtual void UnpauseSubject_AnyThread(FLiveLinkSubjectName SubjectName) override;
	UE_API virtual void ClearSubjectsFrames_AnyThread(FLiveLinkSubjectName SubjectName) override;
	UE_API virtual void ClearSubjectsFrames_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) override;
	UE_API virtual void ClearAllSubjectsFrames_AnyThread() override;
	UE_API virtual TSubclassOf<ULiveLinkRole> GetSubjectRole_AnyThread(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual TSubclassOf<ULiveLinkRole> GetSubjectRole_AnyThread(FLiveLinkSubjectName SubjectName) const override;
	UE_API virtual TSubclassOf<ULiveLinkRole> GetSubjectTranslatedRole_AnyThread(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual bool DoesSubjectSupportsRole_AnyThread(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> SupportedRole) const override;
	UE_API virtual bool DoesSubjectSupportsRole_AnyThread(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> SupportedRole) const override;

	UE_API virtual FLiveLinkSubjectPreset GetSubjectPreset(const FLiveLinkSubjectKey& SubjectKey, UObject* DuplicatedObjectOuter) const override;
	UE_API virtual TArray<FLiveLinkSubjectKey> GetSubjects(bool bIncludeDisabledSubject, bool bIncludeVirtualSubject) const override;

	UE_API virtual bool IsSubjectValid(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual bool IsSubjectValid(FLiveLinkSubjectName SubjectName) const override;
	UE_API virtual bool IsSubjectEnabled(const FLiveLinkSubjectKey& SubjectKey, bool bForThisFrame) const override;
	UE_API virtual bool IsSubjectEnabled(FLiveLinkSubjectName SubjectName) const override;
	UE_API virtual void SetSubjectEnabled(const FLiveLinkSubjectKey& SubjectKey, bool bEnabled) override;
	UE_API virtual void SetSubjectEnabled(FLiveLinkSubjectName SubjectName, bool bEnabled) override;
	UE_API virtual bool IsSubjectTimeSynchronized(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual bool IsSubjectTimeSynchronized(FLiveLinkSubjectName SubjectName) const override;
	UE_API virtual bool IsVirtualSubject(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual ELiveLinkSubjectState GetSubjectState(FLiveLinkSubjectName InSubjectName) const override;


	UE_API virtual TArray<FLiveLinkSubjectKey> GetSubjectsSupportingRole(TSubclassOf<ULiveLinkRole> SupportedRole, bool bIncludeDisabledSubject, bool bIncludeVirtualSubject) const override;
	UE_API virtual TArray<FLiveLinkTime> GetSubjectFrameTimes(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual TArray<FLiveLinkTime> GetSubjectFrameTimes(FLiveLinkSubjectName SubjectName) const override;
	UE_API virtual FText GetSourceNameOverride(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual FText GetSubjectDisplayName(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual ULiveLinkSourceSettings* GetSourceSettings(const FGuid& SourceGuid) const override;
	UE_API virtual UObject* GetSubjectSettings(const FLiveLinkSubjectKey& SubjectKey) const override;
	UE_API virtual const FLiveLinkStaticDataStruct* GetSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, bool bGetOverrideData=true) const override;


	UE_API virtual bool EvaluateFrameFromSource_AnyThread(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, FLiveLinkSubjectFrameData& OutFrame) override;
	UE_API virtual bool EvaluateFrame_AnyThread(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkSubjectFrameData& OutFrame) override;
	UE_API virtual bool EvaluateFrameAtWorldTime_AnyThread(FLiveLinkSubjectName SubjectName, double WorldTime, TSubclassOf<ULiveLinkRole> DesiredRole, FLiveLinkSubjectFrameData& OutFrame) override;
	UE_API virtual bool EvaluateFrameAtSceneTime_AnyThread(FLiveLinkSubjectName SubjectName, const FQualifiedFrameTime& FrameTime, TSubclassOf<ULiveLinkRole> DesiredRole, FLiveLinkSubjectFrameData& OutFrame) override;
	UE_API virtual void ForceTick() override;
	UE_API virtual bool HasPendingSubjectFrames() override;
	UE_API virtual void ClearOverrideStaticData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) override;

	UE_API virtual FSimpleMulticastDelegate& OnLiveLinkTicked() override;
	UE_API virtual FSimpleMulticastDelegate& OnLiveLinkSourcesChanged() override;
	UE_API virtual FSimpleMulticastDelegate& OnLiveLinkSubjectsChanged() override;
	UE_API virtual FOnLiveLinkSourceChangedDelegate& OnLiveLinkSourceAdded() override;
	UE_API virtual FOnLiveLinkSourceChangedDelegate& OnLiveLinkSourceRemoved() override;
	UE_API virtual FOnLiveLinkSubjectChangedDelegate& OnLiveLinkSubjectAdded() override;
	UE_API virtual FOnLiveLinkSubjectChangedDelegate& OnLiveLinkSubjectRemoved() override;
	UE_API virtual FOnLiveLinkSubjectStateChanged& OnLiveLinkSubjectStateChanged() override;
	UE_API virtual FOnLiveLinkSubjectEnabledDelegate& OnLiveLinkSubjectEnabledChanged() override;
	
#if WITH_EDITOR
	UE_API virtual FOnLiveLinkSubjectEvaluated& OnLiveLinkSubjectEvaluated() override;
#endif

	UE_API virtual void RegisterForFrameDataReceived(const FLiveLinkSubjectKey& InSubjectKey, const FOnLiveLinkSubjectStaticDataReceived::FDelegate& OnStaticDataReceived_AnyThread, const FOnLiveLinkSubjectFrameDataReceived::FDelegate& OnFrameDataReceived_AnyThread, FDelegateHandle& OutStaticDataReceivedHandle, FDelegateHandle& OutFrameDataReceivedHandle) override;
	UE_API virtual void UnregisterForFrameDataReceived(const FLiveLinkSubjectKey& InSubjectKey, FDelegateHandle InStaticDataReceivedHandle, FDelegateHandle InFrameDataReceivedHandle) override;
	UE_API virtual bool RegisterForSubjectFrames(FLiveLinkSubjectName SubjectName, const FOnLiveLinkSubjectStaticDataAdded::FDelegate& OnStaticDataAdded, const FOnLiveLinkSubjectFrameDataAdded::FDelegate& OnFrameDataAdded, FDelegateHandle& OutStaticDataReceivedHandle, FDelegateHandle& OutFrameDataReceivedHandle, TSubclassOf<ULiveLinkRole>& OutSubjectRole, FLiveLinkStaticDataStruct* OutStaticData = nullptr) override;
	UE_API virtual void UnregisterSubjectFramesHandle(FLiveLinkSubjectName InSubjectName, FDelegateHandle InStaticDataReceivedHandle, FDelegateHandle InFrameDataReceivedHandle) override;
	//~ End ILiveLinkClient implementation

public:
	/** Struct that hold the pending static data that will be pushed next tick. */
	struct FPendingSubjectStatic
	{
		FLiveLinkSubjectKey SubjectKey;
		TSubclassOf<ULiveLinkRole> Role;
		FLiveLinkStaticDataStruct StaticData;
		TMap<FName, FString> ExtraMetadata;
	};

	/** Struct that hold the pending frame data that will be pushed next tick. */
	struct FPendingSubjectFrame
	{
		FLiveLinkSubjectKey SubjectKey;
		FLiveLinkFrameDataStruct FrameData;
	};


	/** The tick callback to update the pending work and clear the subject's snapshot*/
	UE_API void Tick(ELiveLinkTickType TickType = ELiveLinkTickType::Default);

	/** Remove all sources from the live link client */
	UE_API void RemoveAllSources();

	/**
	 * Remove all sources and wait for them to be removed. This is a blocking operation.
	 * @param InTimeout The timeout in seconds to wait.
	 * @return Whether all sources were removed successfully.
	 */
	UE_API bool RemoveAllSourcesWithTimeout(float InTimeout);

#if WITH_EDITOR
	/** Call initialize again on an existing virtual subject. Used for when a Blueprint Virtual Subject is compiled */
	UE_API void ReinitializeVirtualSubject(const FLiveLinkSubjectKey& SubjectKey);
#endif

	/** Callback when property changed for one of the source settings */
	UE_API void OnPropertyChanged(FGuid EntryGuid, const FPropertyChangedEvent& PropertyChangedEvent);

	/**
	 * Get all sources that can be displayed in the UI's source list.
	 * @param bIncludeVirtualSources Whether or not to include virtual sources in the returned list, since virtual sources are not displayed in the source list.
	 * @return the list of displayable sources.
	 */
	UE_API TArray<FGuid> GetDisplayableSources(bool bIncludeVirtualSources = false) const;

	UE_API FLiveLinkSubjectTimeSyncData GetTimeSyncData(FLiveLinkSubjectName SubjectName);

	/** Get the rebroadcast name for a given subject. (Defaults to the subject's subject name, but can be overriden. */
	UE_API FName GetRebroadcastName(const FLiveLinkSubjectKey& InSubjectKey) const;

	/** Push subject static data with additional metadata. */
	UE_API void PushPendingSubject_AnyThread(FPendingSubjectStatic&& PendingSubject);

	/** (Internal use only) Cache subject settings for a given subject. Currently no-op outside of LiveLinkHub. */
	virtual void CacheSubjectSettings(const FLiveLinkSubjectKey& SubjectKey, ULiveLinkSubjectSettings* Settings) const {}

	UE_API void SetLastSavedPreset(ULiveLinkPreset* InPreset);
	UE_API ULiveLinkPreset* GetLastSavedPreset() const;

protected:
	/**
	 * Add delegates that will be triggered for all subjects.
	 * @param InOnStaticDataAdded The delegate for when static data is added.
	 * @param InOnFrameDataAdded The delegate for when frame data is added.
	 * @param OutStaticDataAddedHandle [Out] The handle for adding static data.
	 * @param OutFrameDataAddedHandle [Out] The handle for adding frame data.
	 * @param bUseUnmappedData Whether to use raw, unmapped data. If false, then the data received may have a remapper applied.
	 */
	UE_API bool RegisterGlobalSubjectFramesDelegate(const FOnLiveLinkSubjectStaticDataAdded::FDelegate& InOnStaticDataAdded,
		const FOnLiveLinkSubjectFrameDataAdded::FDelegate& InOnFrameDataAdded, FDelegateHandle& OutStaticDataAddedHandle,
		FDelegateHandle& OutFrameDataAddedHandle, bool bUseUnmappedData);
	/**
	 * Remove the delegates that were triggered for all subjects.
	 * @param InStaticDataAddedHandle The static data handle to remove.
	 * @param InFrameDataAddedHandle The frame data handle to remove.
	 * @param bUseUnmappedData Whether this is for unmapped or remapped data.
	 */
	UE_API void UnregisterGlobalSubjectFramesDelegate(FDelegateHandle& InStaticDataAddedHandle, FDelegateHandle& InFrameDataAddedHandle, bool bUseUnmappedData);

private:
	/** Common initialization code for the different constructors. */
	void Initialize();

	/** Remove old sources & subject,  */
	void DoPendingWork(TSet<FLiveLinkSubjectKey>& OutUpdatedSubjects);

	/** Update the added sources */
	void UpdateSources();

	/**
	 * Build subject data so that during the rest of the tick it can be read without
	 * thread locking or mem copying
	 * @return Subjects that were updated by this method.
	 */
	TSet<FLiveLinkSubjectKey> BuildThisTicksSubjectSnapshot(const TSet<FLiveLinkSubjectKey>& ReceivedSubjects);


	/** Rebroadcast live (if using evaluated data) and virtual subjects. */
	void RebroadcastSubjects(const TSet<FLiveLinkSubjectKey>& UpdatedSubjects);

	void CacheValues();

	void PushSubjectStaticData_Internal(FPendingSubjectStatic&& SubjectStaticData);
	void PushSubjectFrameData_Internal(FPendingSubjectFrame&& SubjectFrameData);

	/** Remove all sources. */
	void Shutdown();

	/** Process virtual subject for rebroadcast purpose */
	bool HandleSubjectRebroadcast(ILiveLinkSubject* InSubject, const FLiveLinkFrameDataStruct& InFrameData);
	/**
	 * Rebroadcast evaluated livelink data.
	 * @note Calling this method will not apply preprocessors and translators since it's meant to be used to transmit evaluated data. 
	 */
	bool HandleSubjectRebroadcast(ILiveLinkSubject* InSubject, FLiveLinkSubjectFrameData&& SubjectFrameData);

	/** Called when a subject is removed. Used to remove rebroadcasted subjects */
	void OnSubjectRemovedCallback(FLiveLinkSubjectKey InSubjectKey);

	/** Removes a subject from the rebroadcast provider and resets it if there are no more subjects */
	void RemoveRebroadcastedSubject(FLiveLinkSubjectKey InSubjectKey);

	/** Iterate over every live subject. */
	void ForEachLiveSubject(TFunctionRef<void(class FLiveLinkSubject*, struct FLiveLinkCollectionSourceItem&, struct FLiveLinkCollectionSubjectItem&)> VisitorFunc);

	/** Iterate over every virtual subject. */
	void ForEachVirtualSubject(TFunctionRef<void(class ULiveLinkVirtualSubject*, struct FLiveLinkCollectionSourceItem&, struct FLiveLinkCollectionSubjectItem&)> VisitorFunc);

protected:
	/** Broadcast out to the SubjectFrameAddedHandles a frame data update. */
	UE_API void BroadcastFrameDataUpdate(const FLiveLinkSubjectKey& InSubjectKey, const FLiveLinkFrameDataStruct& InFrameData);

	/** Method that can be overriden in child classes to provide their own rebroadcast provider. */
	UE_API virtual TSharedPtr<ILiveLinkProvider> GetRebroadcastLiveLinkProvider() const;

	/** The current collection used. */
	TUniquePtr<FLiveLinkSourceCollection> Collection;

	/** LiveLink Provider for rebroadcasting */
	TSharedPtr<ILiveLinkProvider> RebroadcastLiveLinkProvider;

	/** Lock to protect access on SubjectFrameToPush and SubjectStaticToPush. */
	mutable FCriticalSection PendingFramesCriticalSection;

	/** FrameId counters used when sources don't set it themselves. */
	TMap<FLiveLinkSubjectKey, FLiveLinkFrameIdentifier> SubjectFrameCounters;

	/** Lock to protect access to SubjectFrameCounters. */
	mutable FCriticalSection SubjectFrameCountersLock;

private:
	/** Pending static info to add to a subject. */
	TArray<FPendingSubjectStatic> SubjectStaticToPush;

	/** Pending frame info to add to a subject. */
	TArray<FPendingSubjectFrame> SubjectFrameToPush;

	/** Cached subject settings from presets, used to restore preprocessors when subjects are recreated (e.g. after source reconnection). */
	TMap<FLiveLinkSubjectName, TStrongObjectPtr<ULiveLinkSubjectSettings>> CachedPresetSubjectSettings;

	/** Key funcs for looking up a set of cached keys by its layout element */
	TMap<FLiveLinkSubjectName, FLiveLinkSubjectKey> EnabledSubjects;

	struct FSubjectFramesAddedHandles
	{
		FOnLiveLinkSubjectStaticDataAdded OnStaticDataAdded;
		FOnLiveLinkSubjectFrameDataAdded OnFrameDataAdded;
		/** Original data that hasn't been remapped. */
		FOnLiveLinkSubjectStaticDataAdded OnUnmappedStaticDataAdded;
		/** Original data that hasn't been remapped. */
		FOnLiveLinkSubjectFrameDataAdded OnUnmappedFrameDataAdded;
	};

	/** Map of delegates to notify interested parties when the client receives a static or data frame for each subject */
	TMap<FLiveLinkSubjectName, FSubjectFramesAddedHandles> SubjectFrameAddedHandles;

	struct FSubjectFramesReceivedHandles
	{
		FOnLiveLinkSubjectStaticDataReceived OnStaticDataReceived;
		FOnLiveLinkSubjectFrameDataReceived OnFrameDataReceived;
	};

	/** Delegate when LiveLinkClient received a subject static or frame data. */
	TMap<FLiveLinkSubjectKey, FSubjectFramesReceivedHandles> SubjectFrameReceivedHandles;

	/** Lock to to access SubjectFrameReceivedHandles */
	mutable FCriticalSection SubjectFrameReceivedHandleseCriticalSection;

	/** Delegate when LiveLinkClient has ticked. */
	FSimpleMulticastDelegate OnLiveLinkTickedDelegate;

	FString RebroadcastLiveLinkProviderName;
	TSet<FLiveLinkSubjectKey> RebroadcastedSubjects;


	/** Whether to Preprocess frames before rebroadcasting them. */
	bool bPreProcessRebroadcastFrames = false;

	/** Whether to translate frames before rebroadcasting them. */
	bool bTranslateRebroadcastFrames = false;

	/** Whether or not parent subject support is enabled. Parent subjects allow resampling data to a different subject's rate before rebroadcasting it. */
	bool bEnableParentSubjects = false;

	/** Tracks the last preset saved or loaded by the user in this session. */
	TWeakObjectPtr<ULiveLinkPreset> LastSavedPreset;

#if WITH_EDITOR
	/** Delegate when a subject is evaluated. */
	FOnLiveLinkSubjectEvaluated OnLiveLinkSubjectEvaluatedDelegate;

	/** Cached value of the engine timecode and frame rate*/
	TOptional<FQualifiedFrameTime> CachedEngineFrameTime;
#endif
};

#undef UE_API
