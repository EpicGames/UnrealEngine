// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API LIVELINK_API

class ILiveLinkClient;

/** References the live link client */
struct FLiveLinkClientReference
{
public:
	UE_API ILiveLinkClient* GetClient() const;
};

#undef UE_API
