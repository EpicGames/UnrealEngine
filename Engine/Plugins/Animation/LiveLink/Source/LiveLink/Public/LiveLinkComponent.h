// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "LiveLinkComponent.generated.h"

#define UE_API LIVELINK_API

class ILiveLinkClient;
struct FSubjectFrameHandle;
struct FTimecode;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FLiveLinkTickSignature, float, DeltaTime);

// An actor component to enable accessing LiveLink data in Blueprints. 
// Data can be accessed in Editor through the "OnLiveLinkUpdated" event.
// Any Skeletal Mesh Components on the parent will be set to animate in editor causing their AnimBPs to run.
UCLASS(MinimalAPI,  ClassGroup=(LiveLink), meta=(BlueprintSpawnableComponent), meta = (DisplayName = "LiveLink Skeletal Animation"))
class ULiveLinkComponent : public UActorComponent
{
	GENERATED_BODY()

public:	
	// Sets default values for this component's properties
	UE_API ULiveLinkComponent();

protected:
	UE_API virtual void OnRegister() override;

public:	
	// Called every frame
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	// This Event is triggered any time new LiveLink data is available, including in the editor
	UPROPERTY(BlueprintAssignable, Category = "LiveLink")
	FLiveLinkTickSignature OnLiveLinkUpdated;
private:
	bool HasLiveLinkClient();

	// Record whether we have been recently registered
	bool bIsDirty;

	ILiveLinkClient* LiveLinkClient;
};

#undef UE_API
