// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineCustomTimeStep.h"
#include "GenlockedCustomTimeStep.h"
#include "HAL/Event.h"
#include "LiveLinkTypes.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"

#include <atomic>

#include "LiveLinkCustomTimeStep.generated.h"

class UEngine;
class UObject;
class ILiveLinkClient;

/**
 * Control the Engine TimeStep via a live link source
 * 
 * Philosophy:
 * 
 *   * Quantized time steps based on live link expected data rate.
 *   * Made for Live Link sources can receive data asynchronously, and therefore trigger the waiting game thread.
 * 
 *   * FApp::GetDeltaTime 
 *       - Forced to a multiple of the desired FrameTime.
 *       - This multiple will depend on Frame Id increment and user settings.
 * 
 *   * FApp::GetCurrentTime
 *       - Incremented in multiples of the desired FrameTime.
 * 
 */
UCLASS(Blueprintable, editinlinenew, MinimalAPI)
class ULiveLinkCustomTimeStep : public UGenlockedCustomTimeStep
{
	GENERATED_BODY()

public:
	//~ UFixedFrameRateCustomTimeStep interface
	LIVELINK_API virtual bool Initialize(UEngine* InEngine) override;
	LIVELINK_API virtual void Shutdown(UEngine* InEngine) override;
	LIVELINK_API virtual bool UpdateTimeStep(UEngine* InEngine) override;
	LIVELINK_API virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override;
	LIVELINK_API virtual FFrameRate GetFixedFrameRate() const override;
	LIVELINK_API virtual FFrameRate GetSyncRate() const override;
	LIVELINK_API virtual FString GetDisplayName() const override;

	//~ UGenlockedCustomTimeStep interface
	LIVELINK_API virtual uint32 GetLastSyncCountDelta() const override;
	LIVELINK_API virtual bool IsLastSyncDataValid() const override;
	LIVELINK_API virtual bool WaitForSync() override;
	LIVELINK_API virtual uint32 GetExpectedSyncCountDelta() const override;

	/** Returns the SceneTime captured at the most recent genlock trigger point. */
	LIVELINK_API FQualifiedFrameTime GetGenlockTriggerSceneTime() const;

	//~ Begin UObject Interface
	LIVELINK_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	LIVELINK_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject Interface

private:

	/** Initializes the Live Link client (callbacks, etc) */
	void InitLiveLinkClient();

	/** Uninitializes the Live Link client (callbacks, etc) */
	void UninitLiveLinkClient();

	/** Registers the selected subject for data callbacks */
	void RegisterLiveLinkSubject();

	/** Unregisters the data callbacks related to the live link subject */
	void UnregisterLiveLinkSubject();

	/** Called when a Live Link modular features is registered */
	void OnLiveLinkClientRegistered(const FName& Type, class IModularFeature* ModularFeature);

	/** Called when a Live Link modular feature is unregistered */
	void OnLiveLinkClientUnregistered(const FName& Type, class IModularFeature* ModularFeature);

	/** Called when a new live link subject is added, so that we can check if is the one we're listening for */
	void OnLiveLinkSubjectAdded(FLiveLinkSubjectKey SubjectKey);

	/** Called when a live link subject is removed. It may be the one we were listening to */
	void OnLiveLinkSubjectRemoved(FLiveLinkSubjectKey SubjectKey);

	/** Called when our live link subject has received data. This is equivalent to a genlock sync signal */
	void OnLiveLinkFrameDataReceived_AnyThread(const FLiveLinkFrameDataStruct& FrameData);

public:

	/** Expected Live Link data rate. If real rate differs, then delta times will contract/expand with respect to real time */
	UPROPERTY(EditAnywhere, Category = Timing)
	FFrameRate LiveLinkDataRate = FFrameRate(60, 1);

	/** The specific subject that we listen to. */
	UPROPERTY(EditAnywhere, Category = Timing)
	FLiveLinkSubjectKey SubjectKey;

	/** 
	 * Lockstep mode means that we only allow FrameRateDivider data frames of the selected subject per engine loop. 
	 * The idea here is to process all packets and avoid Live Link evaluation to skip frames when the engine hitches, 
	 * and the live link transport will serve as implicit FIFO buffer. However if the Engine cannot keep up with the data rate,
	 * a large delay will be introduced and the transport buffer will eventually start dropping data.
	 */
	UPROPERTY(EditAnywhere, Category = Timing)
	bool bLockStepMode = true;

	/** Allows genlock to period to be a multiple of the live link data period. For example a value of 2 will run at half the live link data rate */
	UPROPERTY(EditAnywhere, Category = Timing, meta = (ClampMin = 1, ClampMax = 256, UIMin = 1, UIMax = 256))
	uint32 FrameRateDivider = 1;

	/** Determines how long it should wait for live link data before deciding that it is not in synchronized state anymore */
	UPROPERTY(EditAnywhere, Category = Advanced, AdvancedDisplay)
	double TimeoutInSeconds = 1.0;

private:

	/** This is how we access live link objects. It is obtained using modular features. */
	ILiveLinkClient* LiveLinkClient;

	/** Identifies the Live Link subject that we are genlocking to. */
	FLiveLinkSubjectKey RegisteredSubjectKey;

	/** Gets called when the subject gets new data. This is the basis for the genlock. */
	FDelegateHandle RegisterForFrameDataReceivedHandle;

	/** Keeps track of the current state. For the most part, if we are getting data from the subject, it is synchronized. */
	std::atomic<ECustomTimeStepSynchronizationState> State = ECustomTimeStepSynchronizationState::Closed;

	/** Helps trigger the wait for sync function from an asynchronous thread */
	FEventRef EventLiveLink{ EEventMode::AutoReset };

	/** Used in lockstep mode, where we only allow one live link subject data frame per engine tick */
	FEventRef EventWaitForSync{ EEventMode::AutoReset };

	/** Keeps track of the last frame id received for a given subject, and used to calcualte delta time based on source data rate. */
	std::atomic<FLiveLinkFrameIdentifier> LastFrameId = 0;

	/** Incremented based on frame id when live link data is received. Reset when internally read. */
	std::atomic<uint32> SyncCount = 0;

	/** Cache of the last sync count delta, used for functions in the game thread that need it. */
	uint32 LastSyncCountDelta = 0;

	/** Shadow of bLockStepMode property */
	std::atomic<bool> bLockStepModeAnyThread = true;

	/** Shadow of FrameRateDivider property */
	std::atomic<uint32> FrameRateDividerAnyThread = 1;

	/** Shadow of TimeoutInSeconds property */
	std::atomic<double> TimeoutInSecondsAnyThread = 1.0;

	/** SceneTime captured at the most recent genlock trigger point. */
	FQualifiedFrameTime GenlockTriggerSceneTime;

	/** Protects GenlockTriggerSceneTime across threads. */
	mutable FCriticalSection GenlockSceneTimeLock;

	/** Frame number of the last captured genlock trigger time, used to ensure deterministic subframe behavior */
	std::atomic<int32> LastCapturedTimecodeFrameNumber = INDEX_NONE;

	/** FrameId of the reference frame, used for phase alignment */
	std::atomic<int32> ReferenceFrameId = INDEX_NONE;

	/** Cached frames per timecode period (SourceRate / TimecodeRate), calculated at initialization */
	std::atomic<int32> FramesPerTimecodeAnyThread = 0;

	/** Cached flag indicating whether the monitored subject is from a sequencer source */
	std::atomic<bool> bIsSequencerSourceAnyThread = false;

	/** When true, OnLiveLinkFrameDataReceived_AnyThread skips the lockstep wait.
	 *  This is set before unregistering to avoid blocking the game thread on the
	 *  critical section held during BroadcastFrameDataUpdate's lockstep wait. */
	std::atomic<bool> bSkipLockStepWait = false;
};
