// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "LiveLinkTypes.h"
#include "LiveLinkDrivenComponent.generated.h"

/** A component that applies data from Live Link to the owning actor */
UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_LiveLinkDrivenComponent : public UActorComponent
{
public:
	GENERATED_BODY()

	// The name of the live link subject to take data from
	UPROPERTY(EditAnywhere, Category = "Live Link", meta = (ShowOnlyInnerProperties))
	FLiveLinkSubjectName SubjectName;

	// The name of the bone to drive the actors transform with (if None then we will take the first bone)
	UPROPERTY(EditAnywhere, Category = "Live Link")
	FName ActorTransformBone;

	// Should the actors transform be driven by live link
	UPROPERTY(EditAnywhere, Category = "Live Link")
	bool bModifyActorTransform;

	// Should the transform from live link be treated as relative or world space
	UPROPERTY(EditAnywhere, Category = "Live Link")
	bool bSetRelativeLocation;
};
