// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSubjectSettings.h"

#include "LiveLinkHubSubjectSettings.generated.h"

#define UE_API LIVELINK_API

class FLiveLinkHubClient;
class ULiveLinkRole;

/** Settings object for a livelinkhub subject. */
UCLASS(MinimalAPI)
class ULiveLinkHubSubjectSettings : public ULiveLinkSubjectSettings
{
	GENERATED_BODY()
public:

	//~ Begin ULiveLinkSubjectSettings interface
	UE_API virtual void Initialize(FLiveLinkSubjectKey InSubjectKey) override;

	virtual FName GetRebroadcastName() const override
	{
		return *OutboundName;
	}
	//~ End ULiveLinkSubjectSettings interface

#if WITH_EDITOR
	//~ Begin UObject interface
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPropertyModified, const FName& /*PropertyName*/);
	FOnPropertyModified OnPropertyModified() { return OnPropertyModifiedDelegate; }
#endif

	static FName GetOutboundNamePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, OutboundName);
	}

	static FName GetSubjectNamePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, SubjectName);
	}

private:
	/** Name of this subject. */
	UPROPERTY(VisibleAnywhere, Category = "LiveLink")
	FString SubjectName;

	/** Name override that will be transmitted to clients instead of the subject name. */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	FString OutboundName;

	/** Source that contains the subject. */
	UPROPERTY(VisibleAnywhere, Category = "LiveLink")
	FString Source;

	/** Proxy property used edit the translators. */
	UPROPERTY(EditAnywhere, Instanced, Category = "LiveLink", DisplayName = "Translator")
	TObjectPtr<ULiveLinkFrameTranslator> TranslatorsProxy;

private:
	/** Previous outbound name to be used for noticing clients to remove this entry from their subject list. */
	FName PreviousOutboundName;

#if WITH_EDITOR
	/** Triggered when a property is modified. */
	FOnPropertyModified OnPropertyModifiedDelegate;
#endif
};

#undef UE_API