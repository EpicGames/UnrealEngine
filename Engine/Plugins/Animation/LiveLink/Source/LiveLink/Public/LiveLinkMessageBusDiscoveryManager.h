// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"

#include "LiveLinkMessageBusFinder.h"

#define UE_API LIVELINK_API

class IMessageContext;

class FLiveLinkMessageBusSource;

/** A class to asynchronously discover message bus sources. */
class FLiveLinkMessageBusDiscoveryManager : FRunnable
{
public:
	UE_API FLiveLinkMessageBusDiscoveryManager();
	UE_API ~FLiveLinkMessageBusDiscoveryManager();

	//~ Begin FRunnable interface

	UE_API virtual uint32 Run() override;

	UE_API virtual void Stop() override;

	//~ End FRunnable interface

	UE_API void AddDiscoveryMessageRequest();
	UE_API void RemoveDiscoveryMessageRequest();
	UE_API TArray<FProviderPollResultPtr> GetDiscoveryResults() const;

	UE_API bool IsRunning() const;

	// Get the message bus address for the discovery manager's endpoint.
	UE_API FMessageAddress GetEndpointAddress() const;

private:
	void HandlePongMessage(const FLiveLinkPongMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

private:
	// Counter of item that request discovery message
	TAtomic<int32> PingRequestCounter;

	// Last ping Request id
	FGuid LastPingRequest;

	// Time of the last ping request
	double LastPingRequestTime;

	// Ping request timeout
	FTimespan PingRequestFrequency;

	// Result from the last ping request
	TArray<FProviderPollResultPtr> LastProviderPoolResults;

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	// Thread safe bool for stopping the thread
	FThreadSafeBool bRunning;

	// Thread the heartbeats are sent on
	FRunnableThread* Thread;

	// Event used to poll the discovery results.
	class FEvent* PollEvent = nullptr;

	// Critical section for accessing the Source Set
	mutable FCriticalSection SourcesCriticalSection;
};

#undef UE_API
