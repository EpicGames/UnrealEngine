// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "LatentActions.h"
#include "Engine/LatentActionManager.h"
#include "LiveLinkMessageBusFinder.generated.h"

#define UE_API LIVELINK_API

class FMessageEndpoint;
struct FLiveLinkSourceHandle;

struct FLiveLinkPongMessage;


namespace LiveLinkMessageBusHelper
{
	LIVELINK_API double CalculateProviderMachineOffset(double SourceMachinePlatformSeconds, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
}

USTRUCT(BlueprintType)
struct FProviderPollResult
{
public:
	GENERATED_USTRUCT_BODY()
	
	FProviderPollResult() = default;

	FProviderPollResult(const FMessageAddress& InAddress, const FString& InName, const FString& InMachineName, double InMachineOffset, bool bInIsValidProvider)
		: Address(InAddress)
		, Name(InName)
		, MachineName(InMachineName)
		, MachineTimeOffset(InMachineOffset)
		, bIsValidProvider(bInIsValidProvider)
		, Annotations()
	{}

	FProviderPollResult(const FMessageAddress& InAddress, const FString& InName, const FString& InMachineName, double InMachineOffset, bool bInIsValidProvider, TMap<FName, FString> InAnnotations)
		: Address(InAddress)
		, Name(InName)
		, MachineName(InMachineName)
		, MachineTimeOffset(InMachineOffset)
		, bIsValidProvider(bInIsValidProvider)
		, Annotations(MoveTemp(InAnnotations))
	{}

	// Get  the annotations that were present on the message.
	const TMap<FName, FString>& GetAnnotations() const { return Annotations; }

	// Messagebus address of the provider
	FMessageAddress Address;

	// The name of the provider
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="LiveLink", meta=(DisplayName = "Provider Name"))
	FString Name;

	// The name of the machine the provider is running on
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="LiveLink")
	FString MachineName;

	// Offset between sender's engine time and receiver's engine time
	UPROPERTY()
	double MachineTimeOffset = 0.0;

	// Whether the provider is valid (compatible with the current version of LiveLink)
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LiveLink")
	bool bIsValidProvider = true;

	// Version of the discovery protocol.
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LiveLink")
	int32 DiscoveryProtocolVersion = 1;

	// Annotations present on the message
	TMap<FName, FString> Annotations;
};

typedef TSharedPtr<FProviderPollResult, ESPMode::ThreadSafe> FProviderPollResultPtr;

// Asset for finding available Message Bus Sources.
UCLASS(MinimalAPI, BlueprintType)
class ULiveLinkMessageBusFinder
	: public UObject
{
	GENERATED_BODY()
public:

	UE_API ULiveLinkMessageBusFinder();

   /*
	* Broadcasts a message to the network and returns a list of all providers who replied within a set amount of time.
	*
	* @param AvailableProviders Will contain the collection of found Message Bus Providers.
	* @param Duration The amount of time to wait for replies in seconds
	*/
	UFUNCTION(BlueprintCallable, Category = "LiveLink", meta=(Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject", Duration = "0.2"))
	UE_API void GetAvailableProviders(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, float Duration, TArray<FProviderPollResult>& AvailableProviders);

   /*
	* Connects to a given Message Bus Provider and returns a handle to the created LiveLink Source
	*
	* @param Provider The provider to connect to.
	* @param SourceHandle A handle to the created LiveLink Source, lets you query information about the created source and request a shutdown
	*/
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API void ConnectToProvider(UPARAM(ref) FProviderPollResult& Provider, FLiveLinkSourceHandle& SourceHandle);

   /*
	* Constructs a new Message Bus Finder which enables you to detect available Message Bus Providers on the network
	*
	* @return The newly constructed Message Bus Finder
	*/
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	static UE_API ULiveLinkMessageBusFinder* ConstructMessageBusFinder();

	// Broadcast a ping message to the network and listen for responses
	UE_API void PollNetwork();

	// Populates AvailableProviders with the Providers who have responded to the latest poll.
	UE_API void GetPollResults(TArray<FProviderPollResult>& AvailableProviders);

private:
	// Runs when a Provider responds to the ping from PollNetwork()
	void HandlePongMessage(const FLiveLinkPongMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	TArray<FProviderPollResult> PollData;
	FGuid CurrentPollRequest;

	FCriticalSection PollDataCriticalSection;
};


struct FLiveLinkMessageBusFinderAction : public FPendingLatentAction
{
public:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	TWeakObjectPtr<ULiveLinkMessageBusFinder> MessageBusFinderWeakPtr;
	TArray<FProviderPollResult>& OutAvailableProviders;
	float RemainingTime;

	FLiveLinkMessageBusFinderAction(const FLatentActionInfo& InLatentInfo, ULiveLinkMessageBusFinder* InMessageBusFinder, float Duration, TArray<FProviderPollResult>& InAvailableProviders)
		: FPendingLatentAction()
		, ExecutionFunction(InLatentInfo.ExecutionFunction)
		, OutputLink(InLatentInfo.Linkage)
		, CallbackTarget(InLatentInfo.CallbackTarget)
		, MessageBusFinderWeakPtr(InMessageBusFinder)
		, OutAvailableProviders(InAvailableProviders)
		, RemainingTime(Duration)
	{}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		RemainingTime -= Response.ElapsedTime();
		if (RemainingTime <= 0)
		{
			if (ULiveLinkMessageBusFinder* MessageBusFinder = MessageBusFinderWeakPtr.Get())
			{
				MessageBusFinder->GetPollResults(OutAvailableProviders);
			}
			Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
		}
	}

#if WITH_EDITOR
	virtual FString GetDescription() const override
	{
		return FString::Printf(TEXT("Searching for LiveLink Message Bus providers."));
	}
#endif
};

#undef UE_API
