// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"

#include "HAL/ThreadSafeBool.h"
#include "LiveLinkTypes.h"
#include "MessageEndpoint.h"

#define UE_API LIVELINK_API

class ULiveLinkRole;
class ULiveLinkSourceSettings;
struct FMessageEndpointBuilder;

class ILiveLinkClient;
struct FLiveLinkPongMessage;
struct FLiveLinkSubjectDataMessage;
struct FLiveLinkSubjectFrameMessage;
struct FLiveLinkHeartbeatMessage;
struct FLiveLinkClearSubject;

class FLiveLinkMessageBusSource : public ILiveLinkSource
{
public:
	/** Text description for a valid source. */
	static UE_API FText ValidSourceStatus();
	/** Text description for an invalid source. */
	static UE_API FText InvalidSourceStatus();
	/** Text description for a source that has timed out. */
	static UE_API FText TimeoutSourceStatus();

	UE_API FLiveLinkMessageBusSource(const FText& InSourceType, const FText& InSourceMachineName, const FMessageAddress& InConnectionAddress, double InMachineTimeOffset);

	//~ Begin ILiveLinkSource interface
	UE_API virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;
	UE_API virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	UE_API virtual void Update() override;

	UE_API virtual bool IsSourceStillValid() const override;

	UE_API virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override { return SourceType; }
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }

	UE_API virtual FText GetSourceStatus() const override;
	UE_API virtual FText GetSourceToolTip() const override;
	UE_API virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const override;
	//~ End ILiveLinkSource interface

	// Get this source's message bus address.
	UE_API FMessageAddress GetAddress() const;

protected:
	// Returns the source name to uniquely identify it among the FLiveLinkMessageBusSource classes
	UE_API virtual const FName& GetSourceName() const;
	// This lets child classes the opportunity to add custom message handlers to the endpoint builder
	UE_API virtual void InitializeMessageEndpoint(FMessageEndpointBuilder& EndpointBuilder);
	// This lets child classes get a shared ptr of the message endpoint after it's done initializing.
	virtual void PostInitializeMessageEndpoint(const TSharedPtr<FMessageEndpoint>& Endpoint) {}
	// Add MessageBus annotations.
	UE_API void AddAnnotations(TMap<FName, FString>& InOutAnnotations) const;
	// Returns whether this endpoint is currently connected.
	bool IsMessageEndpointConnected() const { return ConnectionAddress.IsValid() && MessageEndpoint.IsValid() && MessageEndpoint->IsConnected(); }

	// Initialize the static data and send it to the clients
	UE_API virtual void InitializeAndPushStaticData_AnyThread(FName SubjectName,
													   TSubclassOf<ULiveLinkRole> SubjectRole,
													   const FLiveLinkSubjectKey& SubjectKey,
													   const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
													   UScriptStruct* MessageTypeInfo);
	// Initialize the frame data and send it to the clients
	UE_API virtual void InitializeAndPushFrameData_AnyThread(FName SubjectName,
													  const FLiveLinkSubjectKey& SubjectKey,
													  const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
													  UScriptStruct* MessageTypeInfo);

	// Allows derived classes to provide their own timeout duration before a source is removed because the heartbeat timeout was hit
	UE_API virtual double GetDeadSourceTimeout() const;

	// Send the static data to the clients
	UE_API void PushClientSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData);
	// Send the frame data to the clients
	UE_API void PushClientSubjectFrameData_AnyThread(const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct&& FrameData);

	// Send connect message to the provider and start the heartbeat emitter
	UE_API virtual void SendConnectMessage();

	// Send a message through the endpoint
	template<typename MessageType>
	void SendMessage(MessageType* Message, const TMap<FName, FString>& Annotations = {})
	{
		if (!Message || !IsMessageEndpointConnected())
		{
			return;
		}

		MessageEndpoint->Send(Message, EMessageFlags::None, Annotations, nullptr, { ConnectionAddress }, FTimespan::Zero(), FDateTime::MaxValue());
	}

	// Start the heartbeat emitter for this connection
	UE_API void StartHeartbeatEmitter();

protected:
	// Message bus endpoint responsible for communication with the livelink provider
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	// Connection address of the livelink provider 
	FMessageAddress ConnectionAddress;

	// Current Validity of Source
	std::atomic<bool> bIsValid;
	// Whether we're in the process of shutting down this source.
	std::atomic<bool> bIsShuttingDown = false;

	ILiveLinkClient* Client;

	// Our identifier in LiveLink
	FGuid SourceGuid;

	// Cached settings values (atomics for thread-safe access from _AnyThread methods)
	std::atomic<bool> bCachedEnableFrameResequencing = true;
	std::atomic<int32> CachedResequencerMaxBufferSize = 32;
	std::atomic<float> CachedResequencerMaxWaitTimeSeconds = 0.1f;

private:
	void CreateAndInitializeMessageEndpoint();

	//~ Message bus message handlers
	void HandleHeartbeat(const FLiveLinkHeartbeatMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleClearSubject(const FLiveLinkClearSubject& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void InternalHandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	//~ End Message bus message handlers

	// Threadsafe update of the last active time
	inline void UpdateConnectionLastActive();

	// Start discovering sources
	void StartDiscovering();

	// Stop discovering sources
	void StopDiscovering();

	// Initialize (can be used after a shutdown request)
	void Initialize();

	// Resequencer for maintaining frame order per subject
	void ResequenceAndPushFrame(const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct&& FrameData);
	
	// Cache Message Bus settings
	void CacheMessageBusSourceSettings();

private:

	/** Per-subject frame resequencer to deliver live link frames in FrameId order */
	struct FFrameResequencer
	{
	public:
		struct FBufferedFrame
		{
			FLiveLinkFrameDataStruct Frame;
			double BufferedAtSeconds = 0.0;
		};

		/** Buffered frames waiting for delivery (only used when frames arrive out of order) */
		TArray<FBufferedFrame> BufferedFrames;

		/** Last delivered FrameId - used to detect and deliver the next expected frame immediately */
		TOptional<FLiveLinkFrameIdentifier> LastDeliveredFrameId;

		/** Timestamp when oldest frame was buffered (for timeout detection) */
		double OldestFrameBufferTime = 0.0;

		/** Maximum number of frames to buffer */
		int32 MaxBufferSize = 32;

		/** Maximum time in seconds to wait for missing frames */
		double MaxWaitTimeSeconds = 0.1;

		/** Stats: Number of frames that arrived out of order and were resequenced */
		int32 ReorderedFrameCount = 0;

		/** Stats: Number of frames dropped because they arrived too late to be delivered in order */
		int32 DroppedLateFrameCount = 0;

		/** Stats: Maximum number of frames held in the buffer simultaneously */
		int32 PeakBufferDepth = 0;

	public:

		/** Add a frame to the buffer */
		void AddFrame(FLiveLinkFrameDataStruct&& Frame, double CurrentTime);

		/** Try to pop the next ordered frame from buffer */
		bool TryPopNextFrame(FLiveLinkFrameDataStruct& OutFrame, double CurrentTime);

		/** Flush all buffered frames */
		void Flush(TArray<FLiveLinkFrameDataStruct>& OutFrames);

		/** Check if a frame should be dropped as too late */
		bool IsFrameTooLate(const FLiveLinkFrameDataStruct& Frame) const;

		/** Update last delivered tracking after delivering a frame */
		void UpdateLastDelivered(const FLiveLinkFrameDataStruct& Frame);
	};

	// Per-subject map of frame resequencers for maintaining ordered frame delivery
	TMap<FName, FFrameResequencer> SubjectResequencers;

	// Critical section for thread-safe access to SubjectResequencers
	FCriticalSection ResequencerCriticalSection;

	// List of the roles available when the bus was opened
	TArray<TWeakObjectPtr<ULiveLinkRole>> RoleInstances;

	FText SourceType;
	FText SourceMachineName;

	/** Weak pointer to the source settings, used to cache source settings values when they're modified. */
	TWeakObjectPtr<class ULiveLinkMessageBusSourceSettings> SourceSettings;

	// Time we last received anything 
	double ConnectionLastActive;

	// Critical section to allow for threadsafe updating of the connection time
	FCriticalSection ConnectionLastActiveSection;

	// Offset between sender's machine engine time and receiver's machine engine time
	double MachineTimeOffset;

	//  Cached throughput (in kb/s) of data going through this messagebus source.
	float CachedThroughput = 0.f;

	// Timestamp of the last update to the cached throughput.
	float LastThroughputUpdate = 0.f;

	//  Number of bytes received since LastThroughputUpdate.
	float AccumulatedBytes = 0.f;

	// Actively discovering the source
	bool bDiscovering;

	// Flag to keep track of initialization state
	bool bInitialized;
};

#undef UE_API
