// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkMessageBusFinder.h"		// For FProviderPollResultPtr typedef
#include "LiveLinkSourceFactory.h"
#include "LiveLinkMessageBusSourceFactory.generated.h"

#define UE_API LIVELINK_API

struct FMessageAddress;


UCLASS(MinimalAPI)
class ULiveLinkMessageBusSourceFactory : public ULiveLinkSourceFactory
{
public:
	GENERATED_BODY()

	UE_API virtual FText GetSourceDisplayName() const override;
	UE_API virtual FText GetSourceTooltip() const override;

	virtual EMenuType GetMenuType() const override { return EMenuType::SubPanel; }
	UE_API virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const override;
	UE_API virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const override;

	static UE_API FString CreateConnectionString(const struct FProviderPollResult& Result);

protected:
	// This allows the creation of a message bus source derived from FLiveLinkMessageBusSource
	UE_API virtual TSharedPtr<class FLiveLinkMessageBusSource> MakeSource(const FText& Name,
																   const FText& MachineName,
																   const FMessageAddress& Address,
																   double TimeOffset) const;

	UE_API void OnSourceSelected(FProviderPollResultPtr SelectedSource, FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const;
};

#undef UE_API
