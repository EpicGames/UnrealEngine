// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceSettings.h"

#include "LiveLinkMessageBusSourceSettings.generated.h"

#define UE_API LIVELINK_API




/**
 * Settings for LiveLinkMessageBusSource.
 * Used to apply default Evaluation mode from project settings when constructed
 */
UCLASS(config=Game, MinimalAPI)
class ULiveLinkMessageBusSourceSettings : public ULiveLinkSourceSettings
{
public:
	GENERATED_BODY()

public:
	UE_API ULiveLinkMessageBusSourceSettings();

	/** Enable frame resequencing to counteract UDP Messaging delivering them out of order. */
	UPROPERTY(config, EditAnywhere, Category = "Resequencing")
	bool bEnableFrameResequencing = true;

	/** Maximum number of frames to buffer per subject. If exceeded, oldest frames are pushed even if out of order. */
	UPROPERTY(config, EditAnywhere, Category = "Resequencing", meta = (EditCondition = "bEnableFrameResequencing", ClampMin = "4", ClampMax = "128"))
	int32 ResequencerMaxBufferSize = 32;

	/** Maximum time in seconds to hold frames waiting for missing frames. */
	UPROPERTY(config, EditAnywhere, Category = "Resequencing", meta = (EditCondition = "bEnableFrameResequencing", ClampMin = "0.01", ClampMax = "10.0"))
	float ResequencerMaxWaitTimeSeconds = 0.1f;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		if (PropertyChangedEvent.Property &&
				(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkMessageBusSourceSettings, bEnableFrameResequencing)
				|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkMessageBusSourceSettings, ResequencerMaxBufferSize)
				|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkMessageBusSourceSettings, ResequencerMaxWaitTimeSeconds)))
		{
			OnPropertyChangedDelegate.Broadcast();
		}
	}

	/** Delegate triggered whenever a message bus source settings property is modified. */
	FTSSimpleMulticastDelegate OnPropertyChangedDelegate;
#endif

};

#undef UE_API
