// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LiveLinkPreset.generated.h"

#define UE_API LIVELINK_API

struct FLatentActionInfo;
struct FLiveLinkSourcePreset;
struct FLiveLinkSubjectPreset;

UCLASS(MinimalAPI, BlueprintType)
class ULiveLinkPreset : public UObject
{
	GENERATED_BODY()

	UE_API virtual ~ULiveLinkPreset();

private:
	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSourcePresets")
	TArray<FLiveLinkSourcePreset> Sources;

	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSubjectPresets")
	TArray<FLiveLinkSubjectPreset> Subjects;

public:
	/** Get the list of source presets. */
	const TArray<FLiveLinkSourcePreset>& GetSourcePresets() const { return Sources; }

	/** Get the list of subject presets. */
	const TArray<FLiveLinkSubjectPreset>& GetSubjectPresets() const { return Subjects; }

	/**
	 * Remove all previous sources and subjects and add the sources and subjects from this preset.
	 * @return True is all sources and subjects from this preset could be created and added.
	 */
	UE_DEPRECATED(5.0, "This function is deprecated, please use ApplyToClientLatent")
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category="LiveLink")
	UE_API bool ApplyToClient() const;

	/**
	 * Remove all previous sources and subjects and add the sources and subjects from this preset.
	 * @return True is all sources and subjects from this preset could be created and added.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category="LiveLink", meta = (Latent, LatentInfo = "LatentInfo", HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	UE_API void ApplyToClientLatent(UObject* WorldContextObject, FLatentActionInfo LatentInfo);
	UE_API void ApplyToClientLatent(TFunction<void(bool)> CompletionCallback = nullptr);
	
	/** Cancels the current latent action and prevents the callback from firing. Only valid for C++ variant. */
	UE_API void CancelLatentAction();

	/**
	 * Add the sources and subjects from this preset, but leave any existing sources and subjects connected.
	 *
	 * @param bRecreatePresets	When true, if subjects and sources from this preset already exist, we will recreate them.
	 *
	 * @return True is all sources and subjects from this preset could be created and added.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "LiveLink")
	UE_API bool AddToClient(const bool bRecreatePresets = true) const;

	/** Reset this preset and build the list of sources and subjects from the client. */
	UFUNCTION(BlueprintCallable, Category="LiveLink")
	UE_API void BuildFromClient();

private:
	/** Clear the timer registered with the current world. */
	UE_API void ClearApplyToClientTimer();
	
private:
	/** Holds a handle to the OnEndFrame delegate used to apply a preset asynchronously with ApplyToClientLatent. */
	FDelegateHandle ApplyToClientEndFrameHandle;

	/** Holds the current ApplyToClient async operation. Only one operation for all presets can be done at a time. */
	static UE_API TPimplPtr<struct FApplyToClientPollingOperation> ApplyToClientPollingOperation;

	/** Utility variable used to keep track of the number of times this was applied. */
	uint32 ApplyCount = 0;
};

#undef UE_API
