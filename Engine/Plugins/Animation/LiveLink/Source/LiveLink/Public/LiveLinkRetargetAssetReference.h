// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LiveLinkRetargetAssetReference.generated.h"

class ULiveLinkRetargetAsset;

// References a live link retarget asset and handles recreation when
USTRUCT()
struct
	UE_DEPRECATED(4.23, "FLiveLinkRetargetAssetReference is no longer used.")
	FLiveLinkRetargetAssetReference
{
public:
	GENERATED_BODY()
};
