// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Templates/SubclassOf.h"

#include "LiveLinkSettings.generated.h"

#define UE_API LIVELINK_API

class ULiveLinkFrameInterpolationProcessor;
class ULiveLinkFramePreProcessor;
class ULiveLinkRole;
class ULiveLinkSubjectSettings;
class ULiveLinkVirtualSubject;

enum class ELiveLinkSourceMode : uint8;


class ULiveLinkPreset;

/**
 * Settings for LiveLinkRole.
 */
USTRUCT()
struct FLiveLinkRoleProjectSetting
{
	GENERATED_BODY()

public:
	FLiveLinkRoleProjectSetting() = default;
	UE_API FLiveLinkRoleProjectSetting(TSubclassOf<ULiveLinkSubjectSettings> DefaultSettingsClass);

public:
	/** The role of the current setting. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink")
	TSubclassOf<ULiveLinkRole> Role;

	/** The settings class to use for the subject. If null, LiveLinkSubjectSettings will be used by default. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink")
	TSubclassOf<ULiveLinkSubjectSettings> SettingClass;

	/** The interpolation to use for the subject. If null, no interpolation will be performed. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink")
	TSubclassOf<ULiveLinkFrameInterpolationProcessor> FrameInterpolationProcessor;

	/** The pre processors to use for the subject. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink")
	TArray<TSubclassOf<ULiveLinkFramePreProcessor>> FramePreProcessors;
};

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class ULiveLinkUserSettings : public UObject
{
	GENERATED_BODY()

public:
	/** The default location in which to save LiveLink presets */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink", meta = (DisplayName = "Preset Save Location"))
	FDirectoryPath PresetSaveDir;

public:
	const FDirectoryPath& GetPresetSaveDir() const { return PresetSaveDir; }
};

/**
 * Settings for LiveLink.
 */
UCLASS(MinimalAPI, config=Game, defaultconfig)
class ULiveLinkSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API ULiveLinkSettings();

	//~ Begin UObject Interface
	UE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

public:
	UPROPERTY(config, EditAnywhere, Category="LiveLink")
	TArray<FLiveLinkRoleProjectSetting> DefaultRoleSettings;

	/** When a settings class is not speficied for a role, this settings class will be used. */
	UPROPERTY(config)
	FSoftClassPath DefaultSettingsClass;

	/** The interpolation class to use for new Subjects if no specific settings we set for the Subject's role. */
	UPROPERTY(config)
	TSubclassOf<ULiveLinkFrameInterpolationProcessor> FrameInterpolationProcessor;

	/** The default preset that should be applied */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink")
	TSoftObjectPtr<ULiveLinkPreset> DefaultLiveLinkPreset;

	/** The default preset that should be applied */
	UPROPERTY(config)
	TSubclassOf<ULiveLinkVirtualSubject> DefaultVirtualSubjectClass;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "PresetSaveDir was moved into LiveLinkUserSettings. Please use ULiveLinkUserSettings::GetPresetSaveDir().")
	UPROPERTY(config)
	FDirectoryPath PresetSaveDir_DEPRECATED;
#endif //WITH_EDITORONLY_DATA

	/** Continuous clock offset correction step */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "LiveLink")
	float ClockOffsetCorrectionStep;

	/** The default evaluation mode a source connected via the message bus should start with. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "LiveLink")
	ELiveLinkSourceMode DefaultMessageBusSourceMode;

	/** The refresh frequency of the list of message bus provider (when discovery is requested). */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="LiveLink", meta=(ConfigRestartRequired=true, ForceUnits=s))
	double MessageBusPingRequestFrequency;

	/** The refresh frequency of the heartbeat when a provider didn't send us an updated. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="LiveLink", meta=(ConfigRestartRequired=true, ForceUnits=s))
	double MessageBusHeartbeatFrequency;

	/** How long we should wait before a provider become unresponsive. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category="LiveLink", meta=(ConfigRestartRequired=true, ForceUnits=s))
	double MessageBusHeartbeatTimeout;

	/** Subjects will be removed when their source has been unresponsive for this long. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "LiveLink", meta=(ForceUnits=s))
	double MessageBusTimeBeforeRemovingInactiveSource;

	/** When an enabled subject is removed and another subject with the same name exists, automatically enable it. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "LiveLink", meta = (DisplayName = "Re-enable shadowed subjects on removal"))
	bool bReEnableSubjectOnRemoval = false;

	/** Whether to Preprocess frames before rebroadcasting them. */
	UPROPERTY(config)
	bool bPreProcessRebroadcastFrames = false;

	/** Whether to translate frames before rebroadcasting them. */
	UPROPERTY(config)
	bool bTranslateRebroadcastFrames = false;

	/**
	 * A source may still exist but does not send frames for a subject.
	 * Time before considering the subject as "invalid".
	 * The subject still exists and can still be evaluated.
	 * An invalid subject is shown as yellow in the LiveLink UI.
	 */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink|UI", DisplayName="Time Without Frame to be Considered as Invalid", meta=(ForceUnits=s))
	double TimeWithoutFrameToBeConsiderAsInvalid;

	/** Color for active Subjects receiving data from their Source. */
	UE_DEPRECATED(5.6, "Not used anymore in favor of using icons.")
	UPROPERTY(config, EditAnywhere, Category = "LiveLink|UI")
	FLinearColor ValidColor;

	/** Color for Subjects that have not received data from their Source for TimeWithoutFrameToBeConsiderAsInvalid. */
	UE_DEPRECATED(5.6, "Not used anymore in favor of using icons.")
	UPROPERTY(config, EditAnywhere, Category = "LiveLink|UI")
	FLinearColor InvalidColor;

	/** Font size of Source names shown in LiveLink Debug View. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink|UI|Debug")
	uint8 TextSizeSource;

	/** Font size of Subject names shown in LiveLink Debug View. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink|UI|Debug")
	uint8 TextSizeSubject;

#if WITH_EDITORONLY_DATA
	/** Whether to rebroadcast evaluted data. */
	UE_DEPRECATED(5.7, "bTransmitEvaluatedData has been moved to the source settings.")
		UPROPERTY(meta = (DeprecatedProperty))
	bool bTransmitEvaluatedData_DEPRECATED = false;
#endif


public:
	UE_API FLiveLinkRoleProjectSetting GetDefaultSettingForRole(TSubclassOf<ULiveLinkRole> Role) const;

	double GetTimeWithoutFrameToBeConsiderAsInvalid() const { return TimeWithoutFrameToBeConsiderAsInvalid; }
	float GetMessageBusPingRequestFrequency() const { return MessageBusPingRequestFrequency; }
	float GetMessageBusHeartbeatFrequency() const { return MessageBusHeartbeatFrequency; }
	double GetMessageBusHeartbeatTimeout() const { return MessageBusHeartbeatTimeout; }
	double GetMessageBusTimeBeforeRemovingDeadSource() const { return MessageBusTimeBeforeRemovingInactiveSource; }

	/** Retrieve the name of the protected DefaultRoleSettings property. */
	static FName GetDefaultRoleSettingsPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(ULiveLinkSettings, DefaultRoleSettings);
	}

};

#undef UE_API
