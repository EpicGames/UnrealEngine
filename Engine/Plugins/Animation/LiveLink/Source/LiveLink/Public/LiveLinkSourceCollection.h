// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Async/TransactionallySafeRecursiveMutex.h"
#include "ILiveLinkClient.h"
#include "LiveLinkSubject.h"
#include "LiveLinkSubjectSettings.h"
#include "LiveLinkVirtualSubject.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"

#define UE_API LIVELINK_API

class FLiveLinkSourceCollection;

class FLiveLinkSubject;
class ILiveLinkSource;
class ILiveLinkSubject;

class ULiveLinkSourceSettings;
class ULiveLinkSubjectSettings;
class ULiveLinkVirtualSubject;

struct FLiveLinkCollectionSourceItem
{
	FLiveLinkCollectionSourceItem() = default;
	FLiveLinkCollectionSourceItem(FLiveLinkCollectionSourceItem&&) = default;
	FLiveLinkCollectionSourceItem& operator=(FLiveLinkCollectionSourceItem&&) = default;

	FLiveLinkCollectionSourceItem(const FLiveLinkCollectionSourceItem&) = delete;
	FLiveLinkCollectionSourceItem& operator=(const FLiveLinkCollectionSourceItem&) = delete;

	FGuid Guid;
	TStrongObjectPtr<ULiveLinkSourceSettings> Setting;
	TSharedPtr<ILiveLinkSource> Source;
	TSharedPtr<FLiveLinkTimedDataInput> TimedData;
	bool bPendingKill = false;
	bool bIsVirtualSource = false;

public:
	UE_API bool IsVirtualSource() const;
};

struct FLiveLinkCollectionSubjectItem
{
	UE_API FLiveLinkCollectionSubjectItem(FLiveLinkSubjectKey InKey, TUniquePtr<FLiveLinkSubject> InLiveSubject, ULiveLinkSubjectSettings* InSettings, bool bInEnabled);
	UE_API FLiveLinkCollectionSubjectItem(FLiveLinkSubjectKey InKey, ULiveLinkVirtualSubject* InVirtualSubject, bool bInEnabled);

public:
	FLiveLinkSubjectKey Key;
	bool bEnabled;
	bool bPendingKill;

	// Todo: These methods should be revisited because they may not be safe to access when LiveLinkHub is ticked outside of  the game thread.
	// ie. Calling methods on a subject that is about to be removed will not keep the underlying livelink subject alive.
	ILiveLinkSubject* GetSubject() { return VirtualSubject ? static_cast<ILiveLinkSubject*>(VirtualSubject.Get()) : static_cast<ILiveLinkSubject*>(LiveSubject.Get()); }
	ULiveLinkVirtualSubject* GetVirtualSubject() { return VirtualSubject.Get(); }
	ILiveLinkSubject* GetSubject() const { return VirtualSubject ? VirtualSubject.Get() : static_cast<ILiveLinkSubject*>(LiveSubject.Get()); }
	ULiveLinkVirtualSubject* GetVirtualSubject() const { return VirtualSubject.Get(); }
	UObject* GetSettings() const { return VirtualSubject ? static_cast<UObject*>(VirtualSubject.Get()) : static_cast<UObject*>(Setting.Get()); }
	ULiveLinkSubjectSettings* GetLinkSettings() const { return Setting.Get(); }
	FLiveLinkSubject* GetLiveSubject() const { return LiveSubject.Get(); }


private:
	TStrongObjectPtr<ULiveLinkSubjectSettings> Setting;
	TUniquePtr<FLiveLinkSubject> LiveSubject;
	TStrongObjectPtr<ULiveLinkVirtualSubject> VirtualSubject;

public:
	FLiveLinkCollectionSubjectItem(const FLiveLinkCollectionSubjectItem&) = delete;
	FLiveLinkCollectionSubjectItem& operator=(const FLiveLinkCollectionSubjectItem&) = delete;
	FLiveLinkCollectionSubjectItem(FLiveLinkCollectionSubjectItem&&) = default;
	FLiveLinkCollectionSubjectItem& operator=(FLiveLinkCollectionSubjectItem&&) = default;

	friend FLiveLinkSourceCollection;
};


class FLiveLinkSourceCollection
{
public:
	// "source guid" for virtual subjects
	static UE_API const FGuid DefaultVirtualSubjectGuid;
	UE_API FLiveLinkSourceCollection();

	UE_NONCOPYABLE(FLiveLinkSourceCollection)

public:

	UE_API void AddSource(FLiveLinkCollectionSourceItem Source);
	UE_API void RemoveSource(FGuid SourceGuid);
	UE_API void RemoveAllSources();
	UE_API FLiveLinkCollectionSourceItem* FindSource(TSharedPtr<ILiveLinkSource> Source);
	UE_API const FLiveLinkCollectionSourceItem* FindSource(TSharedPtr<ILiveLinkSource> Source) const;
	UE_API FLiveLinkCollectionSourceItem* FindSource(FGuid SourceGuid);
	UE_API const FLiveLinkCollectionSourceItem* FindSource(FGuid SourceGuid) const;
	UE_API FLiveLinkCollectionSourceItem* FindVirtualSource(FName VirtualSourceName);
	UE_API const FLiveLinkCollectionSourceItem* FindVirtualSource(FName VirtualSourceName) const;
	/** Get the number of sources in the collection. */
	UE_API int32 NumSources() const;

	UE_API void AddSubject(FLiveLinkCollectionSubjectItem Subject);
	UE_API void RemoveSubject(FLiveLinkSubjectKey SubjectKey);
	UE_API FLiveLinkCollectionSubjectItem* FindSubject(FLiveLinkSubjectKey SubjectKey);
	UE_API const FLiveLinkCollectionSubjectItem* FindSubject(FLiveLinkSubjectKey SubjectKey) const;
	UE_API const FLiveLinkCollectionSubjectItem* FindSubject(FLiveLinkSubjectName SubjectName) const;
	UE_API const FLiveLinkCollectionSubjectItem* FindEnabledSubject(FLiveLinkSubjectName SubjectName) const;
	/** Get the number of subjects in the collection. */
	UE_API int32 NumSubjects() const;

	UE_API bool IsSubjectEnabled(FLiveLinkSubjectKey SubjectKey) const;
	UE_API void SetSubjectEnabled(FLiveLinkSubjectKey SubjectKey, bool bEnabled);

	UE_API void RemovePendingKill();
	UE_API bool RequestShutdown();

	/**
	 * Thread safe way to apply a method over every subject.
	 */
	UE_API void ForEachSubject(TFunctionRef<void(FLiveLinkCollectionSourceItem& /*SourceItem*/, FLiveLinkCollectionSubjectItem& /*SubjectItem*/)> VisitorFunc);

	/**
	 * Thread safe way to apply a method over every subject.
	 */
	UE_API void ForEachSubject(TFunctionRef<void(const FLiveLinkCollectionSourceItem&, const FLiveLinkCollectionSubjectItem&)> VisitorFunc) const;

	/**
	 *  Thread safe way to apply a method over every source.
	 */
	UE_API void ForEachSource(TFunctionRef<void(FLiveLinkCollectionSourceItem& /*SourceItem*/)> VisitorFunc);

	/**
	 * Thread safe way to apply a method over every source.
	 */
	UE_API void ForEachSource(TFunctionRef<void(const FLiveLinkCollectionSourceItem&)> VisitorFunc) const;

	/**
	 * Thread safe way to apply a method over a subject.
	 */
	UE_API void ApplyToSubject(const FLiveLinkSubjectKey& InSubjectKey, TFunctionRef<void(FLiveLinkCollectionSubjectItem& /*SubjectItem*/)> VisitorFunc);

	FSimpleMulticastDelegate& OnLiveLinkSourcesChanged() { return OnLiveLinkSourcesChangedDelegate; }
	FSimpleMulticastDelegate& OnLiveLinkSubjectsChanged() { return OnLiveLinkSubjectsChangedDelegate; }

	FOnLiveLinkSourceChangedDelegate& OnLiveLinkSourceAdded() { return OnLiveLinkSourceAddedDelegate; }
	FOnLiveLinkSourceChangedDelegate& OnLiveLinkSourceRemoved() { return OnLiveLinkSourceRemovedDelegate; }
	FOnLiveLinkSubjectChangedDelegate& OnLiveLinkSubjectAdded() { return OnLiveLinkSubjectAddedDelegate; }
	FOnLiveLinkSubjectChangedDelegate& OnLiveLinkSubjectRemoved() { return OnLiveLinkSubjectRemovedDelegate; }
	FOnLiveLinkSubjectStateChanged& OnLiveLinkSubjectStateChanged() { return OnLiveLinkSubjectStateChangedDelegate; }
	FOnLiveLinkSubjectEnabledDelegate& OnLiveLinkSubjectEnabledChanged() {return OnLiveLinkSubjectEnabledDelegate; } 

	/** Utility method used to broadcast delegates on the game thread if this function is called on a different thread. */
	template <typename DelegateType, typename... ArgTypes>
	void BroadcastOnGameThread(DelegateType& InDelegate, ArgTypes&&... InArgs)
	{
		if (IsInGameThread())
		{
			InDelegate.Broadcast(Forward<ArgTypes>(InArgs)...);
		}
		else
		{
			AsyncTask(ENamedThreads::GameThread, [&InDelegate, ...Args = InArgs] () mutable
			{
				InDelegate.Broadcast(MoveTemp(Args)...);
			});
		}
	}

private:

	/** Handles broadcasting the inner subject state change delegate to all listeners of the collection's delegate. */
	UE_API void HandleSubjectStateChanged(ELiveLinkSubjectState NewState, FLiveLinkSubjectKey SubjectKey);

private:
	TArray<FLiveLinkCollectionSourceItem> Sources;

	TArray<FLiveLinkCollectionSubjectItem> Subjects;

	/** Notify when the client sources list has changed */
	FSimpleMulticastDelegate OnLiveLinkSourcesChangedDelegate;

	/** Notify when a client subjects list has changed */
	FSimpleMulticastDelegate OnLiveLinkSubjectsChangedDelegate;

	/** Notify when a client source's is added */
	FOnLiveLinkSourceChangedDelegate OnLiveLinkSourceAddedDelegate;

	/** Notify when a client source's is removed */
	FOnLiveLinkSourceChangedDelegate OnLiveLinkSourceRemovedDelegate;

	/** Notify when a client subject's is added */
	FOnLiveLinkSubjectChangedDelegate OnLiveLinkSubjectAddedDelegate;

	/** Notify when a client subject's is removed */
	FOnLiveLinkSubjectChangedDelegate OnLiveLinkSubjectRemovedDelegate;

	/** Notify when a client subject's state has changed. (ie. it became stale) */
	FOnLiveLinkSubjectStateChanged OnLiveLinkSubjectStateChangedDelegate;

	/** Notify when a client subject's state of enabled has changed. */
	FOnLiveLinkSubjectEnabledDelegate OnLiveLinkSubjectEnabledDelegate;

	/** Lock to stop multiple threads accessing the Subjects from the collection at the same time */
	mutable UE::FTransactionallySafeRecursiveMutex SubjectsLock;

	/** Lock to stop multiple threads accessing the Sources from the collection at the same time */
	mutable UE::FTransactionallySafeRecursiveMutex SourcesLock;
};

#undef UE_API
