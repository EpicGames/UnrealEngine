// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSubject.h"

#include "Containers/Queue.h"
#include "ITimedDataInput.h"
#include "LiveLinkFrameInterpolationProcessor.h"
#include "LiveLinkFramePreProcessor.h"
#include "LiveLinkSourceSettings.h"
#include "LiveLinkSubjectRemapper.h"

#define UE_API LIVELINK_API

class FLiveLinkTimedDataInput;
class ULiveLinkSubjectSettings;
struct FLiveLinkSubjectTimeSyncData;

enum class ELiveLinkSubjectState : uint8;

struct FLiveLinkInterpolationInfo;

struct FLiveLinkTimeSynchronizationData
{
	/** Whether or not synchronization has been established. */
	bool bHasEstablishedSync = false;

	/** The frame in our buffer where a rollover was detected. Only applicable for time synchronized sources. */
	int32 RolloverFrame = INDEX_NONE;

	/** Frame offset that will be used for this source. */
	int32 Offset = 0;

	/** Frame Time value modulus. When this value is not set, we assume no rollover occurs. */
	TOptional<FFrameTime> RolloverModulus;

	/** Frame rate used as the base for synchronization. */
	FFrameRate SyncFrameRate;

	/** Frame time that synchronization was established (relative to SynchronizationFrameRate). */
	FFrameTime SyncStartTime;
};

/** Struct used to cache information that needs to be accessed on the Game Thread by Timed Data Monitor. */
struct FTimedDataMonitorInfo
{
	FTimedDataChannelSampleTime OldestDataTime;
	FTimedDataChannelSampleTime NewestDataTime;
	TArray<FTimedDataChannelSampleTime> DataTimes;
};

DECLARE_DELEGATE_OneParam(FOnStateChanged, ELiveLinkSubjectState /*NewState*/);

/**
 * Manages subject manipulation either to add or get frame data for specific roles
 */
class FLiveLinkSubject : public ILiveLinkSubject, public ITimedDataInputChannel
{
private:
	using Super = ILiveLinkSubject;

public:
	UE_API explicit FLiveLinkSubject(TSharedPtr<FLiveLinkTimedDataInput> InTimedDataGroup);
	FLiveLinkSubject(const FLiveLinkSubject&) = delete;
	FLiveLinkSubject& operator=(const FLiveLinkSubject&) = delete;
	UE_API virtual ~FLiveLinkSubject();

	//~ Begin ILiveLinkSubject Interface
public:
	UE_API virtual void Initialize(FLiveLinkSubjectKey SubjectKey, TSubclassOf<ULiveLinkRole> InRole, ILiveLinkClient* LiveLinkClient) override;
	UE_API virtual void Update() override;
	UE_API virtual void ClearFrames() override;
	virtual FLiveLinkSubjectKey GetSubjectKey() const override { return SubjectKey; }
	virtual TSubclassOf<ULiveLinkRole> GetRole() const override { return Role; }
	UE_API virtual bool HasValidFrameSnapshot() const override;
	UE_API virtual FLiveLinkStaticDataStruct& GetStaticData(bool bGetOverrideData=true) override;
	UE_API virtual const FLiveLinkStaticDataStruct& GetStaticData() const override;
	UE_API virtual TArray<FLiveLinkTime> GetFrameTimes() const override;
	virtual const TArray<ULiveLinkFrameTranslator::FWorkerSharedPtr> GetFrameTranslators() const override
	{
		FScopeLock Lock(&SettingsCriticalSection);
		return FrameTranslators;
	}
	virtual const ULiveLinkSubjectRemapper::FWorkerSharedPtr GetFrameRemapper() const override
	{
		FScopeLock Lock(&SettingsCriticalSection);
		return SubjectRemapper;
	}
	virtual bool IsRebroadcasted() const override { return bRebroadcastSubject; }
	virtual bool HasStaticDataBeenRebroadcasted() const override { return bRebroadcastStaticDataSent; }
	virtual void SetStaticDataAsRebroadcasted(const bool bInSent) override { bRebroadcastStaticDataSent = bInSent; }

	UE_API virtual void PreprocessFrame(const FLiveLinkStaticDataStruct& InStaticData, FLiveLinkFrameDataStruct& InOutFrameData) override;
	UE_API virtual bool IsPaused() const override;
	UE_API virtual void PauseSubject() override;
	UE_API virtual void UnpauseSubject() override;

protected:
	virtual const FLiveLinkSubjectFrameData& GetFrameSnapshot() const override { return FrameSnapshot; }
	friend class FLiveLinkClient;

public:
	UE_API virtual bool EvaluateFrame(TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame) override;

protected:
	//~ End ILiveLinkSubject Interface

	//~Begin ITimedDataSource Interface
public:
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual ETimedDataInputState GetState() const override;
	UE_API virtual FTimedDataChannelSampleTime GetOldestDataTime() const override;
	UE_API virtual FTimedDataChannelSampleTime GetNewestDataTime() const override;
	UE_API virtual TArray<FTimedDataChannelSampleTime> GetDataTimes() const override;
	UE_API virtual int32 GetNumberOfSamples() const override;
	UE_API virtual bool IsBufferStatsEnabled() const override;
	UE_API virtual void SetBufferStatsEnabled(bool bEnable) override;
	UE_API virtual int32 GetBufferUnderflowStat() const override;
	UE_API virtual int32 GetBufferOverflowStat() const override;
	UE_API virtual int32 GetFrameDroppedStat() const override;
	UE_API virtual void GetLastEvaluationData(FTimedDataInputEvaluationData& OutEvaluationData) const override;
	UE_API virtual void ResetBufferStats() override;
	//~End ITimedDataSource Interface

public:
	UE_API bool EvaluateFrameAtWorldTime(double InWorldTime, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame);
	UE_API bool EvaluateFrameAtSceneTime(const FQualifiedFrameTime& InSceneTime, TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame);

	UE_API bool HasStaticData() const;

	/** Handling setting a new static data. Create role data if not found in map. */
	UE_API void SetStaticData(TSubclassOf<ULiveLinkRole> InRole, FLiveLinkStaticDataStruct&& InStaticData);

	/** Add a frame of data from a FLiveLinkFrameData */
	UE_API void AddFrameData(FLiveLinkFrameDataStruct&& InFrameData);

	UE_API void CacheSettings(ULiveLinkSourceSettings* SourceSetting, ULiveLinkSubjectSettings* SubjectSetting);

	UE_API ELiveLinkSourceMode GetMode() const;
	UE_API FLiveLinkSubjectTimeSyncData GetTimeSyncData();
	UE_API bool IsTimeSynchronized() const;

	/** Get the timestamp of the last time a frame was received for this subject. */
	double GetLastPushTime() const { return LastPushTime; }
	/** Set the last time a frame was received. */
	void SetLastPushTime(double InLastPushTime) { LastPushTime = InLastPushTime; }

	/** Validates if the incoming frame data is compatible with the static data for this subject. */
	UE_API bool ValidateFrameData(const FLiveLinkFrameDataStruct& InFrameData);

	/** Clear the override static data for this subject. */
	UE_API void ClearOverrideStaticData_AnyThread();

	FLiveLinkFrameIdentifier GetLastRebroadcastedFrameId() const
	{
		return LastRebroadcastFrameId;
	}

	void SetLastRebroadcastedFrameId(FLiveLinkFrameIdentifier FrameId)
	{
		LastRebroadcastFrameId = FrameId;
	}

	/** Get the delegate triggered when the state changes. */
	FOnStateChanged& OnStateChanged() { return OnStateChangedDelegate; }

private:
	UE_API int32 FindNewFrame_WorldTime(const FLiveLinkWorldTime& FrameTime) const;
	UE_API int32 FindNewFrame_WorldTimeInternal(const FLiveLinkWorldTime& FrameTime) const;
	UE_API int32 FindNewFrame_SceneTime(const FQualifiedFrameTime& FrameTime, const FLiveLinkWorldTime& WorldTime) const;
	UE_API int32 FindNewFrame_Latest(const FLiveLinkWorldTime& FrameTime) const;

	/** Reorder frame with the same timecode and create subframes */
	UE_API void AdjustSubFrame_SceneTime(int32 FrameIndex);

	/** Populate OutFrame with a frame based off of the supplied time (pre offsetted) */
	UE_API bool GetFrameAtWorldTime(const double InSeconds, FLiveLinkSubjectFrameData& OutFrame);
	UE_API bool GetFrameAtWorldTime_Closest(const double InSeconds, FLiveLinkSubjectFrameData& OutFrame);
	UE_API bool GetFrameAtWorldTime_Interpolated(const double InSeconds, FLiveLinkSubjectFrameData& OutFrame);

	/** Populate OutFrame with a frame based off of the supplied scene time (pre offsetted). */
	UE_API bool GetFrameAtSceneTime(const FQualifiedFrameTime& InSceneTime, FLiveLinkSubjectFrameData& OutFrame);
	UE_API bool GetFrameAtSceneTime_Closest(const FQualifiedFrameTime& InSceneTime, FLiveLinkSubjectFrameData& OutFrame);
	UE_API bool GetFrameAtSceneTime_Interpolated(const FQualifiedFrameTime& InSceneTime, FLiveLinkSubjectFrameData& OutFrame);
	
	/** Verify interpolation result to update our internal statistics */
	UE_API void VerifyInterpolationInfo(const FLiveLinkInterpolationInfo& InterpolationInfo);

	/** Populate OutFrame with the latest frame. */
	UE_API bool GetLatestFrame(FLiveLinkSubjectFrameData& OutFrame);

	UE_API void ResetFrame(FLiveLinkSubjectFrameData& OutFrame) const;

	/** Update our internal statistics */
	UE_API void IncreaseFrameDroppedStat();
	UE_API void IncreaseBufferUnderFlowStat();
	UE_API void IncreaseBufferOverFlowStat();
	UE_API void UpdateEvaluationData(const FTimedDataInputEvaluationData& EvaluationData);

	/** Remove frames from our buffer - based on receiving order */
	UE_API void RemoveFrames(int32 Count);

	/** Update the cached state for this subject. */
	UE_API void UpdateState();

	//~ Updates cached timed data monitor information from the LiveLink thread (either Game Thread or dedicated thread in Live Link Hub)
	void UpdateCachedOldestDataTime();
	void UpdateCachedNewestDataTime();
	void UdpateCachedDataTimes();

protected:
	/** The role the subject was build with */
	TSubclassOf<ULiveLinkRole> Role;

	TArray<ULiveLinkFramePreProcessor::FWorkerSharedPtr> FramePreProcessors;

	ULiveLinkFrameInterpolationProcessor::FWorkerSharedPtr FrameInterpolationProcessor = nullptr;

	/** List of available translator the subject can use. */
	TArray<ULiveLinkFrameTranslator::FWorkerSharedPtr> FrameTranslators;

	/** Subject remapper used to modify static and frame data for a subject. */
	ULiveLinkSubjectRemapper::FWorkerSharedPtr SubjectRemapper;

private:
	struct FLiveLinkCachedSettings
	{
		ELiveLinkSourceMode SourceMode = ELiveLinkSourceMode::EngineTime;
		FLiveLinkSourceBufferManagementSettings BufferSettings;
	};
	
	struct FSubjectEvaluationStatistics
	{
		TAtomic<int32> BufferUnderflow;
		TAtomic<int32> BufferOverflow;
		TAtomic<int32> FrameDrop;
		FTimedDataInputEvaluationData LastEvaluationData;

		FSubjectEvaluationStatistics();
		FSubjectEvaluationStatistics(const FSubjectEvaluationStatistics&) = delete;
		FSubjectEvaluationStatistics& operator=(const FSubjectEvaluationStatistics&) = delete;
	};

	/** Static data of the subject */
	FLiveLinkStaticDataStruct StaticData;

	/** Override static data, set by the remapper. */
	TOptional<FLiveLinkStaticDataStruct> OverrideStaticData;

	/** Frames added to the subject */
	TArray<FLiveLinkFrameDataStruct> FrameData;

	/** Contains identifier of each frame in the order they were received */
	TQueue<FLiveLinkFrameIdentifier> ReceivedOrderedFrames;

	/** Next identifier to assign to next received frame */
	FLiveLinkFrameIdentifier NextIdentifier = 0;

	/** Current frame snapshot of the evaluation */
	FLiveLinkSubjectFrameData FrameSnapshot;

	/** Name of the subject */
	FLiveLinkSubjectKey SubjectKey;

	/** Timed data input group for the subject */
	TWeakPtr<FLiveLinkTimedDataInput> TimedDataGroup;

	/** Connection settings specified by user */
	FLiveLinkCachedSettings CachedSettings;

	/** Override mode, determined by frame data */
	TOptional<ELiveLinkSourceMode> ModeOverride;

	/** Last time a frame was pushed */
	double LastPushTime = 0.0;

	/** Logging stats is enabled by default. If monitor opens at a later stage,previous stats will be able to be seen */
	bool bIsStatLoggingEnabled = true;

	/** Some stats compiled by the subject. */
	FSubjectEvaluationStatistics EvaluationStatistics;

	/** Last Timecode FrameRate received */
	FFrameRate LastTimecodeFrameRate;

	/** If enabled, rebroadcast this subject */
    bool bRebroadcastSubject = false;

	/** If true, static data has been sent for this rebroadcast */
	bool bRebroadcastStaticDataSent = false;

	/** If true, override static data may remap when caching settings. */
	bool bNeedsStaticRemap = false;
	
	/** Flag set to indicate that a subject is currently paused, so it should keep its last snapshot. */
	std::atomic<bool> bPaused = false;

	/** Flag set to clear the override static data for a subject. */
	std::atomic<bool> bClearOverrideStaticData = false;

	/** Current state of this subject. */
	ELiveLinkSubjectState State;

	/** Delegate called when the state of this subject has changed. */
	FOnStateChanged OnStateChangedDelegate;

	/** Last frame ID that was rebroadcasted. Used to avoid LiveLinkClient from rebroadcasting the same frame twice when transmitting evaluated data. */
	FLiveLinkFrameIdentifier LastRebroadcastFrameId = 0;

	/** 
	 * This struct is updated on the LiveLink thread (either GameThread or dedicated thread in Live Link Hub).
	 * Then it's accessed by TimedDataMonitor on the GameThread.
	 */
	FTimedDataMonitorInfo CachedTimeDataMonitorInfo;

	/** Critical section used to protect access to the cached timed data monitor information. */
	mutable FCriticalSection CachedTimedDataMonitorInfoCS;
	
	/** 
	 * Evaluation can be done on any thread so we need to protect statistic logging 
	 * Some stats requires more than atomic sized vars so a critical section is used to protect when necessary
	 */
	mutable FCriticalSection StatisticCriticalSection;

	/** Used to protect access to translators, preprocessors and interpolation processors since they can be set and accessed in different threads. */
	mutable FCriticalSection SettingsCriticalSection;

	/** Read-write lock to protect FrameSnapshot access. Allows multiple concurrent readers (EvaluateFrame_AnyThread) but exclusive writer access (Update). */
	mutable FRWLock FrameSnapshotRWLock;
};

#undef UE_API
