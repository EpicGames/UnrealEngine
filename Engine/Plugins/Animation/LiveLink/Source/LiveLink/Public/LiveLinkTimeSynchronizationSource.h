// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TimeSynchronizationSource.h"
#include "LiveLinkClient.h"
#include "LiveLinkTimeSynchronizationSource.generated.h"

#define UE_API LIVELINK_API

struct FFrameRate;

UCLASS(MinimalAPI, EditInlineNew)
class ULiveLinkTimeSynchronizationSource : public UTimeSynchronizationSource
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category="LiveLink")
	FLiveLinkSubjectName SubjectName;

private:
	FLiveLinkClient* LiveLinkClient;

	enum class ESyncState
	{
		NotSynced,
		Opened,
	};

	mutable ESyncState State = ESyncState::NotSynced;
	mutable FLiveLinkSubjectTimeSyncData CachedData;
	mutable int64 LastUpdateFrame;
	FLiveLinkSubjectKey SubjectKey;

public:

	UE_API ULiveLinkTimeSynchronizationSource();

	//~ Begin TimeSynchronizationSource API
	UE_API virtual FFrameTime GetNewestSampleTime() const override;
	UE_API virtual FFrameTime GetOldestSampleTime() const override;
	UE_API virtual FFrameRate GetFrameRate() const override;
	UE_API virtual bool IsReady() const override;
	UE_API virtual bool Open(const FTimeSynchronizationOpenData& OpenData) override;
	UE_API virtual void Start(const FTimeSynchronizationStartData& StartData) override;
	UE_API virtual void Close() override;
	UE_API virtual FString GetDisplayName() const override;
	//~ End TimeSynchronizationSource API

private:

	bool IsCurrentStateValid() const;
	void OnModularFeatureRegistered(const FName& FeatureName, class IModularFeature* Feature);
	void OnModularFeatureUnregistered(const FName& FeatureName, class IModularFeature* Feature);
	void UpdateCachedState() const;
};

#undef UE_API
