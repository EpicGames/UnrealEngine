// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimecodeProvider.h"

#include "LiveLinkTypes.h"
#include "LiveLinkTimecodeProvider.generated.h"

#define UE_API LIVELINK_API

struct FPropertyChangedEvent;
struct FQualifiedFrameTime;

class ILiveLinkClient;
class IModularFeature;

UENUM()
enum class ELiveLinkTimecodeProviderEvaluationType
{
	/** Interpolate between, or extrapolate using the 2 frames that are the closest to the current world time. */
	Lerp,

	/** Use the frame that is closest to the current world time. */
	Nearest,

	/** Use the newest frame that was received. When LiveLink Custom Time Step is active, uses the timecode captured at the genlock trigger point. */
	Latest,
};

/**
 * Fetch the latest frames from the LiveLink subject and create a Timecode from it.
 */
UCLASS(MinimalAPI, config=Engine, Blueprintable, editinlinenew)
class ULiveLinkTimecodeProvider : public UTimecodeProvider
{
	GENERATED_BODY()

public:
	UE_API ULiveLinkTimecodeProvider();

	//~ Begin UTimecodeProvider Interface
	UE_API virtual FQualifiedFrameTime GetQualifiedFrameTime() const override;
	
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override
	{
		return State;
	}

	UE_API virtual bool Initialize(class UEngine* InEngine) override;
	UE_API virtual void Shutdown(class UEngine* InEngine) override;
	//~ End UTimecodeProvider Interface

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject Interface

	/**
	 * Allows users to override the target timecode provider subject key. Live
	 * Link Hub will send subject name to the host but the host has to "lookup"
	 * and match it with the available subject keys on the local machine.
	 */
	void SetTargetSubjectKey(const FLiveLinkSubjectKey& InKey)
	{
		SubjectKey = InKey;
	}

	/** Clear the frame history. Useful when using Latest/Nearest evaluation type and wanting to accept timecoded frames older than what's currently in the buffer. */
	UE_API void ClearFrameHistory();

private:
	UE_API FQualifiedFrameTime ConvertTo(FQualifiedFrameTime Value) const;

	/**
	 * Infers the frame time for the current world time by either interpolating between or extrapolating a frame time value from two subject frames provided via live link.
	 *
	 * This method uses the FMath::Lerp function but will intentionally provide an Alpha value greater than 1.0 when extrapolation is required.
	 * 
	 * @param Seconds The current world time.
	 * @param IndexA The first subject frame index.
	 * @param IndexB The second subject frame index.
	 * @return The inferred frame time.
	 */
	UE_API FQualifiedFrameTime LerpBetweenFrames(double Seconds, int32 IndexA, int32 IndexB) const;

	UE_API void InitClient();
	UE_API void UninitClient();
	UE_API void RegisterSubject();
	UE_API void UnregisterSubject();
	UE_API void OnLiveLinkClientRegistered(const FName& Type, class IModularFeature* ModularFeature);
	UE_API void OnLiveLinkClientUnregistered(const FName& Type, class IModularFeature* ModularFeature);
	UE_API void OnLiveLinkSubjectAdded(FLiveLinkSubjectKey SubjectKey);
	UE_API void OnLiveLinkSubjectRemoved(FLiveLinkSubjectKey SubjectKey);
	UE_API void OnLiveLinkFrameDataReceived_AnyThread(const FLiveLinkFrameDataStruct& FrameData);

private:
	/** The specific subject that we listen to. */
	UPROPERTY(EditAnywhere, Category = Timecode)
	FLiveLinkSubjectKey SubjectKey;

	/** How to evaluate the timecode. */
	UPROPERTY(EditAnywhere, Category = Timecode)
	ELiveLinkTimecodeProviderEvaluationType Evaluation;

	UPROPERTY(EditAnywhere, Category = Timecode, meta=(InlineEditConditionToggle))
	bool bOverrideFrameRate;

	/**
	 * Override the frame rate at which this timecode provider will create its timecode value.
	 * By default, we use the subject frame rate.
	 */
	UPROPERTY(EditAnywhere, Category = Timecode, meta=(EditCondition="bOverrideFrameRate"))
	FFrameRate OverrideFrameRate;

	/** The number of frame to keep in memory. The provider will not be synchronized until the buffer is full at least once. */
	UPROPERTY(AdvancedDisplay, EditAnywhere, Category = Timecode, meta=(ClampMin = "2", UIMin = "2", ClampMax = "10", UIMax = "10"))
	int32 BufferSize;

private:
	TAtomic<ETimecodeProviderSynchronizationState> State;
	ILiveLinkClient* LiveLinkClient;
	FLiveLinkSubjectKey RegisteredSubjectKey;
	TArray<FLiveLinkTime> SubjectFrameTimes;
	mutable FCriticalSection SubjectFrameLock; // Only lock SubjectFrameTimes
	FDelegateHandle RegisterForFrameDataReceivedHandle;

	/** Cached flag indicating whether the monitored subject is from a sequencer source */
	std::atomic<bool> bIsSequencerSourceAnyThread = false;

	/** Stores the original external subject key before sequencer takes over, so we can restore it when sequencer stops */
	FLiveLinkSubjectKey OriginalExternalSubjectKey;

	/** When sequencer takes over, we also register to the disabled external subject to read its timecode */
	FDelegateHandle ExternalSubjectFrameDataReceivedHandle;

	friend struct FLiveLinkHubTimecodeSettings;
};

#undef UE_API
