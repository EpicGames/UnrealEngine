// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITimedDataInput.h"

#include "ClockOffsetEstimatorRamp.h"

#define UE_API LIVELINK_API

class FLiveLinkClient;
enum class ELiveLinkSourceMode : uint8;
struct FLiveLinkBaseFrameData;

class FLiveLinkTimedDataInput : public ITimedDataInput
{
public:
	UE_API FLiveLinkTimedDataInput(FLiveLinkClient* Client, FGuid Source);
	FLiveLinkTimedDataInput(const FLiveLinkTimedDataInput&) = delete;
	FLiveLinkTimedDataInput& operator=(const FLiveLinkTimedDataInput&) = delete;
	UE_API virtual ~FLiveLinkTimedDataInput();

	//~ Begin ITimedDataInput API
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual TArray<ITimedDataInputChannel*> GetChannels() const override;
	UE_API virtual ETimedDataInputEvaluationType GetEvaluationType() const override;
	UE_API virtual void SetEvaluationType(ETimedDataInputEvaluationType Evaluation) override;
	UE_API virtual double GetEvaluationOffsetInSeconds() const override;
	UE_API virtual void SetEvaluationOffsetInSeconds(double Offset) override;
	UE_API virtual FFrameRate GetFrameRate() const override;
	UE_API virtual int32 GetDataBufferSize() const override;
	UE_API virtual void SetDataBufferSize(int32 BufferSize) override;
	virtual bool IsDataBufferSizeControlledByInput() const override { return true; }
	virtual void AddChannel(ITimedDataInputChannel* Channel) override { Channels.Add(Channel); }
	virtual void RemoveChannel(ITimedDataInputChannel* Channel) override { Channels.RemoveSingleSwap(Channel); }
	UE_API virtual UObject* GetSettingsObject() const override;
#if WITH_EDITOR
	UE_API virtual const FSlateBrush* GetDisplayIcon() const override;
#endif
	//~ End ITimedDataInput API

public:
	//Tracks clock difference between each frame received and arrival time in engine referential.
	UE_API void ProcessNewFrameTimingInfo(FLiveLinkBaseFrameData& NewFrameData);

private:
	//Computes the smooth offset based on the average frame time intervals of the source time
	UE_API void UpdateSmoothEngineTimeOffset(const FLiveLinkBaseFrameData& NewFrameData);

private:
	FLiveLinkClient* LiveLinkClient;
	TArray<ITimedDataInputChannel*> Channels;
	FGuid Source;

	//Continuous clock offset estimator for engine time and timecode
	FClockOffsetEstimatorRamp EngineClockOffset;
	FClockOffsetEstimatorRamp TimecodeClockOffset;

	//We will receive each frame for each subject of this source. Stamp last source time/timecode to only update our offset estimation once per "source frame"
	double LastWorldSourceTime = 0.0;
	double LastSceneTime = 0.0;

	static constexpr double FrameIntervalThreshold = 0.005;
	static constexpr double VeryLargeFrameIntervalThreshold = 0.5;
	static constexpr int32 FrameIntervalSnapCount = 5;
	static constexpr int32 FrameTimeBufferSize = 200;

	TArray<double> FrameTimes;

	int32 FrameIntervalChangeCount = 0;
	int32 NumFramesToConsiderForAverage = TNumericLimits<int32>::Max();
};

#undef UE_API
