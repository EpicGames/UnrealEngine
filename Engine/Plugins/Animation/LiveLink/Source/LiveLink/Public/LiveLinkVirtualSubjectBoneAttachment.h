// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"
#include "Math/MathFwd.h"


#include "LiveLinkVirtualSubjectBoneAttachment.generated.h"

#define UE_API LIVELINK_API

/**
 * A bone attachment used by the virtual subjects to attach 2 subjects together by specifying a child and parent bone.
 */
USTRUCT()
struct FLiveLinkVirtualSubjectBoneAttachment
{
	GENERATED_BODY()

public:
	/** Returns whether the attachment can be used. */
	UE_API bool IsValid(const TArray<FLiveLinkSubjectName>& ActiveSubjects) const;

	/** Subject that the child subject will be attached to. */
	UPROPERTY(EditAnywhere, DisplayName = "Parent", Category = "Live Link")
	FLiveLinkSubjectName ParentSubject;

	/** Name of the bone in the parent subject that will serve as a parent to the child bone of the child subject. */
	UPROPERTY(EditAnywhere, Category = "Live Link")
	FName ParentBone;

	/** Subject that will be attached to the parent. */
	UPROPERTY(EditAnywhere, DisplayName = "Child", Category = "Live Link")
	FLiveLinkSubjectName ChildSubject;

	/** Bone that will be attached to the parent bone.  */
	UPROPERTY(EditAnywhere, Category = "Live Link")
	FName ChildBone;

	/** Location of the component relative to its parent */
	UPROPERTY(EditAnywhere,  Category = "Live Link", meta=(LinearDeltaSensitivity = "1", Delta = "1.0"))
	FVector LocationOffset = FVector::ZeroVector;

	/** Rotation of the component relative to its parent */
	UPROPERTY(EditAnywhere, Category = "Live Link", meta=(LinearDeltaSensitivity = "1", Delta = "1.0"))
	FRotator RotationOffset = FRotator::ZeroRotator;

	/** If not empty, hold the last error text set by the IsValid method. Used to bubble up errors to the UI. */
	mutable FText LastError;
};

#undef UE_API
