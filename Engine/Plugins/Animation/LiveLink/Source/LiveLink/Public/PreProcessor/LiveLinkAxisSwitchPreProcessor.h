// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkFramePreProcessor.h"
#include "LiveLinkAxisSwitchPreProcessor.generated.h"

#define UE_API LIVELINK_API


// DO NOT modify the order or values of these enums - the code relies on the order and values and will fail if it's changed
UENUM()
enum class ELiveLinkAxis : uint8
{
	X = 0 UMETA(DisplayName = "X-Axis"),
	Y = 1 UMETA(DisplayName = "Y-Axis"),
	Z = 2 UMETA(DisplayName = "Z-Axis"),
	XNeg = 3 UMETA(DisplayName = "-X-Axis"),
	YNeg = 4 UMETA(DisplayName = "-Y-Axis"),
	ZNeg = 5 UMETA(DisplayName = "-Z-Axis"),
};

/**
 * Allows to switch any axis of an incoming transform with another axis.
 * @note For example the Z-Axis of an incoming transform can be set to the (optionally negated) Y-Axis of the transform in UE.
 * @note This implies that translation, rotation and scale will be affected by switching an axis.
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Transform Axis Switch"))
class ULiveLinkTransformAxisSwitchPreProcessor : public ULiveLinkFramePreProcessor
{
	GENERATED_BODY()

public:
	class FLiveLinkTransformAxisSwitchPreProcessorWorker : public ILiveLinkFramePreProcessorWorker
	{
	public:
		ELiveLinkAxis FrontAxis = ELiveLinkAxis::X;
		ELiveLinkAxis RightAxis = ELiveLinkAxis::Y;
		ELiveLinkAxis UpAxis = ELiveLinkAxis::Z;

		bool bUseOffsetPosition = false;
		bool bUseOffsetOrientation = false;

		FVector OffsetPosition = FVector::ZeroVector;
		FRotator OffsetOrientation = FRotator::ZeroRotator;

		virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
		virtual bool PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const override;
	};

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FrontAxis, RightAxis, UpAxis instead"))
	ELiveLinkAxis OrientationAxisX_DEPRECATED = ELiveLinkAxis::X;
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FrontAxis, RightAxis, UpAxis instead"))
	ELiveLinkAxis OrientationAxisY_DEPRECATED = ELiveLinkAxis::Y;
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FrontAxis, RightAxis, UpAxis instead"))
	ELiveLinkAxis OrientationAxisZ_DEPRECATED = ELiveLinkAxis::Z;
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FrontAxis, RightAxis, UpAxis instead"))
	ELiveLinkAxis TranslationAxisX_DEPRECATED = ELiveLinkAxis::X;
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FrontAxis, RightAxis, UpAxis instead"))
	ELiveLinkAxis TranslationAxisY_DEPRECATED = ELiveLinkAxis::Y;
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FrontAxis, RightAxis, UpAxis instead"))
	ELiveLinkAxis TranslationAxisZ_DEPRECATED = ELiveLinkAxis::Z;
#endif	// WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "LiveLink")
	ELiveLinkAxis FrontAxis = ELiveLinkAxis::X;
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	ELiveLinkAxis RightAxis = ELiveLinkAxis::Y;
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	ELiveLinkAxis UpAxis = ELiveLinkAxis::Z;

	UPROPERTY(EditAnywhere, Category = "Enables", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool bUseOffsetPosition = false;
	UPROPERTY(EditAnywhere, Category = "Enables", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool bUseOffsetOrientation = false;

	UPROPERTY(EditAnywhere, Category = "LiveLink", meta = (EditCondition = "bUseOffsetPosition"))
	FVector OffsetPosition = FVector::ZeroVector;
	UPROPERTY(EditAnywhere, Category = "LiveLink", meta = (EditCondition = "bUseOffsetOrientation"))
	FRotator OffsetOrientation = FRotator::ZeroRotator;

public:
	UE_API virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
	UE_API virtual ULiveLinkFramePreProcessor::FWorkerSharedPtr FetchWorker() override;

public:
	//~ UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

protected:
	TSharedPtr<FLiveLinkTransformAxisSwitchPreProcessorWorker, ESPMode::ThreadSafe> Instance;
};

/**
 * Allows to switch any axis of an incoming animation with another axis.
 * @note For example the Z-Axis of an incoming transform can be set to the (optionally negated) Y-Axis of the transform in UE.
 * @note This implies that translation, rotation and scale will be affected by switching an axis.
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Animation Axis Switch"))
class ULiveLinkAnimationAxisSwitchPreProcessor : public ULiveLinkTransformAxisSwitchPreProcessor
{
	GENERATED_BODY()

public:
	class FLiveLinkAnimationAxisSwitchPreProcessorWorker : public ULiveLinkTransformAxisSwitchPreProcessor::FLiveLinkTransformAxisSwitchPreProcessorWorker
	{
	public:
		virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
		virtual bool PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const override;
	};

public:
	UE_API virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
	UE_API virtual ULiveLinkFramePreProcessor::FWorkerSharedPtr FetchWorker() override;
};

/**
 * Allows to switch any axis of an incoming locator subject with another axis.
 * @note For example the Z-Axis of an incoming transform can be set to the (optionally negated) Y-Axis of the transform in UE.
 * @note This implies that translation, rotation and scale will be affected by switching an axis.
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Locator Axis Switch"))
class ULiveLinkLocatorAxisSwitchPreProcessor : public ULiveLinkTransformAxisSwitchPreProcessor
{
	GENERATED_BODY()

public:
	class FLiveLinkLocatorAxisSwitchPreProcessorWorker : public ULiveLinkTransformAxisSwitchPreProcessor::FLiveLinkTransformAxisSwitchPreProcessorWorker
	{
	public:
		virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
		virtual bool PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const override;
	};

	public:
	UE_API virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
	UE_API virtual ULiveLinkFramePreProcessor::FWorkerSharedPtr FetchWorker() override;
};

#undef UE_API
