// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkFramePreProcessor.h"
#include "LiveLinkDeadbandPreProcessor.generated.h"

#define UE_API LIVELINK_API


/**
 * Implements a deadband filter that gets applied to the transform, with independent thresholds
 * for rotation and translation.
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Transform Deadband"))
class ULiveLinkTransformDeadbandPreProcessor : public ULiveLinkFramePreProcessor
{
	GENERATED_BODY()

public:
	class FLiveLinkTransformDeadbandPreProcessorWorker : public ILiveLinkFramePreProcessorWorker
	{
	public:

		/** If false, transform is left untouched */
		bool bEnableDeadband = true;

		/** Translation is updated only if the change is larger than this threshold */
		double TranslationDeadband = 0;

		/** Rotation is updated only if the change is larger than this threshold */
		double RotationDeadbandInDegrees = 0;

		/** Keeps track of the last accepted location and translation */
		mutable FTransform StableTransform;

	public:

		//~Begin ILiveLinkFramePreProcessorWorker
		virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
		virtual bool PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const override;
		//~End ILiveLinkFramePreProcessorWorker
	};

protected:

	/** If false, transform is left untouched */
	UPROPERTY(EditAnywhere, Category = "Deadband")
	bool bEnableDeadband = true;

	/** Translation is updated only if the change is larger than this threshold */
	UPROPERTY(EditAnywhere, Category = "Deadband")
	float TranslationDeadband = 0;

	/** Rotation is updated only if the change is larger than this threshold */
	UPROPERTY(EditAnywhere, Category = "Deadband")
	float RotationDeadbandInDegrees = 0;

public:
	//~Begin ULiveLinkFramePreProcessor interface
	UE_API virtual TSubclassOf<ULiveLinkRole> GetRole() const override;
	UE_API virtual ULiveLinkFramePreProcessor::FWorkerSharedPtr FetchWorker() override;
	//~End ULiveLinkFramePreProcessor interface

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

protected:

	/** Instance of the worker object */
	TSharedPtr<FLiveLinkTransformDeadbandPreProcessorWorker, ESPMode::ThreadSafe> Instance;
};

#undef UE_API
