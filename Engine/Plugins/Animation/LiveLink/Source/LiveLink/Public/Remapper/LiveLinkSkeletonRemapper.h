// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LiveLinkSubjectRemapper.h"

#include "Engine/SkeletalMesh.h"
#include "Features/IModularFeatures.h"
#include "LiveLinkClient.h"
#include "LiveLinkTypes.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"

#include "LiveLinkSkeletonRemapper.generated.h"

class FLiveLinkSkeletonRemapperWorker : public ILiveLinkSubjectRemapperWorker
{
public:
	virtual void RemapStaticData(FLiveLinkStaticDataStruct& InOutStaticData)
	{
		if (InOutStaticData.GetStruct() && InOutStaticData.GetStruct()->IsChildOf<FLiveLinkSkeletonStaticData>())
		{
			RemapSkeletonStaticData(*InOutStaticData.Cast<FLiveLinkSkeletonStaticData>());
		}
	}

	virtual void RemapFrameData(const FLiveLinkStaticDataStruct& InOutStaticData, FLiveLinkFrameDataStruct& InOutFrameData)
	{
		if (InOutStaticData.GetStruct() && InOutStaticData.GetStruct()->IsChildOf<FLiveLinkSkeletonStaticData>())
		{
			RemapSkeletonFrameData(*InOutStaticData.Cast<FLiveLinkSkeletonStaticData>(), *InOutFrameData.Cast<FLiveLinkAnimationFrameData>());
		}
	}

	virtual void RemapSkeletonStaticData(FLiveLinkSkeletonStaticData& InOutSkeletonData)
	{
		TArray<FName> SourceBoneNames = InOutSkeletonData.GetBoneNames();

		for (int32 Index = 0; Index < SourceBoneNames.Num(); Index++)
		{
			SourceBoneNames[Index] = GetRemappedBoneName(SourceBoneNames[Index]);
		}

		InOutSkeletonData.SetBoneNames(SourceBoneNames);
	}

	virtual void RemapSkeletonFrameData(const FLiveLinkSkeletonStaticData& InOutSkeletonData, FLiveLinkAnimationFrameData& InOutFrameData)
	{
	}

	FName GetRemappedBoneName(FName BoneName) const
	{
		if (const FName* RemappedName = BoneNameMap.Find(BoneName))
		{
			return *RemappedName;
		}
		return BoneName;
	}

	/** Map used to provide new names for the bones in the static data. */
	TMap<FName, FName> BoneNameMap;
};

UCLASS(MinimalAPI, EditInlineNew)
class ULiveLinkSkeletonRemapper : public ULiveLinkSubjectRemapper
{
public:
	GENERATED_BODY()

	virtual void Initialize(const FLiveLinkSubjectKey& SubjectKey) override
	{
		ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		const FLiveLinkStaticDataStruct* StaticData = LiveLinkClient.GetSubjectStaticData_AnyThread(SubjectKey);

		TSubclassOf<ULiveLinkRole> LiveLinkRole = LiveLinkClient.GetSubjectRole_AnyThread(SubjectKey);

		// Note: Should we initialize the bone name map using the reference skeleton?
		if (StaticData && LiveLinkRole && LiveLinkRole->IsChildOf(ULiveLinkAnimationRole::StaticClass()))
		{
			for (FName BoneName : StaticData->Cast<FLiveLinkSkeletonStaticData>()->GetBoneNames())
			{
				BoneNameMap.Add(BoneName, BoneName);
			}
		}
	}

	virtual TSubclassOf<ULiveLinkRole> GetSupportedRole() const override
	{
		return ULiveLinkAnimationRole::StaticClass();
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		bDirty = true;
	}
#endif

	virtual bool IsValidRemapper() const
	{
		return true;
	}

	virtual FWorkerSharedPtr GetWorker() const override
	{
		return Instance;
	}

	virtual FWorkerSharedPtr CreateWorker() override
	{
		Instance = MakeShared<FLiveLinkSkeletonRemapperWorker>();
		Instance->BoneNameMap = BoneNameMap;
		return Instance;
	}

public:
	UPROPERTY(EditAnywhere, Category="Remapper", meta=(DisplayThumbnail="false"))
	TSoftObjectPtr<USkeletalMesh> ReferenceSkeleton;

private:
	/** Instance of the remapper. */
	TSharedPtr<FLiveLinkSkeletonRemapperWorker> Instance;
};
