// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "LiveLinkMessageBusFinder.h"
#include "Misc/App.h"

#define UE_API LIVELINK_API

namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class SListView;

struct FLiveLinkPongMessage;
struct FMessageAddress;
struct FProviderPollResult;
class ILiveLinkClient;
class ITableRow;
class STableViewBase;


DECLARE_DELEGATE_OneParam(FOnLiveLinkMessageBusSourceSelected, FProviderPollResultPtr);

class SLiveLinkMessageBusSourceFactory : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkMessageBusSourceFactory) {}
		SLATE_EVENT(FOnLiveLinkMessageBusSourceSelected, OnSourceSelected)
		SLATE_ARGUMENT(UClass*, FactoryClass)
	SLATE_END_ARGS()

	UE_API ~SLiveLinkMessageBusSourceFactory();

	UE_API void Construct(const FArguments& Args);

	FProviderPollResultPtr GetSelectedSource() const { return SelectedResult; }

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

private:
	/** Represents a source with its latest time since it was polled successfully. */
	struct FLiveLinkSource
	{
		FProviderPollResultPtr PollResult;
		double LastTimeSincePong = 0.0;
		
		FLiveLinkSource(FProviderPollResultPtr InPollResult)
            : PollResult(MoveTemp(InPollResult))
			, LastTimeSincePong(FApp::GetCurrentTime()) 
		{
		}
	};

private:
	/** Create a widget entry in the source list based on a live link source. */
	UE_API TSharedRef<ITableRow> MakeSourceListViewWidget(TSharedPtr<FLiveLinkSource> Source, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Handles selection changing in the list. */
	UE_API void OnSourceListSelectionChanged(TSharedPtr<FLiveLinkSource> Source, ESelectInfo::Type SelectionType);

private:
	/** The widgets displayed in the list. */
	TSharedPtr<SListView<TSharedPtr<FLiveLinkSource>>> ListView;

	/** Latest sources pulled from the discovery manager. */
	TArray<FProviderPollResultPtr> PollData;

	/** The sources displayed by the list. */
	TArray<TSharedPtr<FLiveLinkSource>> Sources;

	/** The source that was selected. */
	FProviderPollResultPtr SelectedResult;

	/** The last source that was selected. */
	FOnLiveLinkMessageBusSourceSelected OnSourceSelected;

	/** The last time at which the UI was refreshed. */
	double LastUIUpdateSeconds = 0;

	/** Time before a source should disappear from the UI after its last pong. */
	double SecondsBeforeSourcesDisappear = 2.0;

	/** Class of the factory that created this widget. */
	UClass* FactoryClass = nullptr;
};

#undef UE_API
