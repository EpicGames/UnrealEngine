// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkFrameTranslator.h"
#include "LiveLinkAnimationRoleToTransform.generated.h"

#define UE_API LIVELINK_API


/**
 * Basic object to translate data from one role to another
 */
UCLASS(MinimalAPI, meta=(DisplayName="Animation To Transform"))
class ULiveLinkAnimationRoleToTransform : public ULiveLinkFrameTranslator
{
	GENERATED_BODY()

public:
	class FLiveLinkAnimationRoleToTransformWorker : public ILiveLinkFrameTranslatorWorker
	{
	public:
		FName BoneName;
		virtual TSubclassOf<ULiveLinkRole> GetFromRole() const override;
		virtual TSubclassOf<ULiveLinkRole> GetToRole() const override;
		virtual bool Translate(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, FLiveLinkSubjectFrameData& OutTranslatedFrame) const override;
	};

protected:
	UPROPERTY(EditAnywhere, Category="LiveLink")
	FName BoneName;

public:
	UE_API virtual TSubclassOf<ULiveLinkRole> GetFromRole() const override;
	UE_API virtual TSubclassOf<ULiveLinkRole> GetToRole() const override;
	UE_API virtual ULiveLinkFrameTranslator::FWorkerSharedPtr FetchWorker() override;

public:
	//~ UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

private:
	TSharedPtr<FLiveLinkAnimationRoleToTransformWorker, ESPMode::ThreadSafe> Instance;
};

#undef UE_API
