// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkFrameTranslator.h"
#include "LiveLinkTransformRoleToAnimation.generated.h"

#define UE_API LIVELINK_API


/**
 * LiveLink Translator used to convert Transform frame data to Animation (Skeletal) frame data.
 */
UCLASS(MinimalAPI, meta=(DisplayName="Transform to Animation"))
class ULiveLinkTransformRoleToAnimation : public ULiveLinkFrameTranslator
{
	GENERATED_BODY()

public:
	class FLiveLinkTransformRoleToAnimationWorker : public ILiveLinkFrameTranslatorWorker
	{
	public:
		FName OutputBoneName;
		virtual TSubclassOf<ULiveLinkRole> GetFromRole() const override;
		virtual TSubclassOf<ULiveLinkRole> GetToRole() const override;
		virtual bool Translate(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, FLiveLinkSubjectFrameData& OutTranslatedFrame) const override;
	};

public:
	/** Name of the resulting bone. Defaults to "Root" */
	UPROPERTY(EditAnywhere, Category="LiveLink")
	FName OutputBoneName = "Root";

public:
	UE_API virtual TSubclassOf<ULiveLinkRole> GetFromRole() const override;
	UE_API virtual TSubclassOf<ULiveLinkRole> GetToRole() const override;
	UE_API virtual ULiveLinkFrameTranslator::FWorkerSharedPtr FetchWorker() override;

public:
	//~ UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

private:
	TSharedPtr<FLiveLinkTransformRoleToAnimationWorker> Instance;
};

#undef UE_API
