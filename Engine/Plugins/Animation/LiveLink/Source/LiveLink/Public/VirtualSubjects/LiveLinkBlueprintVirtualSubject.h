// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkVirtualSubject.h"

#include "LiveLinkBlueprintVirtualSubject.generated.h"

#define UE_API LIVELINK_API

// Base class for creating virtual subjects in Blueprints
UCLASS(MinimalAPI, Blueprintable, Abstract)
class ULiveLinkBlueprintVirtualSubject : public ULiveLinkVirtualSubject
{
	GENERATED_BODY()

public:
	ULiveLinkBlueprintVirtualSubject() = default;

	UE_API virtual void Initialize(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, ILiveLinkClient* InLiveLinkClient) override;
	UE_API virtual void Update() override;

	UFUNCTION(BlueprintImplementableEvent, Category="LiveLink")
	UE_API void OnInitialize();

	UFUNCTION(BlueprintImplementableEvent, Category="LiveLink")
	UE_API void OnUpdate();

	UE_API void UpdateVirtualSubjectStaticData(const FLiveLinkBaseStaticData* InStaticData);
	UE_API void UpdateVirtualSubjectFrameData(const FLiveLinkBaseFrameData* InFrameData, bool bInShouldStampCurrentTime);

	UFUNCTION(BlueprintCallable, CustomThunk, Category = "LiveLink", meta = (CustomStructureParam = "InStruct", BlueprintInternalUseOnly = "true", AllowAbstract = "false"))
	UE_API bool UpdateVirtualSubjectStaticData_Internal(const FLiveLinkBaseStaticData& InStruct);

	UFUNCTION(BlueprintCallable, CustomThunk, Category = "LiveLink", meta = (CustomStructureParam = "InStruct", BlueprintInternalUseOnly = "true", AllowAbstract = "false"))
	UE_API bool UpdateVirtualSubjectFrameData_Internal(const FLiveLinkBaseFrameData& InStruct, bool bInShouldStampCurrentTime);

protected:
	friend class ULiveLinkBlueprintVirtualSubjectFactory;

private:
	UE_API UScriptStruct* GetRoleStaticStruct();
	UE_API UScriptStruct* GetRoleFrameStruct();

	FLiveLinkStaticDataStruct CachedStaticData;
	

	DECLARE_FUNCTION(execUpdateVirtualSubjectStaticData_Internal);
	DECLARE_FUNCTION(execUpdateVirtualSubjectFrameData_Internal);
};

#undef UE_API
