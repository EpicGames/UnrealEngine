// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/BillboardComponent.h"
#include "GameFramework/Actor.h"
#include "LiveLinkTypes.h"
#include "Visualizers/LiveLinkDataPreviewComponent.h"

#include "LiveLinkDataPreview.generated.h"

/**
 * Actor for visualizing LiveLink data in the viewport/level editor
 */

UCLASS(Blueprintable, BlueprintType, DisplayName = "Live Link Data Preview Actor", Category = "LiveLink", MinimalApi)
class ALiveLinkDataPreview : public AActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties
	LIVELINK_API ALiveLinkDataPreview();

	/**
	 * Stop Start animation evaluation.
	 * @param bNewEvaluate Enable Live Link Evaluation
	 */
	UFUNCTION(CallInEditor, BlueprintCallable, DisplayName = "Evaluate Live Link", Category = LiveLink)
	LIVELINK_API void SetEnableLiveLinkData(bool bNewEvaluate);

public:
	/** The list of Live Link subjects this actor will draw. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LiveLink, meta = (ShowOnlyInnerProperties = "true"))
	TArray<FLiveLinkSubjectName> Subjects;

	TSoftObjectPtr<UStaticMesh> BoneMesh;
	TSoftObjectPtr<UStaticMesh> JointMesh;
	TSoftObjectPtr<UStaticMesh> AxisMesh;
	TSoftObjectPtr<UStaticMesh> TransformMesh;
	TSoftObjectPtr<UStaticMesh> CameraMesh;
	TSoftObjectPtr<UStaticMesh> LocatorMesh;
	TSoftObjectPtr<UTexture2D> SpriteTexture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LiveLink)
	bool bDrawLabels;

	// Enable Live Link evaluation
	UPROPERTY(EditAnywhere, Category = LiveLink)
	bool bEvaluateLiveLink;

	TArray<ULiveLinkDataPreviewComponent> Visualizers;

	/** The billboard component showing the actor's root position. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Components)
	TObjectPtr<UBillboardComponent> BillboardComponent;

protected:
	/**
	 * Initialize all the subjects.
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Initialize All Subjects", Category = LiveLink)
	void InitializeSubjects();

	/**
	 * Set the material instance on a Live Link data preview component.
	 * @param InDataPreviewComponent Preview component instanced static mesh.
	 */
	void SetMaterialInstance(ULiveLinkDataPreviewComponent* InDataPreviewComponent);

	// Called every frame
	virtual void Tick(float DeltaTime) override;

	//Actor Interface
	virtual void OnConstruction(const FTransform& Transform) override;

	//PostEnd Actor Interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	virtual bool ShouldTickIfViewportsOnly() const override
	{
		return true;
	}
#endif

private:
	TArray<FLiveLinkSubjectKey> CachedSubjects;
};
