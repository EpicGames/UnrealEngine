// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkTypes.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkLocatorTypes.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "LiveLinkDataPreviewComponent.generated.h"

class FCanvas;
class UFont;
class FPrimitiveDrawInterface;
class FSceneView;

UENUM(BlueprintType)
enum class ELiveLinkVisualBoneType : uint8
{
	Joint = 0		UMETA(DisplayName = "Joint", Toolip = "For use drawing the location of a joint"),
	Bone = 1		UMETA(DisplayName = "Bone", Toolip = "For use drawing a bone pointing at the next child in a chain")
};

/**
 * An instance static mesh component for drawing LiveLink subject data in-level. Support drawing Transforms, Locators, Skeletons and Cameras
 */
UCLASS(ClassGroup=(LiveLink), Transient, DisplayName = "Live Link Data Preview", MinimalApi)
class ULiveLinkDataPreviewComponent : public UInstancedStaticMeshComponent
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	LIVELINK_API ULiveLinkDataPreviewComponent(const FObjectInitializer& ObjectInitializer);

	// Called every frame
	LIVELINK_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	/** The Live Link subject this component will preview. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LiveLink)
	FLiveLinkSubjectName SubjectName;

	/** Bool to control animation evaluation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LiveLink)
	bool bEvaluateLiveLink;

	/** Bool to control visibility of labels. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="LiveLink|Visualization")
	bool bDrawLabels;

	/** Type of bone visualization - joint or bone.  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="LiveLink|Visualization")
	ELiveLinkVisualBoneType BoneVisualType = ELiveLinkVisualBoneType::Bone;

	/**
	 * Stop/Start animation updates. 
	 * @param bNewEvaluateLiveLink New evaluation state.
	 */
	UFUNCTION(BlueprintCallable, Category="LiveLink|Visualization")
	void SetEvaluateLiveLinkData(bool bNewEvaluateLiveLink) { bEvaluateLiveLink = bNewEvaluateLiveLink; }

	/**
	 * Show or Hide Labels
	 * @param bNewDrawLabel New lable visiblity..
	 */
	//TODO: not yet implemented, waiting on slate immediate mode
	UFUNCTION(BlueprintCallable, Category="LiveLink|Visualization")
	void SetDrawLabels(bool bNewDrawLabel) { bDrawLabels = bNewDrawLabel; }
	
	bool bIsDirty;
	
#if WITH_EDITOR
	LIVELINK_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
protected:
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	
	void CreateInstances();
	
	TArray<FTransform> GetJointTransforms() const;
	TArray<FTransform> GetBoneTransforms() const;
	TArray<FVector> GetLocatorPositions() const;
	FTransform GetSingleTransform() const;;

private:
	TArray<FName> MarkerLabels;

	void CacheSkeletalAnimationData(const FLiveLinkSkeletonStaticData* InStaticData, const FLiveLinkAnimationFrameData* InFrameData);
	
	FLiveLinkSkeletonStaticData CachedSkeletonData;
	FLiveLinkAnimationFrameData CachedAnimationData;

	void GetTransformRootSpace(const int32 InTransformIndex, FTransform& OutTransform) const;

	mutable TArray<TPair<bool, FTransform>> CachedRootSpaceTransforms;
	mutable TArray<TPair<bool, TArray<int32>>> CachedChildTransformIndices;
	static bool IsValidTransformIndex(int32 InTransformIndex, FLiveLinkAnimationFrameData AnimData);
	
};
