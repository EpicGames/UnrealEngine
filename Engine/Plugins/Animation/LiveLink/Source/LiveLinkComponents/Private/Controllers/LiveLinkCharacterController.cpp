// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controllers/LiveLinkCharacterController.h"

#include "Components/SceneComponent.h"
#include "Roles/LiveLinkCharacterTypes.h"
#include "Roles/LiveLinkCharacterRole.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkCharacterController)

#define LOCTEXT_NAMESPACE "LiveLinkCharacterController"

bool ULiveLinkCharacterController::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport)
{
	return RoleToSupport == ULiveLinkCharacterRole::StaticClass();
}

void ULiveLinkCharacterController::Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData)
{
	const FLiveLinkCharacterStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkCharacterStaticData>();
	const FLiveLinkCharacterFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkCharacterFrameData>();

	if (StaticData && FrameData)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(GetAttachedComponent()))
		{
			FTransform Transform = OffsetTransform * FrameData->Transform;

			FLiveLinkTransformStaticData TransformStaticData;
			TransformStaticData.bIsLocationSupported = StaticData->bIsLocationSupported;
			TransformStaticData.bIsRotationSupported = StaticData->bIsRotationSupported;
			TransformStaticData.bIsScaleSupported = StaticData->bIsScaleSupported;

			TransformData.ApplyTransform(SceneComponent, Transform, TransformStaticData);
		}
	}
}

#undef LOCTEXT_NAMESPACE

