// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controllers/LiveLinkLightController.h"

#include "Components/SpotLightComponent.h"
#include "Controllers/LiveLinkTransformController.h"
#include "GameFramework/Actor.h"
#include "LiveLinkComponentController.h"
#include "Roles/LiveLinkLightRole.h"
#include "Roles/LiveLinkLightTypes.h"
#include "UObject/EnterpriseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkLightController)

#if WITH_EDITOR
#endif


void ULiveLinkLightController::Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData)
{
	const FLiveLinkLightStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkLightStaticData>();
	const FLiveLinkLightFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkLightFrameData>();

	if (StaticData && FrameData)
	{
		if (ULightComponent* LightComponent = Cast<ULightComponent>(GetAttachedComponent()))
		{
			if (StaticData->bIsTemperatureSupported != LightComponent->bUseTemperature)
			{
				LightComponent->SetUseTemperature(StaticData->bIsTemperatureSupported);
			}
			if (StaticData->bIsTemperatureSupported) { LightComponent->SetTemperature(FrameData->Temperature); }
			if (StaticData->bIsIntensitySupported) { LightComponent->SetIntensity(FrameData->Intensity); }
			if (StaticData->bIsLightColorSupported) { LightComponent->SetLightColor(FrameData->LightColor); }

			if (UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(LightComponent))
			{
				if (StaticData->bIsAttenuationRadiusSupported) { PointLightComponent->SetAttenuationRadius(FrameData->AttenuationRadius); }
				if (StaticData->bIsSourceRadiusSupported) { PointLightComponent->SetSourceRadius(FrameData->SourceRadius); }
				if (StaticData->bIsSoftSourceRadiusSupported) { PointLightComponent->SetSoftSourceRadius(FrameData->SoftSourceRadius); }
				if (StaticData->bIsSourceLenghtSupported) { PointLightComponent->SetSourceLength(FrameData->SourceLength); }

				if (USpotLightComponent* SpotlightComponent = Cast<USpotLightComponent>(LightComponent))
				{
					if (StaticData->bIsInnerConeAngleSupported) { SpotlightComponent->SetInnerConeAngle(FrameData->InnerConeAngle); }
					if (StaticData->bIsOuterConeAngleSupported) { SpotlightComponent->SetOuterConeAngle(FrameData->OuterConeAngle); }
				}
			}
		}
	}
}

bool ULiveLinkLightController::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport)
{
	return RoleToSupport == ULiveLinkLightRole::StaticClass();
}

TSubclassOf<UActorComponent> ULiveLinkLightController::GetDesiredComponentClass() const
{
	return ULightComponent::StaticClass();
}

void ULiveLinkLightController::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	const int32 Version = GetLinkerCustomVersion(FEnterpriseObjectVersion::GUID);
	if (Version < FEnterpriseObjectVersion::LiveLinkControllerSplitPerRole)
	{
		AActor* MyActor = GetOuterActor();
		if (MyActor)
		{
			//Make sure all UObjects we use in our post load have been postloaded
			MyActor->ConditionalPostLoad();

			ULiveLinkComponentController* LiveLinkComponent = Cast<ULiveLinkComponentController>(MyActor->GetComponentByClass(ULiveLinkComponentController::StaticClass()));
			if (LiveLinkComponent)
			{
				LiveLinkComponent->ConditionalPostLoad();

				//If the transform controller that was created to drive the TransformRole is the built in one, set its data structure with the one that we had internally
				if (LiveLinkComponent->ControllerMap.Contains(ULiveLinkTransformRole::StaticClass()))
				{
					ULiveLinkTransformController* TransformController = Cast<ULiveLinkTransformController>(LiveLinkComponent->ControllerMap[ULiveLinkTransformRole::StaticClass()]);
					if (TransformController)
					{
						TransformController->ConditionalPostLoad();
						TransformController->TransformData = TransformData_DEPRECATED;
					}
				}

				//if Subjects role direct controller is us, set the component to control to what we had
 				if (LiveLinkComponent->SubjectRepresentation.Role == ULiveLinkLightRole::StaticClass())
 				{
 					ComponentPicker = ComponentToControl_DEPRECATED;
 				}
			}
		}
	}
#endif //WITH_EDITOR
}


