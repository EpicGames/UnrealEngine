// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controllers/LiveLinkTransformController.h"

#include "GameFramework/Actor.h"
#include "LiveLinkComponentController.h"
#include "LiveLinkComponentPrivate.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "UObject/EnterpriseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkTransformController)

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "LiveLinkTransformController"

void FLiveLinkTransformControllerData::ApplyTransform(USceneComponent* SceneComponent, const FTransform& Transform, const FLiveLinkTransformStaticData& StaticData) const
{
	
	if (SceneComponent)
	{
		if (bUseLocation && StaticData.bIsLocationSupported)
		{
			if (bWorldTransform)
			{
				SceneComponent->SetWorldLocation(Transform.GetLocation(), bSweep, nullptr, bTeleport ? ETeleportType::TeleportPhysics : ETeleportType::ResetPhysics);
			}
			else
			{
				SceneComponent->SetRelativeLocation(Transform.GetLocation(), bSweep, nullptr, bTeleport ? ETeleportType::TeleportPhysics : ETeleportType::ResetPhysics);
			}
		}

		if (bUseRotation && StaticData.bIsRotationSupported)
		{
			if (bWorldTransform)
			{
				SceneComponent->SetWorldRotation(Transform.GetRotation(), bSweep, nullptr, bTeleport ? ETeleportType::TeleportPhysics : ETeleportType::ResetPhysics);
			}
			else
			{
				SceneComponent->SetRelativeRotation(Transform.GetRotation(), bSweep, nullptr, bTeleport ? ETeleportType::TeleportPhysics : ETeleportType::ResetPhysics);
			}
		}

		if (bUseScale && StaticData.bIsScaleSupported)
		{
			if (bWorldTransform)
			{
				SceneComponent->SetWorldScale3D(Transform.GetScale3D());
			}
			else
			{
				SceneComponent->SetRelativeScale3D(Transform.GetScale3D());
			}
		}
	}
}

void FLiveLinkTransformControllerData::CheckForError(FName OwnerName, USceneComponent* SceneComponent) const
{
	if (SceneComponent == nullptr)
	{
		UE_LOGF(LogLiveLinkComponents, Warning, "The component to control is invalid for '%ls'.", *OwnerName.ToString());
#if WITH_EDITOR
		FNotificationInfo NotificationInfo(LOCTEXT("InvalidComponent", "The component to control is invalid."));
		NotificationInfo.ExpireDuration = 2.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
#endif
	}
	else if (SceneComponent->Mobility != EComponentMobility::Movable)
	{
		UE_LOGF(LogLiveLinkComponents, Warning, "The component '%ls' has an invalid mobility.", *OwnerName.ToString());
#if WITH_EDITOR
		FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("InvalidMobility", "'{0}' has an invalid mobility"), FText::FromName(OwnerName)));
		NotificationInfo.ExpireDuration = 2.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
#endif
	}
}


void ULiveLinkTransformController::OnEvaluateRegistered()
{
	AActor* OuterActor = GetOuterActor();
	TransformData.CheckForError(OuterActor ? OuterActor->GetFName() : NAME_None, Cast<USceneComponent>(GetAttachedComponent()));
}

void ULiveLinkTransformController::Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData)
{
	const FLiveLinkTransformStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkTransformStaticData>();
	const FLiveLinkTransformFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkTransformFrameData>();

	if (StaticData && FrameData)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(GetAttachedComponent()))
		{
			CombinedTransform = OffsetTransform * FrameData->Transform;
			TransformData.ApplyTransform(SceneComponent, CombinedTransform, *StaticData);
		}
	}
}

bool ULiveLinkTransformController::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport)
{
	return RoleToSupport == ULiveLinkTransformRole::StaticClass();
}

TSubclassOf<UActorComponent> ULiveLinkTransformController::GetDesiredComponentClass() const
{
	return USceneComponent::StaticClass();
}

void ULiveLinkTransformController::SetAttachedComponent(UActorComponent* ActorComponent)
{
	Super::SetAttachedComponent(ActorComponent);

	AActor* OuterActor = GetOuterActor();
	TransformData.CheckForError(OuterActor ? OuterActor->GetFName() : NAME_None, Cast<USceneComponent>(GetAttachedComponent()));
}

void ULiveLinkTransformController::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	const int32 Version = GetLinkerCustomVersion(FEnterpriseObjectVersion::GUID);
	if (Version < FEnterpriseObjectVersion::LiveLinkControllerSplitPerRole)
	{
		AActor* MyActor = GetOuterActor();
		if (MyActor)
		{
			//Make sure all UObjects we use in our post load have been postloaded
			MyActor->ConditionalPostLoad();

			ULiveLinkComponentController* LiveLinkComponent = Cast<ULiveLinkComponentController>(MyActor->GetComponentByClass(ULiveLinkComponentController::StaticClass()));
			if (LiveLinkComponent)
			{
				LiveLinkComponent->ConditionalPostLoad();

				//if Subjects role direct controller is us, set the component to control to what we had
				if (LiveLinkComponent->SubjectRepresentation.Role == ULiveLinkTransformRole::StaticClass())
				{
					ComponentPicker = ComponentToControl_DEPRECATED;
				}
			}
		}
		
	}
#endif //WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE

