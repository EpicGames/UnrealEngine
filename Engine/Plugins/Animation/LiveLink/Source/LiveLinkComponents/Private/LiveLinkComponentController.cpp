// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkComponentController.h"

#include "ILiveLinkClient.h"
#include "ILiveLinkComponentModule.h"
#include "LiveLinkComponentPrivate.h"
#include "LiveLinkComponentSettings.h"
#include "LiveLinkControllerBase.h"

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "UObject/EnterpriseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkComponentController)

#if WITH_EDITOR
#include "Editor.h"
#include "Kismet2/ComponentEditorUtils.h"
#else
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "LiveLinkController"


static TAutoConsoleVariable<bool> CVarEnableLiveLinkEvaluation(
	TEXT("LiveLink.Component.EnableLiveLinkEvaluation"),
	true,
	TEXT("Whether LiveLink components should evaluate their subject."),
	ECVF_Default);



ULiveLinkComponentController::ULiveLinkComponentController()
	: bUpdateInEditor(true)
	, bIsDirty(false)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_PrePhysics;
	bTickInEditor = true;

#if WITH_EDITOR
	FEditorDelegates::EndPIE.AddUObject(this, &ULiveLinkComponentController::OnEndPIE);
#endif //WITH_EDITOR
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ULiveLinkComponentController::~ULiveLinkComponentController()
{
#if WITH_EDITOR
	FEditorDelegates::EndPIE.RemoveAll(this);
#endif //WITH_EDITOR
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ULiveLinkComponentController::OnSubjectRoleChanged()
{
	//Whenever the subject role is changed, we start from clean controller map. Cleanup the ones currently active
	CleanupControllersInMap();

	if (SubjectRepresentation.Role == nullptr)
	{
		ControllerMap.Empty();
	}
	else
	{
		TArray<TSubclassOf<ULiveLinkRole>> SelectedRoleHierarchy = GetSelectedRoleHierarchyClasses(SubjectRepresentation.Role);
		ControllerMap.Empty(SelectedRoleHierarchy.Num());
		for (const TSubclassOf<ULiveLinkRole>& RoleClass : SelectedRoleHierarchy)
		{
			if (RoleClass)
			{
				//Add each role class of the hierarchy in the map and assign a controller, if any, to each of them
				ControllerMap.FindOrAdd(RoleClass);

				TSubclassOf<ULiveLinkControllerBase> SelectedControllerClass = GetControllerClassForRoleClass(RoleClass);
				SetControllerClassForRole(RoleClass, SelectedControllerClass);
			}
		}
	}

	if (OnControllerMapUpdatedDelegate.IsBound())
	{
		FEditorScriptExecutionGuard ScriptGuard;
		OnControllerMapUpdatedDelegate.Broadcast();
	}
}

void ULiveLinkComponentController::SetSubjectRepresentation(FLiveLinkSubjectRepresentation InSubjectRepresentation)
{
	SubjectRepresentation = InSubjectRepresentation;

	if (IsControllerMapOutdated())
	{
		OnSubjectRoleChanged();
	}
}

void ULiveLinkComponentController::SetControllerClassForRole(TSubclassOf<ULiveLinkRole> RoleClass, TSubclassOf<ULiveLinkControllerBase> DesiredControllerClass)
{
	if (ControllerMap.Contains(RoleClass))
	{
		TObjectPtr<ULiveLinkControllerBase>& CurrentController = ControllerMap.FindOrAdd(RoleClass);
		if (CurrentController == nullptr || CurrentController->GetClass() != DesiredControllerClass)
		{
			//Controller is about to change, cleanup current one before 
			if (CurrentController)
			{
				CurrentController->Cleanup();
			}

			if (DesiredControllerClass != nullptr)
			{
				const EObjectFlags ControllerObjectFlags = GetMaskedFlags(RF_Public | RF_Transactional | RF_ArchetypeObject);
				CurrentController = NewObject<ULiveLinkControllerBase>(this, DesiredControllerClass, NAME_None, ControllerObjectFlags);
				InitializeController(CurrentController);

#if WITH_EDITOR		
				CurrentController->InitializeInEditor();
#endif
			}
			else
			{
				CurrentController = nullptr;
			}
		}
	}

	//Mark ourselves as dirty to update each controller's on next tick
	bIsDirty = true;
}

void ULiveLinkComponentController::OnRegister()
{
	Super::OnRegister();

	bIsDirty = true;

	ILiveLinkComponentsModule& LiveLinkComponentsModule = FModuleManager::GetModuleChecked<ILiveLinkComponentsModule>(TEXT("LiveLinkComponents"));

	if (LiveLinkComponentsModule.OnLiveLinkComponentRegistered().IsBound())
	{
		LiveLinkComponentsModule.OnLiveLinkComponentRegistered().Broadcast(this);
	}
}

#if WITH_EDITOR
void ULiveLinkComponentController::OnEndPIE(bool bIsSimulating)
{
	const UWorld* const World = GetWorld();
	if (World && World->WorldType == EWorldType::PIE)
	{
		// Cleanup each controller when PIE session is ending
		CleanupControllersInMap();
	}
}
#endif //WITH_EDITOR

void ULiveLinkComponentController::DestroyComponent(bool bPromoteChildren /*= false*/)
{
	// Cleanup each controller before this component is destroyed
	CleanupControllersInMap();

	Super::DestroyComponent(bPromoteChildren);
}

void ULiveLinkComponentController::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	UWorld* OwningWorld = GetWorld();

	// Verify if we are in an editor preview world (blueprint editor). Without being able to select the desired component
	// When you spawn a LL component, it defaults to the root component on which we can't reset the transform in case it's manipulated by LL automatically
	if (OwningWorld && OwningWorld->WorldType == EWorldType::EditorPreview && bUpdateInPreviewEditor == false)
	{
		return;
	}

	// Check for spawnable
	if (bIsDirty || !bIsSpawnableCache.IsSet())
	{
		static const FName SequencerActorTag(TEXT("SequencerActor"));
		AActor* OwningActor = GetOwner();

		bIsSpawnableCache = OwningActor && OwningActor->ActorHasTag(SequencerActorTag);

		if (*bIsSpawnableCache && bDisableEvaluateLiveLinkWhenSpawnable)
		{
			bEvaluateLiveLink = false;
		}
	}

	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	if (OwningWorld && OwningWorld->WorldType == EWorldType::PIE)
	{
		const bool bUpdateImmediatelyInPIE = GetDefault<ULiveLinkComponentSettings>()->bUpdateSubjectsImmediatelyInPIE;
		if (bUpdateImmediatelyInPIE && LiveLinkClient.HasPendingSubjectFrames())
		{
			LiveLinkClient.ForceTick();
		}
	}

	// Evaluate subject frame once and pass the data to our controllers
	FLiveLinkSubjectFrameData SubjectData;

	// Verify if global evaluation cvar is on or off. This can be used to stop LL evaluation at large for Pathtracing rendering for example
	const bool bCanEvaluate = bEvaluateLiveLink && CVarEnableLiveLinkEvaluation.GetValueOnGameThread();
	const bool bHasValidData = bCanEvaluate ? LiveLinkClient.EvaluateFrame_AnyThread(SubjectRepresentation.Subject, SubjectRepresentation.Role, SubjectData) : false;

	//Go through each controllers and initialize them if we're dirty and tick them if there's valid data to process
	for (auto& ControllerEntry : ControllerMap)
	{
		ULiveLinkControllerBase* Controller = ControllerEntry.Value;
		if (Controller)
		{
			if (bIsDirty)
			{
				Controller->SetSelectedSubject(SubjectRepresentation);
				Controller->OnEvaluateRegistered();
			}
			
			if (bHasValidData)
			{
				Controller->Tick(DeltaTime, SubjectData);
			}
		}
	}

	if (OnLiveLinkUpdated.IsBound())
	{
		FEditorScriptExecutionGuard ScriptGuard;
		OnLiveLinkUpdated.Broadcast(DeltaTime);
	}

	if (bHasValidData && OnLiveLinkControllersTicked().IsBound())
	{
		OnLiveLinkControllersTicked().Broadcast(this, SubjectData);
	}

	bIsDirty = false;

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}


void ULiveLinkComponentController::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FEnterpriseObjectVersion::GUID) < FEnterpriseObjectVersion::LiveLinkControllerSplitPerRole)
		{
			ConvertOldControllerSystem();
		}
	}
#endif
}

void ULiveLinkComponentController::PostLoad()
{
	Super::PostLoad();

	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return;
	}

#if WITH_EDITOR
 	const int32 Version = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID);
 	if (Version < FUE5MainStreamObjectVersion::LiveLinkComponentPickerPerController)
 	{
		for (auto& ControllerEntry : ControllerMap)
		{
			ULiveLinkControllerBase* Controller = ControllerEntry.Value;
			if (Controller)
			{
				Controller->ConditionalPostLoad();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				UActorComponent* OldAttachedComponent = ComponentToControl_DEPRECATED.GetComponent(GetOwner());
				Controller->SetAttachedComponent(OldAttachedComponent);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}
#endif //WITH_EDITOR
}

#if WITH_EDITOR

void ULiveLinkComponentController::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkComponentController, bUpdateInEditor))
	{
		bTickInEditor = bUpdateInEditor;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ULiveLinkComponentController::ConvertOldControllerSystem()
{
	if (Controller_DEPRECATED)
	{
		TArray<TSubclassOf<ULiveLinkRole>> SelectedRoleHierarchy = GetSelectedRoleHierarchyClasses(SubjectRepresentation.Role);
		ControllerMap.Empty(SelectedRoleHierarchy.Num());
		for (const TSubclassOf<ULiveLinkRole>& RoleClass : SelectedRoleHierarchy)
		{
			if (RoleClass)
			{
				ControllerMap.FindOrAdd(RoleClass);

				//Set the previous controller on the Subject Role entry and create new controllers for parent role classes
				if (RoleClass == SubjectRepresentation.Role)
				{
					ControllerMap[RoleClass] = Controller_DEPRECATED;
				}
				else
				{
					//Verify in project settings if there is a controller associated with this component type. If not, pick the first one we find
					TSubclassOf<ULiveLinkControllerBase> SelectedControllerClass = GetControllerClassForRoleClass(RoleClass);
					SetControllerClassForRole(RoleClass, SelectedControllerClass);
				}
			}
		}
	}

	Controller_DEPRECATED = nullptr;
}

#endif //WITH_EDITOR

bool ULiveLinkComponentController::IsControllerMapOutdated() const
{
	TArray<TSubclassOf<ULiveLinkRole>> SelectedRoleHierarchy = GetSelectedRoleHierarchyClasses(SubjectRepresentation.Role);
	
	//If the role class hierarchy doesn't have the same number of controllers, early exit, we need to update
	if (ControllerMap.Num() != SelectedRoleHierarchy.Num())
	{
		return true;
	}
	
	//Check if all map matches class hierarchy
	for (const TSubclassOf<ULiveLinkRole>& RoleClass : SelectedRoleHierarchy)
	{
		TObjectPtr<ULiveLinkControllerBase> const* FoundController = ControllerMap.Find(RoleClass);

		//If ControllerMap doesn't have an entry for one of the role class hierarchy, we need to update
		if (FoundController == nullptr)
		{
			return true;
		}
	}

	return false;
}

TArray<TSubclassOf<ULiveLinkRole>> ULiveLinkComponentController::GetSelectedRoleHierarchyClasses(const TSubclassOf<ULiveLinkRole> InCurrentRoleClass) const
{
	TArray<TSubclassOf<ULiveLinkRole>> ClassHierarchy;

	if (InCurrentRoleClass)
	{
		for (TObjectIterator<UClass> It; It; ++It)
		{
			if (!It->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				if (InCurrentRoleClass->IsChildOf(*It))
				{
					ClassHierarchy.AddUnique(*It);
				}
			}
		}
	}

	return ClassHierarchy;
}

TSubclassOf<ULiveLinkControllerBase> ULiveLinkComponentController::GetControllerClassForRoleClass(const TSubclassOf<ULiveLinkRole> RoleClass) const
{
	//Verify in project settings if there is a controller associated with this component type. If not, pick the first one we find that supports that role
	TSubclassOf<ULiveLinkControllerBase> SelectedControllerClass = nullptr;
	const TSubclassOf<ULiveLinkControllerBase>* ControllerClass = GetDefault<ULiveLinkComponentSettings>()->DefaultControllerForRole.Find(RoleClass);
	if (ControllerClass == nullptr || ControllerClass->Get() == nullptr)
	{
		TArray<TSubclassOf<ULiveLinkControllerBase>> NewControllerClasses = ULiveLinkControllerBase::GetControllersForRole(RoleClass);
		if (NewControllerClasses.Num() > 0)
		{
			SelectedControllerClass = NewControllerClasses[0];
		}
	}
	else
	{
		SelectedControllerClass = *ControllerClass;
	}

	return SelectedControllerClass;
}

void ULiveLinkComponentController::CleanupControllersInMap()
{
	//Cleanup the currently active controllers in the map
	for (auto& ControllerPair : ControllerMap)
	{
		if (ControllerPair.Value)
		{
			ControllerPair.Value->Cleanup();
		}
	}
}

void ULiveLinkComponentController::InitializeController(ULiveLinkControllerBase* InController)
{
#if WITH_EDITOR
	// If the OuterActor has a component that matches the desired class of this controller, set that as the component to control. 
	// Otherwise, the default root component will set as the component to control.
	if (AActor* OuterActor = GetOwner())
	{
		TInlineComponentArray<UActorComponent*> ActorComponents;
		OuterActor->GetComponents(InController->GetDesiredComponentClass(), ActorComponents);

		if (ActorComponents.Num() == 0)
		{
			UE_LOGF(LogLiveLinkComponents, Warning, "The desired component class for %ls is %ls, but %ls does not have a component of that type.", *InController->GetName(), *InController->GetDesiredComponentClass()->GetName(), *OuterActor->GetActorLabel(false));
		}

		bool bFoundValidComponent = false;

		// Look through the list of components matching the desired component class, and choose the first editable instance
		for (UActorComponent* ActorComponent : ActorComponents)
		{
			// Check that the selected component is editable (and thus appropriate to be driven by a LiveLink controller)
			if (FComponentEditorUtils::CanEditComponentInstance(ActorComponent, Cast<USceneComponent>(ActorComponent), false))
			{
				InController->SetAttachedComponent(ActorComponent);
				bFoundValidComponent = true;
				break;
			}
		}

		if (!bFoundValidComponent)
		{
			UE_LOGF(LogLiveLinkComponents, Verbose, "%ls has no editable components of type %ls.", *OuterActor->GetActorLabel(false), *InController->GetDesiredComponentClass()->GetName());
		}
	}
#endif //WITH_EDITOR
}

UActorComponent* ULiveLinkComponentController::GetControlledComponent(TSubclassOf<ULiveLinkRole> InRoleClass) const
{
	if (const TObjectPtr<ULiveLinkControllerBase>* ControllerPtr = ControllerMap.Find(InRoleClass))
	{	
		if (const TObjectPtr<ULiveLinkControllerBase> Controller = *ControllerPtr)
		{
			return Controller->GetAttachedComponent();
		}
	}
	return nullptr;
}

void ULiveLinkComponentController::SetControlledComponent(TSubclassOf<ULiveLinkRole> InRoleClass, UActorComponent* InComponent)
{
	if (TObjectPtr<ULiveLinkControllerBase>* ControllerPtr = ControllerMap.Find(InRoleClass))
	{
		if (TObjectPtr<ULiveLinkControllerBase> Controller = *ControllerPtr)
		{
			Controller->SetAttachedComponent(InComponent);
		}
	}
}

#undef LOCTEXT_NAMESPACE
