// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkComponentModule.h"

class FLiveLinkComponentsModule : public ILiveLinkComponentsModule
{
public:
	virtual FLiveLinkComponentRegistered& OnLiveLinkComponentRegistered() override { return OnLiveLinkComponentRegisteredDelegate; }

private:
	FLiveLinkComponentRegistered OnLiveLinkComponentRegisteredDelegate;
};
