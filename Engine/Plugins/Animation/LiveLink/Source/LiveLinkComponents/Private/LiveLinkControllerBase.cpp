// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkControllerBase.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkControllerBase)

#if WITH_EDITOR
#include "Kismet2/ComponentEditorUtils.h"
#else
#include "GameFramework/Actor.h"
#endif // WITH_EDITOR

void ULiveLinkControllerBase::SetAttachedComponent(UActorComponent* ActorComponent)
{
#if WITH_EDITOR
	if (GetAttachedComponent() != ActorComponent)
	{
		ComponentPicker = FComponentEditorUtils::MakeComponentReference(GetOuterActor(), ActorComponent);
	}
#endif // WITH_EDITOR
}

void ULiveLinkControllerBase::SetSelectedSubject(FLiveLinkSubjectRepresentation LiveLinkSubject)
{
	SelectedSubject = LiveLinkSubject;
}

AActor* ULiveLinkControllerBase::GetOuterActor() const
{
	for (UObject* Outer = GetOuter(); Outer != nullptr; Outer = Outer->GetOuter())
	{
		if (AActor* Actor = Cast<AActor>(Outer))
		{
			return Actor;
		}
		if (UActorComponent* Component = Cast<UActorComponent>(Outer))
		{
			return Component->GetOwner();
		}
	}

	return nullptr;
}

TSubclassOf<ULiveLinkControllerBase> ULiveLinkControllerBase::GetControllerForRole(const TSubclassOf<ULiveLinkRole>& RoleToSupport)
{
	TArray<TSubclassOf<ULiveLinkControllerBase>> Controllers = GetControllersForRole(RoleToSupport);
	if (Controllers.Num() > 0)
	{
		return Controllers[0];
	}

	return TSubclassOf<ULiveLinkControllerBase>();
}

TArray<TSubclassOf<ULiveLinkControllerBase>> ULiveLinkControllerBase::GetControllersForRole(const TSubclassOf<ULiveLinkRole>& RoleToSupport)
{
	TArray<TSubclassOf<ULiveLinkControllerBase>> Controllers;

	for (TObjectIterator<UClass> Itt; Itt; ++Itt)
	{
		if (Itt->IsChildOf(ULiveLinkControllerBase::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			if (Itt->GetDefaultObject<ULiveLinkControllerBase>()->IsRoleSupported(RoleToSupport))
			{
				Controllers.Add(TSubclassOf<ULiveLinkControllerBase>(*Itt));
			}
		}
	}

	return MoveTemp(Controllers);
}

UActorComponent* ULiveLinkControllerBase::GetAttachedComponent() const
{
	return ComponentPicker.GetComponent(GetOuterActor());
}

void ULiveLinkControllerBase::OnComponentToControlChanged()
{
}

void ULiveLinkControllerBase::PostLoad()
{
	Super::PostLoad();
}
