// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Controllers/LiveLinkTransformController.h"

#include "LiveLinkCharacterController.generated.h"

#define UE_API LIVELINKCOMPONENTS_API

/** Controller that allows driving an actor's transform through a subject with the Character role. */
UCLASS(MinimalAPI, BlueprintType)
class ULiveLinkCharacterController : public ULiveLinkTransformController
{
	GENERATED_BODY()

public:
	//~Begin ULiveLinkControllerBase interface
	UE_API virtual void Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData) override;
	UE_API virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) override;
	//~End ULiveLinkControllerBase interface
};

#undef UE_API
