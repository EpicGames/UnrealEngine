// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTransformController.h"
#include "LiveLinkLightController.generated.h"

#define UE_API LIVELINKCOMPONENTS_API


/**
 * Controller that uses LiveLink light data to drive a light component. 
 * UPointLightComponent and USpotLightComponent are supported for specific properties
 */
UCLASS(MinimalAPI)
class ULiveLinkLightController : public ULiveLinkControllerBase
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FComponentReference ComponentToControl_DEPRECATED;

	UPROPERTY()
	FLiveLinkTransformControllerData TransformData_DEPRECATED;
#endif

public:

	//~ Begin ULiveLinkControllerBase interface
	UE_API virtual void Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData) override;
	UE_API virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) override;
	UE_API virtual TSubclassOf<UActorComponent> GetDesiredComponentClass() const override;
	//~ End ULiveLinkControllerBase interface

	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface
};

#undef UE_API
