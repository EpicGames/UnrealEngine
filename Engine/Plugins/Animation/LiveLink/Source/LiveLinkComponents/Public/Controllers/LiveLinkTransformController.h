// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkControllerBase.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "LiveLinkTransformController.generated.h"

#define UE_API LIVELINKCOMPONENTS_API

class ULiveLinkRole;

struct FLiveLinkTransformStaticData;
class USceneComponent;

USTRUCT(Blueprintable)
struct FLiveLinkTransformControllerData
{
	GENERATED_BODY()
	
	/** Set the transform of the component in world space of in its local reference frame. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bWorldTransform = false;

	/** Whether we should set the owning actor's location with the value coming from live link. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", AdvancedDisplay)
	bool bUseLocation = true;

	/** Whether we should set the owning actor's rotation with the value coming from live link. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", AdvancedDisplay)
	bool bUseRotation = true;
	
	/** Whether we should set the owning actor's scale with the value coming from live link. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", AdvancedDisplay)
	bool bUseScale = true;

	/**
	 * Whether we sweep to the destination location, triggering overlaps along the way and stopping short of the target if blocked by something.
	 * Only the root component is swept and checked for blocking collision, child components move without sweeping. If collision is off, this has no effect.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", AdvancedDisplay)
	bool bSweep = false;

	/**
	 * Whether we teleport the physics state (if physics collision is enabled for this object).
	 * If true, physics velocity for this object is unchanged (so ragdoll parts are not affected by change in location).
	 * If false, physics velocity is updated based on the change in position (affecting ragdoll parts).
	 * If CCD is on and not teleporting, this will affect objects along the entire sweep volume.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", AdvancedDisplay)
	bool bTeleport = true;
	
	UE_API void ApplyTransform(USceneComponent* SceneComponent, const FTransform& Transform, const FLiveLinkTransformStaticData& StaticData) const;
	UE_API void CheckForError(FName OwnerName, USceneComponent* SceneComponent) const;
};

/**
 */
UCLASS(MinimalAPI, BlueprintType)
class ULiveLinkTransformController : public ULiveLinkControllerBase
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FComponentReference ComponentToControl_DEPRECATED;
#endif //WITH_EDITORONLY_DATA
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LiveLink", meta=(ShowOnlyInnerProperties))
	FLiveLinkTransformControllerData TransformData;
	
	/**
	 * Offset transform applied in local space to the controlled scene component
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FTransform OffsetTransform = FTransform::Identity;

private:

	/** Combined transform resulting from composing the incoming LiveLink transform & the offset transform */
	FTransform CombinedTransform;

public:
	//~Begin ULiveLinkControllerBase interface
	UE_API virtual void OnEvaluateRegistered() override;
	UE_API virtual void Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData) override;
	UE_API virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) override;
	UE_API virtual TSubclassOf<UActorComponent> GetDesiredComponentClass() const override;
	UE_API virtual void SetAttachedComponent(UActorComponent* ActorComponent) override;
	//~End ULiveLinkControllerBase interface

	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface
};

#undef UE_API
