// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"


class ULiveLinkComponentController;

DECLARE_MULTICAST_DELEGATE_OneParam(FLiveLinkComponentRegistered, ULiveLinkComponentController*);

/**
 * Interface for LiveLinkComponent module
 */
class ILiveLinkComponentsModule : public IModuleInterface
{
public:
	virtual FLiveLinkComponentRegistered& OnLiveLinkComponentRegistered() = 0;

public:
	/** Virtual destructor. */
	virtual ~ILiveLinkComponentsModule() = default;
};
