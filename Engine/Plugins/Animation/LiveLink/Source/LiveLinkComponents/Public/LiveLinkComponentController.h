// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "LiveLinkRole.h"
#include "LiveLinkComponentController.generated.h"

#define UE_API LIVELINKCOMPONENTS_API

class ULiveLinkComponentController;
struct FLiveLinkSubjectFrameData;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FLiveLinkTickDelegate, float, DeltaTime);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnControllerMapUpdatedDelegate);

DECLARE_MULTICAST_DELEGATE_TwoParams(FLiveLinkControllersTicked, const ULiveLinkComponentController* const, const FLiveLinkSubjectFrameData&);

class ULiveLinkControllerBase;

UCLASS(MinimalAPI,  ClassGroup=(LiveLink), meta=(DisplayName="LiveLink Controller", BlueprintSpawnableComponent) )
class ULiveLinkComponentController : public UActorComponent
{
	GENERATED_BODY()
	
public:
	UE_API ULiveLinkComponentController();
	UE_API ~ULiveLinkComponentController();

public:
	UPROPERTY(EditAnywhere, BlueprintSetter = SetSubjectRepresentation, BlueprintGetter = GetSubjectRepresentation, Category="LiveLink")
	FLiveLinkSubjectRepresentation SubjectRepresentation;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Instanced, NoClear)
	TObjectPtr<ULiveLinkControllerBase> Controller_DEPRECATED;
#endif

	/** Instanced controllers used to control the desired role */
	UPROPERTY(Interp, BlueprintReadOnly, EditAnywhere, Category = "LiveLink", Instanced, NoClear, meta = (ShowOnlyInnerProperties))
	TMap<TSubclassOf<ULiveLinkRole>, TObjectPtr<ULiveLinkControllerBase>> ControllerMap;

	UPROPERTY(EditAnywhere, Category="LiveLink", AdvancedDisplay)
	bool bUpdateInEditor;
	
	// This Event is triggered any time new LiveLink data is available, including in the editor
	UPROPERTY(BlueprintAssignable, Category = "LiveLink")
	FLiveLinkTickDelegate OnLiveLinkUpdated;

	// This Event is triggered any time the controller map is updated
	UPROPERTY(BlueprintAssignable, Category = "LiveLink")
	FOnControllerMapUpdatedDelegate OnControllerMapUpdatedDelegate;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property has been deprecated. Please use the ComponentPicker property of each controller in this LiveLink component's controller map.")
	UPROPERTY()
	FComponentReference ComponentToControl_DEPRECATED;
#endif //WITH_EDITORONLY_DATA

	// If true, will not evaluate LiveLink if the attached actor is a spawnable in Sequencer
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bDisableEvaluateLiveLinkWhenSpawnable = true;

	// If false, will not evaluate live link, effectively pausing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bEvaluateLiveLink = true;

	// If true, will tick when the world is a preview (i.e Blueprint editors)
	UPROPERTY(EditAnywhere, Category = "LiveLink", AdvancedDisplay, meta = (EditCondition = "bUpdateInEditor"))
	bool bUpdateInPreviewEditor = false;
	
protected:
	// Keep track when component gets registered or controller map gets changed
	bool bIsDirty;

	// Cache if the owner is a spawnable.
	TOptional<bool> bIsSpawnableCache;

public:
	
	/** Creates an instance of the desired controller class for a specified Role class */
	UE_API void SetControllerClassForRole(TSubclassOf<ULiveLinkRole> RoleClass, TSubclassOf<ULiveLinkControllerBase> DesiredControllerClass);

	/** Return Representation of Subject that is used in the controller */
	UFUNCTION(BlueprintGetter)
	FLiveLinkSubjectRepresentation GetSubjectRepresentation() const { return SubjectRepresentation; }

	/** Set Representation of Subject that is used in the controller and update the controller map */
	UFUNCTION(BlueprintSetter)
	UE_API void SetSubjectRepresentation(FLiveLinkSubjectRepresentation InSubjectRepresentation);

	/** Returns true if ControllerMap needs to be updated for the current Role. Useful for customization or C++ modification to the Role */
	UE_API bool IsControllerMapOutdated() const;
	
	/** Used to notify that the subject role has changed. Mainly from Customization or C++ modification to the subject's Role */
	UE_API void OnSubjectRoleChanged();

	/** Returns the component controlled by the LiveLink controller of the input Role. Returns null if there is no controller for that Role */
	UFUNCTION(BlueprintPure, Category="LiveLink")
	UE_API UActorComponent* GetControlledComponent(TSubclassOf<ULiveLinkRole> InRoleClass) const;

	/** Set the component to control for the LiveLink controller of the input Role */
	UFUNCTION(BlueprintCallable, Category = "LiveLink")
	UE_API void SetControlledComponent(TSubclassOf<ULiveLinkRole> InRoleClass, UActorComponent* InComponent);

	/** Multicast delegate that broadcasts after LiveLink controllers have ticked with the latest frame of subject data */
	FLiveLinkControllersTicked& OnLiveLinkControllersTicked() { return LiveLinkControllersTickedDelegate; }

#if WITH_EDITOR
	/** Used to cleanup controllers when exiting PIE */
	UE_API void OnEndPIE(bool bIsSimulating);
#endif //WITH_EDITOR

	//~ Begin UActorComponent interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void DestroyComponent(bool bPromoteChildren = false) override;
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent interface

	//~ UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

protected:

	/** Returns an array representing the class hierarchy of the given class */
	UE_API TArray<TSubclassOf<ULiveLinkRole>> GetSelectedRoleHierarchyClasses(const TSubclassOf<ULiveLinkRole> InCurrentRoleClass) const;
	UE_API TSubclassOf<ULiveLinkControllerBase> GetControllerClassForRoleClass(const TSubclassOf<ULiveLinkRole> RoleClass) const;

	/** Loops through the controller map and calls Cleanup() on each entry */
	UE_API void CleanupControllersInMap();

	/** Initializes the component that the newly created input controller should control based on its specified desired component class */
	UE_API void InitializeController(ULiveLinkControllerBase* InController);

#if WITH_EDITOR
	/** Called during loading to convert old data to new scheme. */
	UE_API void ConvertOldControllerSystem();
#endif //WITH_EDITOR

private:
	FLiveLinkControllersTicked LiveLinkControllersTickedDelegate;
};

#undef UE_API
