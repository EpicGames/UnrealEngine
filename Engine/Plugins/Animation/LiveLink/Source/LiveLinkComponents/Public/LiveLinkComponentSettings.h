// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Templates/SubclassOf.h"

#include "LiveLinkComponentSettings.generated.h"

class ULiveLinkControllerBase;
class ULiveLinkRole;


/**
 * Settings for LiveLink.
 */
UCLASS(MinimalAPI, config=Game, defaultconfig)
class ULiveLinkComponentSettings : public UObject
{
	GENERATED_BODY()

public:

	/** Default Controller class to use for the specified role */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink", meta = (AllowAbstract = "false"))
	TMap<TSubclassOf<ULiveLinkRole>, TSubclassOf<ULiveLinkControllerBase>> DefaultControllerForRole;

	/** If true, all LiveLink subjects with pending frame data will be updated immediately before the LiveLink components execute (only applies to PIE) */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink")
	bool bUpdateSubjectsImmediatelyInPIE = true;
};
