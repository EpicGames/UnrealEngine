// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Components/ActorComponent.h"

#include "LiveLinkRole.h"
#include "LiveLinkControllerBase.generated.h"

#define UE_API LIVELINKCOMPONENTS_API

struct FLiveLinkSubjectFrameData;

class AActor;

/**
 */
UCLASS(MinimalAPI, Abstract, ClassGroup = (LiveLink), editinlinenew)
class ULiveLinkControllerBase : public UObject
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

	/** Initialize the controller at the first tick of its owner component. */
	virtual void OnEvaluateRegistered() { }

	UE_DEPRECATED(4.25, "This function is deprecated. Use Tick function that received evaluated data instead.")
	virtual void Tick(float DeltaTime, const FLiveLinkSubjectRepresentation& SubjectRepresentation) { }

	/**
	 * Function called every frame with the data evaluated by the component.
	 */
	virtual void Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData) { }

	/**
	 * Can it support a specific role.
	 * This is called on the default object before creating an instance.
	 */
	virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) { return false; }

	/**
	 * Returns the component class that this controller wants to control
	 */
	virtual TSubclassOf<UActorComponent> GetDesiredComponentClass() const { return UActorComponent::StaticClass(); }

	/**
	 * Sets the component this controller is driving
	 */
	UE_API virtual void SetAttachedComponent(UActorComponent* ActorComponent);

	/**
	 * Sets the live link subject from which this controller is receiving data
	 */
	UE_API virtual void SetSelectedSubject(FLiveLinkSubjectRepresentation LiveLinkSubject);

	/**
	 * Cleanup controller state before getting removed
	 */
	virtual void Cleanup() { };

#if WITH_EDITOR
	virtual void InitializeInEditor() {}
#endif

	/** Get the selected LiveLink subject for this controller */
	virtual FLiveLinkSubjectRepresentation GetSelectedSubject() { return SelectedSubject; }

	/**
	 * Returns the component controlled by this controller
	 */
	UE_API UActorComponent* GetAttachedComponent() const;

	/**
	 * Callback to reset the AttachedComponent when the value of the ComponentPicker is changed
	 */
	UE_API void OnComponentToControlChanged();

protected:
	UE_API AActor* GetOuterActor() const;

public:

	UE_DEPRECATED(4.25, "This function is deprecated. Use GetControllersForRole instead and use first element to have the same result.")
	static UE_API TSubclassOf<ULiveLinkControllerBase> GetControllerForRole(const TSubclassOf<ULiveLinkRole>& RoleToSupport);

	/**
	 * Returns the list of ULiveLinkControllerBase classes that support the given role
	 */
	static UE_API TArray<TSubclassOf<ULiveLinkControllerBase>> GetControllersForRole(const TSubclassOf<ULiveLinkRole>& RoleToSupport);

protected:
	/**
	 * A component reference (customized) that allows the user to specify a component that this controller should control
	 */
	UPROPERTY(EditInstanceOnly, Category = "ComponentToControl", meta = (UseComponentPicker, AllowedClasses = "/Script/Engine.ActorComponent", DisallowedClasses = "/Script/LiveLinkComponents.LiveLinkComponentController"))
	FComponentReference ComponentPicker;

protected:
	FLiveLinkSubjectRepresentation SelectedSubject;
};

#undef UE_API
