// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkEditor : ModuleRules
	{
		public LiveLinkEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"LiveLinkAnimationCore",
					"LiveLinkInterface",
					"LiveLink",
					"SlateCore",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"AnimGraph",
					"AssetRegistry",
					"BlueprintGraph",
					"ClassViewer",
					"Core",
					"CoreUObject",
					"DetailCustomizations",
					"EditorStyle",
					"EditorWidgets",
					"Engine",
					"EngineAssetDefinitions",
					"InputCore",
					"KismetCompiler",
					"GraphEditor",
					"LiveLinkComponents",
					"LiveLinkDevice",
					"LiveLinkGraphNode",
					"LiveLinkMessageBusFramework",
					"LiveLinkMovieScene",
					"MessageLog",
					"MovieScene",
					"Persona",
					"PlacementMode",
					"Projects",
					"PropertyEditor",
					"Settings",
					"Sequencer",
					"Slate",
					"TimeManagement",
					"EditorFramework",
					"UnrealEd",
					"WorkspaceMenuStructure",
					"ToolMenus",
					"ToolWidgets",
				}
			);
		}
	}
}
