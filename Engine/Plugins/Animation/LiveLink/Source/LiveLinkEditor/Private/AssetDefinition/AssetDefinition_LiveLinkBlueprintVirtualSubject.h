// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Script/AssetDefinition_Blueprint.h"
#include "VirtualSubjects/LiveLinkBlueprintVirtualSubject.h"
#include "AssetDefinition_LiveLinkBlueprintVirtualSubject.generated.h"

UCLASS()
class UAssetDefinition_LiveLinkBlueprintVirtualSubject : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_LiveLinkBlueprintVirtualSubject", "Blueprint Virtual Subject"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return ULiveLinkBlueprintVirtualSubject::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] =
		{
			FAssetCategoryPath(
				FAssetCategoryPath(NSLOCTEXT("AssetDefinition", "LiveLinkBlueprintVirtualSubject_AssetCategory", "Virtual Production")),
				NSLOCTEXT("AssetDefinition", "LiveLinkBlueprintVirtualSubject_CategorySection", "Live Link"), ECategoryMenuType::Section)
		};

		return Categories;
	}
	// UAssetDefinition End
};
