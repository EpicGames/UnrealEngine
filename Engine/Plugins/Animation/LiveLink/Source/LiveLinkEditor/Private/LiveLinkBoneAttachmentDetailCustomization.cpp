// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkBoneAttachmentDetailCustomization.h"

#include "DetailWidgetRow.h"
#include "Features/IModularFeatures.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "ILiveLinkClient.h"
#include "LiveLinkEditorPrivate.h"
#include "LiveLinkVirtualSubject.h"
#include "LiveLinkVirtualSubjectBoneAttachment.h"
#include "Misc/Guid.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "SLiveLinkBoneSelectionWidget.h"
#include "Templates/SubclassOf.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "LiveLinkBoneAttachmentDetailCustomization"

TSharedRef<IPropertyTypeCustomization> FLiveLinkBoneAttachmentDetailCustomization::MakeInstance()
{
	return MakeShared<FLiveLinkBoneAttachmentDetailCustomization>();
}

void FLiveLinkBoneAttachmentDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FLiveLinkVirtualSubjectBoneAttachment::StaticStruct());
	
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
				.Image(FLiveLinkEditorPrivate::GetStyleSet()->GetBrush("LiveLinkController.WarningIcon"))
				.ToolTipText_Raw(this, &FLiveLinkBoneAttachmentDetailCustomization::GetWarningTooltip)
				.Visibility_Raw(this, &FLiveLinkBoneAttachmentDetailCustomization::HandleWarningVisibility)
		]
	].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}

void FLiveLinkBoneAttachmentDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();
	uint32 NumberOfChild;

	FName ParentSubjectPropertyName = GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ParentSubject);
	FName ParentBonePropertyName = GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ParentBone);
	FName ChildSubjectPropertyName = GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ChildSubject);
	FName ChildBonePropertyName = GET_MEMBER_NAME_CHECKED(FLiveLinkVirtualSubjectBoneAttachment, ChildBone);

	TSharedPtr<IPropertyHandle> ParentSubjectHandle = PropertyHandle->GetChildHandle(ParentSubjectPropertyName);
	TSharedPtr<IPropertyHandle> ParentBoneHandle = PropertyHandle->GetChildHandle(ParentBonePropertyName);
	TSharedPtr<IPropertyHandle> ChildSubjectHandle = PropertyHandle->GetChildHandle(ChildSubjectPropertyName);
	TSharedPtr<IPropertyHandle> ChildBoneHandle = PropertyHandle->GetChildHandle(ChildBonePropertyName);

	// Since bone properties are displayed inline, reset them when we reset the subject.
	ParentSubjectHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateLambda([ParentBoneHandle]() 
		{
			if (ParentBoneHandle)
			{
				ParentBoneHandle->ResetToDefault();
			}
		}));

	ChildSubjectHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateLambda([ChildBoneHandle]()
		{
			if (ChildBoneHandle)
			{
				ChildBoneHandle->ResetToDefault();
			}
		}));

	if (PropertyHandle->GetNumChildren(NumberOfChild) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumberOfChild; ++Index)
		{
			TSharedRef<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(Index).ToSharedRef();

			if (ChildPropertyHandle->GetProperty()->GetFName() != ParentBonePropertyName &&
								ChildPropertyHandle->GetProperty()->GetFName() != ChildBonePropertyName)
			{
				IDetailPropertyRow& DetailRow = ChildBuilder.AddProperty(ChildPropertyHandle)
					.ShowPropertyButtons(true)
					.IsEnabled(MakeAttributeLambda([=] { return !PropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));

				if (ChildPropertyHandle->GetProperty()->GetFName() == ParentSubjectPropertyName)
				{

					CustomizeSubjectRow(ChildPropertyHandle, ParentBoneHandle, DetailRow, CustomizationUtils, BonePickerWidgets.ParentWidget);
				}
				else if (ChildPropertyHandle->GetProperty()->GetFName() == ChildSubjectPropertyName)
				{
					CustomizeSubjectRow(ChildPropertyHandle, ChildBoneHandle, DetailRow, CustomizationUtils, BonePickerWidgets.ChildWidget);
				}
			}
		}
	}
}

void FLiveLinkBoneAttachmentDetailCustomization::CustomizeSubjectRow(TSharedPtr<IPropertyHandle> SubjectPropertyHandle, TSharedPtr<IPropertyHandle> BonePropertyHandle, IDetailPropertyRow& PropertyRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils, TSharedPtr<SLiveLinkBoneSelectionWidget>& BonePickerWidget)
{
	FLiveLinkSubjectKey SubjectKey;
	
	TArray<const void*> RawData;
	SubjectPropertyHandle->AccessRawData(RawData);
	if (RawData.Num() && RawData[0])
	{
		const FLiveLinkSubjectName* SubjectName = reinterpret_cast<const FLiveLinkSubjectName*>(RawData[0]);

		constexpr bool bIncludeDisabledSubject = false;
		constexpr bool bIncludeVirtualSubject = true;

		ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		TArray<FLiveLinkSubjectKey> SubjectKeys = LiveLinkClient.GetSubjects(bIncludeDisabledSubject, bIncludeVirtualSubject);

		if (FLiveLinkSubjectKey* Subject = SubjectKeys.FindByPredicate([Name = *SubjectName](const FLiveLinkSubjectKey& Other) { return Other.SubjectName.Name == Name; }))
		{
			SubjectKey = *Subject;
		}
	}

	BonePickerWidget = SNew(SLiveLinkBoneSelectionWidget, SubjectKey)
					.OnBoneSelectionChanged(this, &FLiveLinkBoneAttachmentDetailCustomization::OnBoneSelected, BonePropertyHandle)
					.OnGetSelectedBone(this, &FLiveLinkBoneAttachmentDetailCustomization::GetSelectedBone, BonePropertyHandle);

	PropertyRow
		.CustomWidget()
		.NameContent()
		[
			SubjectPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(StructCustomizationUtils.GetRegularFont())
				.Text(LOCTEXT("SubjectLabel", "Subject"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SLiveLinkSubjectRepresentationPicker)
					.Font(StructCustomizationUtils.GetRegularFont())
					.Value(this, &FLiveLinkBoneAttachmentDetailCustomization::GetValue, SubjectPropertyHandle)
					.OnValueChanged(this, &FLiveLinkBoneAttachmentDetailCustomization::SetValue, SubjectPropertyHandle, BonePickerWidget)
					.OnGetSubjects(this, &FLiveLinkBoneAttachmentDetailCustomization::GetSubjectsForPicker)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(5.0f, 2.0f, 2.0f, 2.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
					.Font(StructCustomizationUtils.GetRegularFont())
					.Text(LOCTEXT("BoneLabel", "Bone"))
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoWidth()
			[
				BonePickerWidget.ToSharedRef()
			]
		];
}

SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole FLiveLinkBoneAttachmentDetailCustomization::GetValue(TSharedPtr<IPropertyHandle> Handle) const
{
	TArray<const void*> RawData;
	Handle->AccessRawData(RawData);

	for (const void* RawPtr : RawData)
	{
		if (RawPtr)
		{
			FLiveLinkSubjectName SubjectName = *reinterpret_cast<const FLiveLinkSubjectName*>(RawPtr);
			return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole(FGuid(), SubjectName, nullptr);
		}
	}

	return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole();
}

void FLiveLinkBoneAttachmentDetailCustomization::SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue, TSharedPtr<IPropertyHandle> Handle, TSharedPtr<SLiveLinkBoneSelectionWidget> BonePickerWidget)
{
	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Handle->GetProperty());

	FProperty* Property = Handle->GetProperty();

	TArray<void*> RawData;
	Handle->AccessRawData(RawData);
	FLiveLinkSubjectKey NewSubjectKey = NewValue.ToSubjectKey();

	if (BonePickerWidget)
	{
		BonePickerWidget->SetSubject(NewSubjectKey);
	}

	TArray<UObject*> Objects;
	Handle->GetOuterObjects(Objects);

	if (Objects.Num() == 1)
	{
		FLiveLinkSubjectName* Key = (FLiveLinkSubjectName*) Handle->GetValueBaseAddress((uint8*)Objects[0]);
		*Key = NewSubjectKey.SubjectName;
	}
}

FText FLiveLinkBoneAttachmentDetailCustomization::GetWarningTooltip() const
{
	if (StructPropertyHandle)
	{
		TArray<const void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);

		for (const void* RawPtr : RawData)
		{
			if (RawPtr)
			{
				const FLiveLinkVirtualSubjectBoneAttachment* Attachment = reinterpret_cast<const FLiveLinkVirtualSubjectBoneAttachment*>(RawPtr);
				return Attachment->LastError;
			}
		}
	}

	return FText::GetEmpty();
}

EVisibility FLiveLinkBoneAttachmentDetailCustomization::HandleWarningVisibility() const
{
	if (StructPropertyHandle)
	{
		TArray<const void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);

		for (const void* RawPtr : RawData)
		{
			if (RawPtr)
			{
				const FLiveLinkVirtualSubjectBoneAttachment* Attachment = reinterpret_cast<const FLiveLinkVirtualSubjectBoneAttachment*>(RawPtr);
				return Attachment->LastError.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

void FLiveLinkBoneAttachmentDetailCustomization::GetSubjectsForPicker(TArray<FLiveLinkSubjectKey>& OutSubjectKeys)
{
	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	constexpr bool bIncludeDisabledSubject = true;
	constexpr bool bIncludeVirtualSubject = true;
	TArray<FLiveLinkSubjectKey> SubjectKeys = LiveLinkClient.GetSubjects(bIncludeDisabledSubject, bIncludeVirtualSubject);
	TArray<FLiveLinkSubjectKey> FilteredSubjectKeys;

	FLiveLinkSubjectName ParentSubjectName;
	FLiveLinkSubjectName ChildSubjectName;

	// Key of the virtual subject that owns the bone attachment.
	FLiveLinkSubjectKey VirtualSubjectKey;

	if (StructPropertyHandle)
	{
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);

		if (Objects.Num() && Objects[0])
		{
			VirtualSubjectKey = Cast<ULiveLinkVirtualSubject>(Objects[0])->GetSubjectKey();
		}

		TArray<const void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);

		if (RawData.Num() && RawData[0])
		{
			const FLiveLinkVirtualSubjectBoneAttachment* Attachment = reinterpret_cast<const FLiveLinkVirtualSubjectBoneAttachment*>(RawData[0]);
			ParentSubjectName = Attachment->ParentSubject;
			ChildSubjectName = Attachment->ChildSubject;
		}
	}

	for (const FLiveLinkSubjectKey& SubjectKey : SubjectKeys)
	{
		bool bAddKey = true;

		if (!LiveLinkClient.DoesSubjectSupportsRole_AnyThread(SubjectKey, ULiveLinkAnimationRole::StaticClass()))
		{
			bAddKey = false;
		}

		bAddKey = bAddKey && SubjectKey != VirtualSubjectKey;

		bAddKey = bAddKey && SubjectKey.SubjectName != ChildSubjectName
			&& SubjectKey.SubjectName != ParentSubjectName;

		if (bAddKey)
		{
			OutSubjectKeys.Add(SubjectKey);
		}
	}
}

void FLiveLinkBoneAttachmentDetailCustomization::OnBoneSelected(FName SelectedBone, TSharedPtr<IPropertyHandle> BonePropertyHandle) const
{
	if (BonePropertyHandle)
	{
		BonePropertyHandle->SetValue(SelectedBone.ToString());
	}
}

FName FLiveLinkBoneAttachmentDetailCustomization::GetSelectedBone(TSharedPtr<IPropertyHandle> BonePropertyHandle) const
{
	FName BoneName;
	if (BonePropertyHandle)
	{
		FString BoneNameStr;
		BonePropertyHandle->GetValueAsFormattedString(BoneNameStr);
		BoneName = *BoneNameStr;
	}

	return BoneName;
}

#undef LOCTEXT_NAMESPACE
