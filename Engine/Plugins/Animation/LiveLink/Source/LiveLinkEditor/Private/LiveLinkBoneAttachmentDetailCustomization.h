// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "SLiveLinkSubjectRepresentationPicker.h"

class SLiveLinkBoneSelectionWidget;

class FLiveLinkBoneAttachmentDetailCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:
	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	/** Customize a subject row to combine the subject and bone pickers. */
	void CustomizeSubjectRow(TSharedPtr<IPropertyHandle> SubjectPropertyHandle, TSharedPtr<IPropertyHandle> BonePropertyHandle, class IDetailPropertyRow& PropertyRow, class IPropertyTypeCustomizationUtils& StructCustomizationUtils, TSharedPtr<SLiveLinkBoneSelectionWidget>& BonePickerWidget);
	/** When the warning icon is visible, returns the text that will be shown in its tooltip. */
	FText GetWarningTooltip() const;
	/** Returns whether the warning symbol should be visible according to the status of the attachment. */
	EVisibility HandleWarningVisibility() const;
	/** Retrieves the value of a subject property on the attachment. */
	SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole GetValue(TSharedPtr<IPropertyHandle> Handle) const;
	/** Sets the value of a subject property on the attachment. */
	void SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue, TSharedPtr<IPropertyHandle> Handle, TSharedPtr<SLiveLinkBoneSelectionWidget> BonePickerWidget);
	/** Populate a filtered list of subjects for the subject representation. */
	void GetSubjectsForPicker(TArray<FLiveLinkSubjectKey>& OutSubjectKeys);
	/** Update a bone attachment after a bone is selected. */
	void OnBoneSelected(FName SelectedBone, TSharedPtr<IPropertyHandle> BonePropertyHandle) const;
	/** Returns the bone from the property handle. */
	FName GetSelectedBone(TSharedPtr<IPropertyHandle> BonePropertyHandle) const;

private:
	/** Handle to the attachment struct. */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	struct FBonePickerWidgets
	{
		TSharedPtr<SLiveLinkBoneSelectionWidget> ParentWidget;
		TSharedPtr<SLiveLinkBoneSelectionWidget> ChildWidget;
	};

	/** Holds the bone picker widgets for the parent and child bone. */
	FBonePickerWidgets BonePickerWidgets;
};
