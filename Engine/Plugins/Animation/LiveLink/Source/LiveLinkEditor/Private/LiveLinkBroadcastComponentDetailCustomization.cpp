// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkBroadcastComponentDetailCustomization.h"

#include "LiveLinkBroadcastComponent.h"
#include "DetailLayoutBuilder.h"
#include "Roles/LiveLinkAnimationRole.h"

void FLiveLinkBroadcastComponentDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayout = &DetailBuilder;

	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailBuilder.GetSelectedObjects();

	// Hide everything when more than one are selected
	if (SelectedObjects.Num() != 1 || !SelectedObjects[0].IsValid())
	{
		return;
	}

	ULiveLinkBroadcastComponent* Component = CastChecked<ULiveLinkBroadcastComponent>(SelectedObjects[0].Get());

	TSharedRef<IPropertyHandle> RoleProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, Role));
	RoleProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(DetailLayout, &IDetailLayoutBuilder::ForceRefreshDetails));

	TSharedRef<IPropertyHandle> SourceMeshProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, SourceMesh));
	TSharedRef<IPropertyHandle> AllowedBonesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, AllowedBoneNames));
	TSharedRef<IPropertyHandle> AllowedCurvesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkBroadcastComponent, AllowedCurveNames));

	if (Component->Role == ULiveLinkAnimationRole::StaticClass())
	{
		SourceMeshProperty->SetInstanceMetaData("AllowedClasses", TEXT("/Script/Engine.SkeletalMeshComponent"));
	}
	else
	{
		AllowedBonesProperty->MarkHiddenByCustomization();
		AllowedCurvesProperty->MarkHiddenByCustomization();
		SourceMeshProperty->SetInstanceMetaData("AllowedClasses", TEXT(""));
	}
}
