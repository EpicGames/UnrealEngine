// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

/**
* Customizes a LiveLinkBroadcastComponent details.
*/
class FLiveLinkBroadcastComponentDetailCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FLiveLinkBroadcastComponentDetailCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

protected:
	/** Keep a reference to force refresh the layout */
	IDetailLayoutBuilder* DetailLayout = nullptr;
};
