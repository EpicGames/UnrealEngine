// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkClientCommands.h"

#define LOCTEXT_NAMESPACE "LiveLinkClientCommands"

FLiveLinkClientCommands::FLiveLinkClientCommands()
	: TCommands<FLiveLinkClientCommands>("LiveLinkClient.Common", LOCTEXT("LiveLinkCommandsLabel", "Live Link"), NAME_None, FName(TEXT("LiveLinkStyle")))
{
}

void FLiveLinkClientCommands::RegisterCommands()
{
	UI_COMMAND(RemoveSource, "Remove Selected Source(s)", "Remove selected live link source", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveAllSources, "Remove All Sources", "Remove all live link sources", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveSubject, "Remove Subject", "Remove selected live link subject", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveDevice, "Remove Device", "Remove selected live link device", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PauseSubject, "Pause/Unpause Subject", "Pause or unpause selected live link subject", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
