// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FLiveLinkClientCommands : public TCommands<FLiveLinkClientCommands>
{
public:
	FLiveLinkClientCommands();

	TSharedPtr<FUICommandInfo> RemoveSource;
	TSharedPtr<FUICommandInfo> RemoveAllSources;
	TSharedPtr<FUICommandInfo> RemoveSubject;
	TSharedPtr<FUICommandInfo> PauseSubject;
	TSharedPtr<FUICommandInfo> RemoveDevice;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
