// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkClientPanel.h"

#include "Editor.h"
#include "Editor/EditorPerformanceSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "LiveLinkClient.h"
#include "LiveLinkClientPanelToolbar.h"
#include "LiveLinkClientPanelViews.h"
#include "LiveLinkEditorSettings.h"
#include "LiveLinkLog.h"
#include "LiveLinkRole.h"
#include "LiveLinkSourceFactory.h"
#include "LiveLinkSourceSettings.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "SLiveLinkDataView.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"


#define LOCTEXT_NAMESPACE "LiveLinkClientPanel"

SLiveLinkClientPanel::~SLiveLinkClientPanel()
{
	GEditor->UnregisterForUndo(this);
}

void SLiveLinkClientPanel::Construct(const FArguments& Args, FLiveLinkClient* InClient)
{
	GEditor->RegisterForUndo(this);

	check(InClient);
	Client = InClient;

	DetailWidgetIndex = 0;
	
	PanelController = MakeShared<FLiveLinkPanelController>(TAttribute<bool>::CreateSP(this, &SLiveLinkClientPanel::IsInReadOnlyMode));

	const FName LogName = "Live Link";
	TSharedPtr<class IMessageLogListing> MessageLogListing;

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if (MessageLogModule.IsRegisteredLogListing(LogName))
	{
		MessageLogListing = MessageLogModule.GetLogListing(LogName);
	}

	TSharedRef<class SWidget> MessageLogListingWidget = MessageLogListing.IsValid() ? MessageLogModule.CreateLogListingWidget(MessageLogListing.ToSharedRef()) : SNullWidget::NullWidget;

	FProperty* PerformanceThrottlingProperty = FindFieldChecked<FProperty>(UEditorPerformanceSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bThrottleCPUWhenNotForeground));
	PerformanceThrottlingProperty->GetDisplayNameText();
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("PropertyName"), PerformanceThrottlingProperty->GetDisplayNameText());
	FText PerformanceWarningText = FText::Format(LOCTEXT("LiveLinkPerformanceWarningMessage", "Warning: The editor setting '{PropertyName}' is currently enabled.  This will stop editor windows from updating in realtime while the editor is not in focus."), Arguments);
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("MessageLog.ListBorder")) // set panel background color to same color as message log at the bottom
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SLiveLinkClientPanelToolbar, Client)
				.Visibility(this, &SLiveLinkClientPanel::GetVisibilityBasedOnReadOnly)
			]
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Vertical)
				+SSplitter::Slot()
				.Value(0.6f)
				[
					SNew(SBorder)
					.Clipping(EWidgetClipping::ClipToBounds)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(FMargin(4.0f, 4.0f))
					[
						PanelController->SubjectsView->GetWidget()
					]
				]
				+ SSplitter::Slot()
				.Value(0.3f)
				[
					PanelController->GetCombinedDetailsWidget()
				]
				+SSplitter::Slot()
				.Value(0.1f)
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						MessageLogListingWidget
					]
					+SOverlay::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Bottom)
					[
						SNew(SBox)
						.Padding(FMargin(100, 4, 4, 10))
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(this, &SLiveLinkClientPanel::GetMessageCountText)
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(this, &SLiveLinkClientPanel::GetSelectedMessageOccurrenceText)
							]
						]
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 8.0f, 0.0f, 4.0f))
			[
				SNew(SWarningOrErrorBox)
				.Visibility(this, &SLiveLinkClientPanel::ShowEditorPerformanceThrottlingWarning)
				.MessageStyle(EMessageStyle::Warning)
				.Message(PerformanceWarningText)
				[
					SNew(SButton)
					.OnClicked(this, &SLiveLinkClientPanel::DisableEditorPerformanceThrottling)
					.TextStyle(FAppStyle::Get(), "DialogButtonText")
					.Text(LOCTEXT("LiveLinkPerformanceWarningDisable", "Disable"))
				]
			]
		]
	];
}

void SLiveLinkClientPanel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(DetailsPanelEditorObjects);
}

bool SLiveLinkClientPanel::IsInReadOnlyMode() const
{
	return GetDefault<ULiveLinkEditorSettings>()->bReadOnly;
}

EVisibility SLiveLinkClientPanel::GetVisibilityBasedOnReadOnly() const
{
	return IsInReadOnlyMode() ? EVisibility::Collapsed : EVisibility::Visible;
}

void SLiveLinkClientPanel::PostUndo(bool bSuccess)
{
	PanelController->SourcesDetailsView->ForceRefresh();
}

void SLiveLinkClientPanel::PostRedo(bool bSuccess)
{
	PanelController->SourcesDetailsView->ForceRefresh();
}

EVisibility SLiveLinkClientPanel::ShowEditorPerformanceThrottlingWarning() const
{
	const UEditorPerformanceSettings* Settings = GetDefault<UEditorPerformanceSettings>();
	return Settings->bThrottleCPUWhenNotForeground ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SLiveLinkClientPanel::DisableEditorPerformanceThrottling()
{
	UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();
	Settings->bThrottleCPUWhenNotForeground = false;
	Settings->PostEditChange();
	Settings->SaveConfig();
	return FReply::Handled();
}

FText SLiveLinkClientPanel::GetMessageCountText() const
{
	int32 ErrorCount, WarningCount, InfoCount;
	FLiveLinkLog::GetInstance()->GetLogCount(ErrorCount, WarningCount, InfoCount);
	return FText::Format(LOCTEXT("MessageCountText", "{0} Error(s)  {1} Warning(s)"), FText::AsNumber(ErrorCount), FText::AsNumber(WarningCount));
}

FText SLiveLinkClientPanel::GetSelectedMessageOccurrenceText() const
{
	TPair<int32, FTimespan> Occurrence = FLiveLinkLog::GetInstance()->GetSelectedOccurrence();
	if (Occurrence.Get<0>() > 1)
	{
		return FText::Format(LOCTEXT("SelectedMessageOccurrenceText", "Last selected occurrence: {0}"), FText::FromString(Occurrence.Get<1>().ToString()));
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
