// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "LiveLinkPanelController.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API LIVELINKEDITOR_API

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
struct FPropertyChangedEvent;
template <typename ItemType> class STreeView;

class ITableRow;
class FLiveLinkClient;
struct FLiveLinkSourceUIEntry;
struct FLiveLinkSubjectUIEntry;
class FMenuBuilder;
class FUICommandList;
class SLiveLinkDataView;
class ULiveLinkSourceFactory;
class SLiveLinkSourceListView;
class STableViewBase;

class SLiveLinkClientPanel : public SCompoundWidget, public FGCObject, public FEditorUndoClient
{
	SLATE_BEGIN_ARGS(SLiveLinkClientPanel) {}
	SLATE_END_ARGS()

	UE_API virtual ~SLiveLinkClientPanel() override;
	
	UE_API void Construct(const FArguments& Args, FLiveLinkClient* InClient);

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SLiveLinkClientPanel");
	}
	// End of FGCObject interface

	// FEditorUndoClient interface
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	// End FEditorUndoClient interface

private:
	// Controls whether the editor performance throttling warning should be visible
	EVisibility ShowEditorPerformanceThrottlingWarning() const;

	// Handle disabling of editor performance throttling
	FReply DisableEditorPerformanceThrottling();

	// Return the message count text
	FText GetMessageCountText() const;

	// Return the occurrence count and last time occurred text
	FText GetSelectedMessageOccurrenceText() const;

	// Returns whether the panel is in read only mode or not.
	bool IsInReadOnlyMode() const;

	/** Get widget visibility according to whether the panel is in read-only mode. */
	EVisibility GetVisibilityBasedOnReadOnly() const;

private:
	// Handles callback connections between the sources, subjects and details views
	TSharedPtr<FLiveLinkPanelController> PanelController;

	// Pointer to the livelink client
	FLiveLinkClient* Client = nullptr;

	// Map to cover 
	TMap<TObjectPtr<UClass>, TObjectPtr<UObject>> DetailsPanelEditorObjects;

	// Details index
	int32 DetailWidgetIndex = -1;
};

#undef UE_API
