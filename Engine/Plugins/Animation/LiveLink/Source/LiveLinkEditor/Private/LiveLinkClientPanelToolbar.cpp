// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkClientPanelToolbar.h"

#include "Algo/Accumulate.h"
#include "Algo/StableSort.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ISettingsModule.h"
#include "LiveLinkClient.h"
#include "LiveLinkEditorPrivate.h"
#include "LiveLinkPreset.h"
#include "LiveLinkRole.h"
#include "LiveLinkSettings.h"
#include "LiveLinkSourceFactory.h"
#include "Logging/MessageLog.h"
#include "LiveLinkSourceSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "LiveLinkVirtualSubject.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "ScopedTransaction.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SPositiveActionButton.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"


#define LOCTEXT_NAMESPACE "LiveLinkClientPanel"

void SLiveLinkClientPanelToolbar::Construct(const FArguments& Args, FLiveLinkClient* InClient)
{
	Client = InClient;

	ParentWindowOverride = Args._ParentWindow;

	const int32 ButtonBoxSize = 28;
	FMargin ButtonPadding(4.f);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(.0f)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			Args._CustomHeader ? Args._CustomHeader.ToSharedRef() : SNullWidget::NullWidget
		]
		+ SHorizontalBox::Slot()
		.Padding(.0f)
		.HAlign(Args._SourceButtonAlignment)
		.FillWidth(1.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(8.f, 0.f, 0.f, 0.f)
			.AutoWidth()
			[
				SNew(SComboButton)
				.ContentPadding(4.f)
				.ComboButtonStyle(FLiveLinkEditorPrivate::GetStyleSet(), "ComboButton")
				.OnGetMenuContent(this, &SLiveLinkClientPanelToolbar::OnPresetGeneratePresetsMenu)
				.ForegroundColor(FSlateColor::UseForeground())
				.Visibility(Args._ShowPresetPicker ? EVisibility::Visible : EVisibility::Collapsed)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(4.f, 0.f, 4.f, 0.f)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateIconFinder::FindIconBrushForClass(ULiveLinkPreset::StaticClass()))
					]
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PresetsToolbarButton", "Presets"))
					]
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(8.f, 0.f, 0.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				//.ContentPadding(FMargin(8.f, 0.f, 0.f, 0.f))
				.ToolTipText(LOCTEXT("RevertChanges_Text", "Revert all changes made to this take back its original state (either its original preset, or an empty preset)."))
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &SLiveLinkClientPanelToolbar::OnRevertChanges)
				.Visibility(Args._ShowPresetPicker ? EVisibility::Visible : EVisibility::Collapsed)
				.IsEnabled(this, &SLiveLinkClientPanelToolbar::HasLoadedLiveLinkPreset)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Undo)
				]
			]

			// Save preset button
			+ SHorizontalBox::Slot()
			.Padding(8.f, 0.f, 0.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("SavePreset_Tip", "Save the current Live Link preset"))
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &SLiveLinkClientPanelToolbar::OnSavePreset)
				.IsEnabled_Lambda([this]() 
					{ 
						return !Client->GetSources().IsEmpty(); 
					})
				.Visibility(Args._ShowPresetPicker ? EVisibility::Visible : EVisibility::Collapsed)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Floppy_O)
				]
			]

			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(.0f)
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.WidthOverride(ButtonBoxSize)
				.Visibility(Args._ShowSettings ? EVisibility::Visible : EVisibility::Collapsed)
				.HeightOverride(ButtonBoxSize)
				[
					SNew(SCheckBox)
					.Padding(4.f)
					.ToolTipText(LOCTEXT("ShowUserSettings_Tip", "Show/Hide the general user settings for Live Link"))
					.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
					.ForegroundColor(FSlateColor::UseForeground())
					.IsChecked_Lambda([]() { return ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([](ECheckBoxState CheckState){ FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Plugins", "Live Link"); })
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Toolbar.Settings").GetIcon())
					]
				]
			]
		]
	];
}

TSharedRef<SWidget> SLiveLinkClientPanelToolbar::OnPresetGeneratePresetsMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SaveAsPreset_Text", "Save As Preset"),
		LOCTEXT("SaveAsPreset_Tip", "Save the current setup as a new preset that can be imported at a later date"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SLiveLinkClientPanelToolbar::OnSaveAsPreset)
		)
	);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bSortByPathInColumnView = false;

		AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoPresets_Warning", "No Presets Found");
		AssetPickerConfig.Filter.ClassPaths.Add(ULiveLinkPreset::StaticClass()->GetClassPathName());
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SLiveLinkClientPanelToolbar::OnImportPreset);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ImportPreset_MenuSection", "Import Preset"));
	{
		TSharedRef<SWidget> PresetPicker = SNew(SBox)
			.MinDesiredWidth(400.f)
			.MinDesiredHeight(400.f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(PresetPicker, FText(), true, false);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

static bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, TSharedPtr<SWindow> InParentWindowOverride, FString& OutPackageName)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(ULiveLinkPreset::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveLiveLinkPresetDialogTitle", "Save Live Link Preset");
		SaveAssetDialogConfig.WindowOverride = InParentWindowOverride;
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		return true;
	}

	return false;
}

bool GetSavePresetPackageName(TSharedPtr<SWindow> InParentWindowOverride, FString& OutName)
{
	ULiveLinkUserSettings* ConfigSettings = GetMutableDefault<ULiveLinkUserSettings>();

	FDateTime Today = FDateTime::Now();

	TMap<FString, FStringFormatArg> FormatArgs;
	FormatArgs.Add(TEXT("date"), Today.ToString());

	// determine default package path
	const FString DefaultSaveDirectory = FString::Format(*ConfigSettings->GetPresetSaveDir().Path, FormatArgs);

	FString DialogStartPath;
	FPackageName::TryConvertFilenameToLongPackageName(DefaultSaveDirectory, DialogStartPath);
	if (DialogStartPath.IsEmpty())
	{
		DialogStartPath = TEXT("/Game");
	}

	// determine default asset name
	FString DefaultName = LOCTEXT("NewLiveLinkPreset", "NewLiveLinkPreset").ToString();

	FString UniquePackageName;
	FString UniqueAssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(DialogStartPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);

	FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);

	FString UserPackageName;
	FString NewPackageName;

	// get destination for asset
	bool bFilenameValid = false;
	while (!bFilenameValid)
	{
		if (!OpenSaveDialog(DialogStartPath, DialogStartName, InParentWindowOverride, UserPackageName))
		{
			return false;
		}

		NewPackageName = FString::Format(*UserPackageName, FormatArgs);

		FText OutError;
		bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
	}

	ConfigSettings->PresetSaveDir.Path = FPackageName::GetLongPackagePath(UserPackageName);
	ConfigSettings->SaveConfig();
	OutName = MoveTemp(NewPackageName);
	return true;
}

void SLiveLinkClientPanelToolbar::OnSaveAsPreset()
{
	FString PackageName;
	if (!GetSavePresetPackageName(ParentWindowOverride, PackageName))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SaveAsPreset", "Save As Preset"));

	// Saving into a new package
	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* NewPackage = CreatePackage(*PackageName);
	ULiveLinkPreset* NewPreset = NewObject<ULiveLinkPreset>(NewPackage, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);

	if (NewPreset)
	{
		NewPreset->BuildFromClient();

		NewPreset->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(NewPreset);

		FEditorFileUtils::PromptForCheckoutAndSave({ NewPackage }, false, false);
	}
	Client->SetLastSavedPreset(NewPreset);
}


void SLiveLinkClientPanelToolbar::OnImportPreset(const FAssetData& InPreset)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* PresetAssetData = InPreset.GetAsset();
	if (!PresetAssetData)
	{
		FNotificationInfo Info(LOCTEXT("LoadPresetFailed", "Failed to load preset"));
		Info.ExpireDuration = 5.0f;
		Info.Hyperlink = FSimpleDelegate::CreateStatic([]() { FMessageLog("LoadErrors").Open(EMessageSeverity::Info, true); });
		Info.HyperlinkText = LOCTEXT("LoadObjectHyperlink", "Show Message Log");

		FSlateNotificationManager::Get().AddNotification(Info);
		return;
	}

	ULiveLinkPreset* ImportedPreset = Cast<ULiveLinkPreset>(PresetAssetData);
	if (ImportedPreset)
	{
		FScopedTransaction Transaction(LOCTEXT("ImportPreset_Transaction", "Import Live Link Preset"));
		ImportedPreset->ApplyToClientLatent();
	}
	Client->SetLastSavedPreset(ImportedPreset);
}

FReply SLiveLinkClientPanelToolbar::OnSavePreset()
{
	ULiveLinkPreset* CurrentPreset = Client->GetLastSavedPreset();
	if (!CurrentPreset)
	{
		OnSaveAsPreset();
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("SavePreset", "Save Preset"));
	CurrentPreset->BuildFromClient();
	CurrentPreset->MarkPackageDirty();
	FEditorFileUtils::PromptForCheckoutAndSave({ CurrentPreset->GetPackage() }, false, false);

	return FReply::Handled();
}

FReply SLiveLinkClientPanelToolbar::OnRevertChanges()
{
	FText WarningMessage(LOCTEXT("Warning_RevertChanges", "Are you sure you want to revert changes? Your current changes will be discarded."));
	if (EAppReturnType::No == FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage))
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("RevertChanges_Transaction", "Revert Changes"));
	ULiveLinkPreset* CurrentPreset = Client->GetLastSavedPreset();
	if (CurrentPreset)
	{
		CurrentPreset->ApplyToClientLatent();
	}

	return FReply::Handled();
}

bool SLiveLinkClientPanelToolbar::HasLoadedLiveLinkPreset() const
{
	return Client->GetLastSavedPreset() != nullptr;
}


#undef LOCTEXT_NAMESPACE
