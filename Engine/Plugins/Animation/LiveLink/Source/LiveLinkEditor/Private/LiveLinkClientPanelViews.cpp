// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkClientPanelViews.h"

#include "Algo/Accumulate.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "Features/IModularFeatures.h"
#include "Filters/FilterBase.h"
#include "Filters/GenericFilter.h"
#include "Filters/SBasicFilterBar.h"
#include "Framework/Commands/UICommandList.h"
#include "IDetailsView.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkModule.h"
#include "ILiveLinkDeviceModule.h"
#include "Internationalization/Text.h"
#include "LiveLinkClient.h"
#include "LiveLinkClientCommands.h"
#include "LiveLinkDefaultColumns.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability.h"
#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkSettings.h"
#include "LiveLinkSubjectSettings.h"
#include "LiveLinkTypes.h"
#include "LiveLinkVirtualSubject.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SLiveLinkDataView.h"
#include "SPositiveActionButton.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "ToolMenus.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "LiveLinkClientPanel.PanelViews"

// Static Source UI FNames
namespace SourceListUI
{
	static const FName TypeColumnName(TEXT("Type"));
	static const FName MachineColumnName(TEXT("Machine"));
	static const FName StatusColumnName(TEXT("Status"));
	static const FName ActionsColumnName(TEXT("Action"));
};

/**
 * Widget for the filter bar, needs to be a subclass that overrides MakeAddFilterMenu to give the filter bar its own unique menu name,
 * otherwise the editor will get confused with any other basic filter bar used elsewhere.
 */
template<typename TFilterType>
class SLiveLinkFilterBar : public SBasicFilterBar<TFilterType>
{
	using Super = SBasicFilterBar<TFilterType>;

public:

	using FOnFilterChanged = typename SBasicFilterBar<TFilterType>::FOnFilterChanged;
	using FCreateTextFilter = typename SBasicFilterBar<TFilterType>::FCreateTextFilter;

	SLATE_BEGIN_ARGS(SLiveLinkFilterBar)
		{
		}
		SLATE_EVENT(SLiveLinkFilterBar<TFilterType>::FOnFilterChanged, OnFilterChanged)
		SLATE_ARGUMENT(TArray<TSharedRef<FFilterBase<TFilterType>>>, CustomFilters)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		typename SBasicFilterBar<TFilterType>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._CustomFilters = InArgs._CustomFilters;
		Args._UseSectionsForCategories = true;

		SBasicFilterBar<TFilterType>::Construct(Args.FilterPillStyle(EFilterPillStyle::Basic));
	}

private:
	virtual TSharedRef<SWidget> MakeAddFilterMenu() override
	{
		const FName FilterMenuName = "LiveLinkFilterBar.FilterMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					if (UFilterBarContext* Context = InMenu->FindContext<UFilterBarContext>())
					{
						Context->PopulateFilterMenu.ExecuteIfBound(InMenu);
						Context->OnExtendAddFilterMenu.ExecuteIfBound(InMenu);
					}
				}));
		}

		UFilterBarContext* FilterBarContext = NewObject<UFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddFilterMenu::CreateLambda([this](UToolMenu* Menu)
			{
				Super::PopulateCommonFilterSections(Menu);
				Super::PopulateCustomFilters(Menu);
			});
		FToolMenuContext ToolMenuContext(FilterBarContext);

		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	}
};


/** Dialog to create a new virtual subject */
class SVirtualSubjectCreateDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVirtualSubjectCreateDialog) {}

		/** Pointer to the LiveLinkClient instance. */
		SLATE_ARGUMENT(FLiveLinkClient*, LiveLinkClient)
		SLATE_ARGUMENT(TSubclassOf<ULiveLinkVirtualSubject>, DefaultClass)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs)
	{
		static const FName DefaultVirtualSubjectName = TEXT("Virtual");

		bOkClicked = false;
		VirtualSubjectClass = InArgs._DefaultClass;
		bCreatedWithDefaultClass = InArgs._DefaultClass != nullptr;
		LiveLinkClient = InArgs._LiveLinkClient;

		check(LiveLinkClient);

		int32 NumVirtualSubjects = Algo::TransformAccumulate(LiveLinkClient->GetSubjects(true, true), [this](const FLiveLinkSubjectKey& SubjectKey)
			{
				return LiveLinkClient->IsVirtualSubject(SubjectKey) ? 1 : 0;
			}, 0);
		VirtualSubjectName = DefaultVirtualSubjectName;

		if (NumVirtualSubjects > 0)
		{
			VirtualSubjectName = *FString::Printf(TEXT("%s %d"), *DefaultVirtualSubjectName.ToString(), NumVirtualSubjects + 1);
		}
		//Default VirtualSubject Source should always exist
		TArray<FGuid> Sources = LiveLinkClient->GetVirtualSources();
		check(Sources.Num() > 0);
		VirtualSourceGuid = Sources[0];

		TSharedPtr<STextEntryPopup> TextEntry;
		SAssignNew(TextEntry, STextEntryPopup)
			.Label(LOCTEXT("AddVirtualSubjectName", "New Virtual Subject Name"))
			.DefaultText(FText::FromName(VirtualSubjectName))
			.OnTextChanged(this, &SVirtualSubjectCreateDialog::HandleAddVirtualSubjectChanged);

		VirtualSubjectTextWidget = TextEntry;

		ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(SBox)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.AutoHeight()
						[
							TextEntry->AsShared()
						]

						+ SVerticalBox::Slot()
						.FillHeight(VirtualSubjectClass == nullptr ? 1.0 : 0.0)
						[
							SNew(SBorder)
							.Visibility(VirtualSubjectClass == nullptr ? EVisibility::Visible : EVisibility::Collapsed)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.Content()
							[
								SAssignNew(RoleClassPicker, SVerticalBox)
							]
						]

						// Ok/Cancel buttons
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						.Padding(8)
						[
							SNew(SUniformGridPanel)
							.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
							.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
							.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
							+ SUniformGridPanel::Slot(0, 0)
							[
								SNew(SButton)
									.HAlign(HAlign_Center)
									.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
									.OnClicked(this, &SVirtualSubjectCreateDialog::OkClicked)
									.Text(LOCTEXT("AddVirtualSubjectAdd", "Add"))
									.IsEnabled(this, &SVirtualSubjectCreateDialog::IsVirtualSubjectClassSelected)
							]
							+ SUniformGridPanel::Slot(1, 0)
							[
								SNew(SButton)
									.HAlign(HAlign_Center)
									.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
									.OnClicked(this, &SVirtualSubjectCreateDialog::CancelClicked)
									.Text(LOCTEXT("AddVirtualSubjectCancel", "Cancel"))
							]
						]
					]
				]
			];

		MakeRoleClassPicker();
	}

	bool IsVirtualSubjectClassSelected() const
	{
		return VirtualSubjectClass != nullptr;
	}

	bool ConfigureVirtualSubject()
	{
		const double WindowHeight = bCreatedWithDefaultClass ? 100 : 300;
		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("CreateVirtualSubjectCreation", "Create Virtual Subject"))
			.ClientSize(FVector2D(400, WindowHeight))
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			[
				AsShared()
			];

		PickerWindow = Window;

		GEditor->EditorAddModalWindow(Window);

		return bOkClicked;
	}

private:

	/** Class filter to display virtual subjects classes for the role class picker.. */
	class FLiveLinkRoleClassFilter : public IClassViewerFilter
	{
	public:
		FLiveLinkRoleClassFilter() = default;

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			if (InClass->IsChildOf(ULiveLinkVirtualSubject::StaticClass()))
			{
				return InClass->GetDefaultObject<ULiveLinkVirtualSubject>()->GetRole() != nullptr && !InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown | CLASS_Deprecated);
			}
			return false;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InUnloadedClassData->IsChildOf(ULiveLinkVirtualSubject::StaticClass());
		}
	};

	/** Creates the combo menu for the role class */
	void MakeRoleClassPicker()
	{
		// Load the classviewer module to display a class picker
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		// Fill in options
		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;

		Options.ClassFilters.Add(MakeShared<FLiveLinkRoleClassFilter>());

		RoleClassPicker->ClearChildren();
		RoleClassPicker->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("VirtualSubjectRole", "Virtual Subject Role:"))
					.ShadowOffset(FVector2D(1.0f, 1.0f))
			];

		RoleClassPicker->AddSlot()
			[
				ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SVirtualSubjectCreateDialog::OnClassPicked))
			];
	}

	/** Handler for when a parent class is selected */
	void OnClassPicked(UClass* ChosenClass)
	{
		VirtualSubjectClass = ChosenClass;
	}

	/** Handler for when ok is clicked */
	FReply OkClicked()
	{
		if (LiveLinkClient)
		{
			const FLiveLinkSubjectKey NewVirtualSubjectKey(VirtualSourceGuid, VirtualSubjectName);
			LiveLinkClient->AddVirtualSubject(NewVirtualSubjectKey, VirtualSubjectClass);
		}

		CloseDialog(true);

		return FReply::Handled();
	}

	/** Close the role picker window. */
	void CloseDialog(bool bWasPicked = false)
	{
		bOkClicked = bWasPicked;
		if (PickerWindow.IsValid())
		{
			PickerWindow.Pin()->RequestDestroyWindow();
		}
	}

	/** Handler for when cancel is clicked */
	FReply CancelClicked()
	{
		CloseDialog();
		return FReply::Handled();
	}

	/** Handles closing the role picker window when pressing escape. */
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			CloseDialog();
			return FReply::Handled();
		}
		return SWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	/** Handles setting the error when changing the name of a new virtual subject. */
	void HandleAddVirtualSubjectChanged(const FText& NewSubjectName)
	{
		TSharedPtr<STextEntryPopup> VirtualSubjectTextWidgetPin = VirtualSubjectTextWidget.Pin();
		if (VirtualSubjectTextWidgetPin.IsValid())
		{
			TArray<FLiveLinkSubjectKey> SubjectKey = LiveLinkClient->GetSubjects(true, true);
			FName SubjectName = *NewSubjectName.ToString();
			const FLiveLinkSubjectKey ThisSubjectKey(VirtualSourceGuid, SubjectName);

			if (SubjectName.IsNone())
			{
				VirtualSubjectTextWidgetPin->SetError(LOCTEXT("VirtualInvalidName", "Invalid Virtual Subject"));
			}
			else if (SubjectKey.FindByPredicate([ThisSubjectKey](const FLiveLinkSubjectKey& Key) { return Key == ThisSubjectKey; }))
			{
				VirtualSubjectTextWidgetPin->SetError(LOCTEXT("VirtualExistingName", "Subject already exist"));
			}
			else
			{
				VirtualSubjectName = SubjectName;
				VirtualSubjectTextWidgetPin->SetError(FText::GetEmpty());
			}
		}
	}

private:
	/** Cached livelink client. */
	FLiveLinkClient* LiveLinkClient;

	/** Holds the text entry widget to name a virtual subject. */
	TWeakPtr<STextEntryPopup> VirtualSubjectTextWidget;

	/** A pointer to the window that is asking the user to select a role class */
	TWeakPtr<SWindow> PickerWindow;

	/** The container for the role Class picker */
	TSharedPtr<SVerticalBox> RoleClassPicker;

	/** The virtual subject's class */
	TSubclassOf<ULiveLinkVirtualSubject> VirtualSubjectClass;

	/** Selected source guid */
	FGuid VirtualSourceGuid;

	/** The virtual subject's name */
	FName VirtualSubjectName;

	/** True if Ok was clicked */
	bool bOkClicked;

	/** Whether the widget was created with a default class, used to resize the window accordingly. */
	bool bCreatedWithDefaultClass = false;
};

namespace UE::LiveLink
{
// Source creation window is static so that the code in this namespace can be used by the subjects and the sources view. 
// This code could be moved to the subjects view once the SourcesView is deprecated in 5.10
static TSharedPtr<SWindow> SourceCreationWindow;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<IDetailsView> CreateSourcesDetailsView(const TSharedPtr<FLiveLinkSourcesView>& InSourcesView, const TAttribute<bool>& bInReadOnly)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	TSharedPtr<IDetailsView> SettingsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	// todo: use controller here instead of view widget
	SettingsDetailsView->OnFinishedChangingProperties().AddRaw(InSourcesView.Get(), &FLiveLinkSourcesView::OnPropertyChanged);
	SettingsDetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda(
		[bInReadOnly]() {
			return !bInReadOnly.Get();
		}));

	return SettingsDetailsView;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TSharedPtr<IDetailsView> CreateSourcesDetailsView(FOnFinishedChangingProperties InChangedPropertiesDelegate, const TAttribute<bool>& bInReadOnly)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	TSharedPtr<IDetailsView> SettingsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	SettingsDetailsView->OnFinishedChangingProperties() = InChangedPropertiesDelegate;
	SettingsDetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda(
		[bInReadOnly](){
		return !bInReadOnly.Get();
	}));

	return SettingsDetailsView;
}

TSharedPtr<SLiveLinkDataView> CreateSubjectsDetailsView(FLiveLinkClient* InLiveLinkClient, const TAttribute<bool>& bInReadOnly)
{
	return SNew(SLiveLinkDataView, InLiveLinkClient)
		.ReadOnly(bInReadOnly);
}

TSharedPtr<IDetailsView> CreateDevicesDetailsView(FOnFinishedChangingProperties OnFinishedChangingProperties, const TAttribute<bool>& bInReadOnly)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	TSharedPtr<IDetailsView> SettingsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	SettingsDetailsView->OnFinishedChangingProperties() = OnFinishedChangingProperties;
	SettingsDetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda(
		[bInReadOnly]() {
			return !bInReadOnly.Get();
		}));

	return SettingsDetailsView;
}

void OnSourceCreated(TSharedPtr<ILiveLinkSource> NewSource, FString ConnectionString, TSubclassOf<ULiveLinkSourceFactory> Factory)
{
	FLiveLinkClient* LiveLinkClient = (FLiveLinkClient*)&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	if (NewSource.IsValid())
	{
		FGuid NewSourceGuid = LiveLinkClient->AddSource(NewSource);
		if (NewSourceGuid.IsValid())
		{
			if (ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(NewSourceGuid))
			{
				Settings->ConnectionString = ConnectionString;
				Settings->Factory = Factory;
			}
		}
	}

	if (SourceCreationWindow)
	{
		SourceCreationWindow->RequestDestroyWindow();
		SourceCreationWindow.Reset();
	}
}

void OpenCreateMenuWindow(TArray<TObjectPtr<ULiveLinkSourceFactory>> Factories, int32 FactoryIndex)
{
	if (Factories.IsValidIndex(FactoryIndex))
	{
		if (ULiveLinkSourceFactory* FactoryInstance = Factories[FactoryIndex])
		{
			TSharedPtr<SWidget> Widget = FactoryInstance->BuildCreationPanel(ULiveLinkSourceFactory::FOnLiveLinkSourceCreated::CreateStatic(&OnSourceCreated, TSubclassOf<ULiveLinkSourceFactory>(FactoryInstance->GetClass())));
			if (Widget.IsValid())
			{
				FText CreateText = FText::Format(LOCTEXT("CreateSourceLabel", "Create {0} connection"), FactoryInstance->GetSourceDisplayName());
				SourceCreationWindow = SNew(SWindow)
					.Title(CreateText)
					.ClientSize(FVector2D(400, 150))
					.SizingRule(ESizingRule::Autosized)
					.SupportsMinimize(false)
					.SupportsMaximize(false)
					[
						Widget.ToSharedRef()
					];

				GEditor->EditorAddModalWindow(SourceCreationWindow.ToSharedRef());
			}
		}
	}
}

void ExecuteCreateSource(TArray<TObjectPtr<ULiveLinkSourceFactory>> Factories, int32 FactoryIndex)
{
	if (Factories.IsValidIndex(FactoryIndex))
	{
		if (ULiveLinkSourceFactory* FactoryInstance = Factories[FactoryIndex])
		{
			OnSourceCreated(FactoryInstance->CreateSource(FString()), FString(), FactoryInstance->GetClass());
		}
	}
}

void AddVirtualSubject()
{
	FLiveLinkClient* LiveLinkClient = (FLiveLinkClient*)&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	const TSubclassOf<ULiveLinkVirtualSubject> Subclass = GetDefault<ULiveLinkSettings>()->DefaultVirtualSubjectClass;

	TSharedRef<SVirtualSubjectCreateDialog> Dialog =
		SNew(SVirtualSubjectCreateDialog)
		.DefaultClass(Subclass)
		.LiveLinkClient(LiveLinkClient);

	Dialog->ConfigureVirtualSubject();
}

TArray<TObjectPtr<ULiveLinkSourceFactory>> GetAllSourceFactories()
{
	TArray<TObjectPtr<ULiveLinkSourceFactory>> Factories;

	TArray<UClass*> Results;
	GetDerivedClasses(ULiveLinkSourceFactory::StaticClass(), Results, true);
	for (UClass* SourceFactory : Results)
	{
		if (!SourceFactory->HasAllClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists) && CastChecked<ULiveLinkSourceFactory>(SourceFactory->GetDefaultObject())->IsEnabled())
		{
			Factories.Add(NewObject<ULiveLinkSourceFactory>(GetTransientPackage(), SourceFactory));
		}
	}

	Algo::StableSort(Factories, [](ULiveLinkSourceFactory* LHS, ULiveLinkSourceFactory* RHS)
	{
		return LHS->GetSourceDisplayName().CompareTo(RHS->GetSourceDisplayName()) <= 0;
	});

	return Factories;
}

void GenerateSourceMenuEntries(const TArray<TObjectPtr<ULiveLinkSourceFactory>>& Factories, FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("SourceSection", LOCTEXT("Sources", "Live Link Sources"));

	for (int32 FactoryIndex = 0; FactoryIndex < Factories.Num(); ++FactoryIndex)
	{
		ULiveLinkSourceFactory* FactoryInstance = Factories[FactoryIndex];
		if (FactoryInstance)
		{
			ULiveLinkSourceFactory::EMenuType MenuType = FactoryInstance->GetMenuType();

			if (MenuType == ULiveLinkSourceFactory::EMenuType::SubPanel)
			{
				MenuBuilder.AddMenuEntry(
					FactoryInstance->GetSourceDisplayName(),
					FactoryInstance->GetSourceTooltip(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(OpenCreateMenuWindow, Factories, FactoryIndex)
					),
					NAME_None,
					EUserInterfaceActionType::Button);
			}
			else if (MenuType == ULiveLinkSourceFactory::EMenuType::MenuEntry)
			{
				MenuBuilder.AddMenuEntry(
					FactoryInstance->GetSourceDisplayName(),
					FactoryInstance->GetSourceTooltip(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(ExecuteCreateSource, Factories, FactoryIndex)
					),
					NAME_None,
					EUserInterfaceActionType::Button);
			}
			else
			{
				MenuBuilder.AddMenuEntry(
					FactoryInstance->GetSourceDisplayName(),
					FactoryInstance->GetSourceTooltip(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction::CreateLambda([]() { return false; })
					),
					NAME_None,
					EUserInterfaceActionType::Button);
			}
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("VirtualSourceSection", LOCTEXT("VirtualSources", "Live Link VirtualSubject Sources"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddVirtualSubject", "Add Virtual Subject"),
		LOCTEXT("AddVirtualSubject_Tooltip", "Adds a new virtual subject to Live Link. Instead of coming from a source a virtual subject is a combination of 2 or more real subjects"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&AddVirtualSubject)
		),
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.EndSection();
}

} // namespace UE::LiveLink

FGuid FLiveLinkSourceUIEntry::GetGuid() const
{
	return EntryGuid;
}
FText FLiveLinkSourceUIEntry::GetSourceType() const
{
	return Client->GetSourceType(EntryGuid);
}
FText FLiveLinkSourceUIEntry::GetMachineName() const
{
	return Client->GetSourceMachineName(EntryGuid);
}
FText FLiveLinkSourceUIEntry::GetStatus() const
{
	return Client->GetSourceStatus(EntryGuid);
}
ULiveLinkSourceSettings* FLiveLinkSourceUIEntry::GetSourceSettings() const
{
	return Client->GetSourceSettings(EntryGuid);
}
void FLiveLinkSourceUIEntry::RemoveFromClient() const
{
	Client->RemoveSource(EntryGuid);
}
FText FLiveLinkSourceUIEntry::GetDisplayName() const
{
	return GetSourceType();
}
FText FLiveLinkSourceUIEntry::GetToolTip() const
{
	return Client->GetSourceToolTip(EntryGuid);
}

FLiveLinkSubjectUIEntry::FLiveLinkSubjectUIEntry(const FLiveLinkSubjectKey& InSubjectKey, FLiveLinkClient* InClient, bool bInIsSource)
	: SubjectKey(InSubjectKey)
	, Client(InClient)
{
	if (Client)
	{
		bIsVirtualSubject = Client->IsVirtualSubject(InSubjectKey);

		// LiveLinkClient doesn't have a concept of playback sources since they're a LiveLinkHub concept.
		// We could replace this by allowing sources to modify their appearance in the source list.
		bIsPlaybackSource = IsSource() && Client->GetSourceSettings(SubjectKey.Source)->GetName().Contains(TEXT("Playback"));
	}

	Type = bInIsSource ? ELiveLinkEntryType::Source : ELiveLinkEntryType::Subject;
}

FLiveLinkSubjectUIEntry::FLiveLinkSubjectUIEntry(FLiveLinkClient* InClient, ELiveLinkEntryType InType)
	: Client(InClient)
	, Type(InType)
{
	if (Client)
	{
		// LiveLinkClient doesn't have a concept of playback sources since they're a LiveLinkHub concept.
		// We could replace this by allowing sources to modify their appearance in the source list.
		bIsPlaybackSource = IsSource() && Client->GetSourceSettings(SubjectKey.Source)->GetName().Contains(TEXT("Playback"));
	}
}

bool FLiveLinkSubjectUIEntry::IsSubject() const
{
	return Type == ELiveLinkEntryType::Subject;
}

bool FLiveLinkSubjectUIEntry::IsSource() const
{
	return Type == ELiveLinkEntryType::Source;
}

bool FLiveLinkSubjectUIEntry::IsDevice() const
{
	return Type == ELiveLinkEntryType::Device;
}

bool FLiveLinkSubjectUIEntry::IsVirtualSubject() const
{
	return IsSubject() && bIsVirtualSubject;
}

UObject* FLiveLinkSubjectUIEntry::GetSettings() const
{
	if (IsSource())
	{
		return Client->GetSourceSettings(SubjectKey.Source);
	}
	else if (IsSubject())
	{
		return Client->GetSubjectSettings(SubjectKey);
	}
	else
	{
		return WeakDevice.Get();
	}
}

bool FLiveLinkSubjectUIEntry::IsSubjectEnabled() const
{
	return IsSubject() ? Client->IsSubjectEnabled(SubjectKey, false) : false;
}

bool FLiveLinkSubjectUIEntry::IsSubjectValid() const
{
	return IsSubject() ? Client->IsSubjectValid(SubjectKey) : false;
}

bool FLiveLinkSubjectUIEntry::IsPlaybackSource() const
{
	return bIsPlaybackSource;
}

void FLiveLinkSubjectUIEntry::SetSubjectEnabled(bool bIsEnabled)
{
	if (IsSubject())
	{
		Client->SetSubjectEnabled(SubjectKey, bIsEnabled);
	}
}

FText FLiveLinkSubjectUIEntry::GetItemText() const
{
	switch (Type)
	{
	case ELiveLinkEntryType::SourcesHeader:
		return LOCTEXT("SourcesLabel", "Sources");
	case ELiveLinkEntryType::DevicesHeader:
		return LOCTEXT("DevicesLabel", "Devices");
	case ELiveLinkEntryType::Subject:
		return Client->GetSubjectDisplayName(SubjectKey);
	case ELiveLinkEntryType::Source:
		return Client->GetSourceNameOverride(SubjectKey);
	case ELiveLinkEntryType::Device:
	{
		FText Name;
		if (TStrongObjectPtr<ULiveLinkDevice> Device = WeakDevice.Pin())
		{
			Name = Device->GetDisplayName();
		}
		return Name;
	}
	default:
		checkNoEntry();
		return FText::GetEmpty();
	}
}

FText FLiveLinkSubjectUIEntry::GetMachineName() const
{
	if (IsSource())
	{
		return Client->GetSourceMachineName(SubjectKey.Source);
	}

	return FText::GetEmpty();
}

TSubclassOf<ULiveLinkRole> FLiveLinkSubjectUIEntry::GetItemRole() const
{
	return IsSubject() ? Client->GetSubjectRole_AnyThread(SubjectKey) : TSubclassOf<ULiveLinkRole>();
}

TSubclassOf<ULiveLinkRole> FLiveLinkSubjectUIEntry::GetItemTranslatedRole() const
{
	return IsSubject() ? Client->GetSubjectTranslatedRole_AnyThread(SubjectKey) : TSubclassOf<ULiveLinkRole>();
}

void FLiveLinkSubjectUIEntry::RemoveFromClient() const
{
	switch (Type)
	{
	case ELiveLinkEntryType::Subject:
		Client->RemoveSubject_AnyThread(SubjectKey);
		break;
	case ELiveLinkEntryType::Source:
		Client->RemoveSource(SubjectKey.Source);
		break;
	case ELiveLinkEntryType::Device:
	{
		if (ULiveLinkDevice* Device = WeakDevice.Get())
		{
			ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
			Subsystem->RemoveDevice(Device);
		}
		break;
	}
	}
}

void FLiveLinkSubjectUIEntry::PauseSubject()
{
	if (IsPaused())
	{
		Client->UnpauseSubject_AnyThread(SubjectKey.SubjectName);
	}
	else
	{
		Client->PauseSubject_AnyThread(SubjectKey.SubjectName);
	}
}

bool FLiveLinkSubjectUIEntry::IsPaused() const
{
	return Client->GetSubjectState(SubjectKey.SubjectName) == ELiveLinkSubjectState::Paused;
}

void FLiveLinkSubjectUIEntry::GetFilterText(TArray<FString>& OutStrings) const
{
	OutStrings.Add(GetItemText().ToString());
}

class SLiveLinkClientPanelSubjectRow : public SMultiColumnTableRow<FLiveLinkSubjectUIEntryPtr>
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkClientPanelSubjectRow) {}
	/** The list item for this row */
	SLATE_ARGUMENT(FLiveLinkSubjectUIEntryPtr, Entry)
	SLATE_ATTRIBUTE(bool, ReadOnly)
	SLATE_END_ARGS()


	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		EntryPtr = Args._Entry;
		bReadOnly = Args._ReadOnly;

		TSharedPtr<FSlateStyleSet> StyleSet = ILiveLinkModule::Get().GetStyle();

		OkayIcon = StyleSet->GetBrush("LiveLink.Subject.Okay");
		WarningIcon = StyleSet->GetBrush("LiveLink.Subject.Warning");
		PauseIcon = StyleSet->GetBrush("LiveLink.Subject.Pause");

		SMultiColumnTableRow<FLiveLinkSubjectUIEntryPtr>::Construct(
			FSuperRowType::FArguments()
			.Style(&FAppStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			.Padding(1.0f),
			OwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TDelegate<bool()> IsRowSelected = TDelegate<bool()>::CreateLambda([WeakRow = TWeakPtr<SLiveLinkClientPanelSubjectRow>(StaticCastSharedRef<SLiveLinkClientPanelSubjectRow>(AsShared()))]() -> bool
		{
			if (TSharedPtr<SLiveLinkClientPanelSubjectRow> Row = WeakRow.Pin())
			{
				return Row->IsSelected();
			}
			return false;
		});

		constexpr double BaseIndent = 12.0f;
		if (ColumnName == UE::LiveLink::DefaultColumn::Action)
		{
			if (EntryPtr->IsSubject())
			{
				return SNew(SCheckBox)
					.Visibility(this, &SLiveLinkClientPanelSubjectRow::GetVisibilityFromReadOnly)
					.IsChecked(MakeAttributeSP(this, &SLiveLinkClientPanelSubjectRow::GetSubjectEnabled))
					.OnCheckStateChanged(this, &SLiveLinkClientPanelSubjectRow::OnEnabledChanged);
			}
			else if (EntryPtr->IsDevice())
			{
				if (TStrongObjectPtr<ULiveLinkDevice> Device = EntryPtr->WeakDevice.Pin())
				{
					FLiveLinkDeviceWidgetArguments Args;
					Args.IsRowSelected = IsRowSelected;
					return Device->GenerateWidgetForColumn(UE::LiveLink::DefaultColumn::Action, Args);
				}
			}
			else
			{
				const float IndentAmount = EntryPtr->IsSource() ? BaseIndent : 0.f;
				return SNew(SBox)
				.Padding(2.f, 0.f, 0.f, 0.f)
				.HAlign(HAlign_Left)
				[
					SNew(SExpanderArrow, SharedThis(this)).IndentAmount(IndentAmount)
				];
			}
		}
		else if (ColumnName == UE::LiveLink::DefaultColumn::Name)
		{
			FMargin Padding = EntryPtr->IsSource() || EntryPtr->IsDevice() ? FMargin(BaseIndent, 0, 0, 0) : FMargin(0);
			if (EntryPtr->IsSubject())
			{
				Padding = FMargin(BaseIndent * 2, 0, 0, 0);
			}

			TAttribute<FSlateFontInfo> Font;
			if (EntryPtr->Type == ELiveLinkEntryType::SourcesHeader || EntryPtr->Type == ELiveLinkEntryType::DevicesHeader)
			{
				Font = FCoreStyle::GetDefaultFontStyle("Bold", 10);
			}

			if (EntryPtr->IsPlaybackSource())
			{
				Font = FCoreStyle::GetDefaultFontStyle("Italic", 10);
			}
			
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(Padding)
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Font(Font)
					.ColorAndOpacity(this, &SLiveLinkClientPanelSubjectRow::GetSubjectTextColor)
					.Text(MakeAttributeSP(this, &SLiveLinkClientPanelSubjectRow::GetItemText))
				];
		}
		else if (ColumnName == UE::LiveLink::DefaultColumn::Machine)
		{
			if (EntryPtr->IsSource())
			{
				return SNew(STextBlock)
					.ColorAndOpacity(this, &SLiveLinkClientPanelSubjectRow::GetSubjectTextColor)
					.Text(MakeAttributeSP(this, &SLiveLinkClientPanelSubjectRow::GetItemMachineName));
			}
			else if (EntryPtr->IsDevice()) 
			{
				if (TStrongObjectPtr<ULiveLinkDevice> Device = EntryPtr->WeakDevice.Pin())
				{
					FLiveLinkDeviceWidgetArguments Args;
					Args.IsRowSelected = IsRowSelected;
					return Device->GenerateWidgetForColumn(UE::LiveLink::DefaultColumn::Machine, Args);
				}
			}
		}
		else if (ColumnName == UE::LiveLink::DefaultColumn::Role)
		{
			TAttribute<FText> RoleAttribute;
			
			if (EntryPtr->IsSubject())
			{
				RoleAttribute = MakeAttributeSP(this, &SLiveLinkClientPanelSubjectRow::GetItemRole);
			}
			else if (EntryPtr->IsPlaybackSource())
			{
				RoleAttribute = LOCTEXT("Playback", "Playback");
			}
			else
			{
				RoleAttribute = FText::GetEmpty();
			}

			TAttribute<FSlateColor> TextColorAttribute = MakeAttributeSP(this, &SLiveLinkClientPanelSubjectRow::GetSubjectTextColor);
			if (EntryPtr->IsPlaybackSource())
			{
				TextColorAttribute = FSlateColor::UseSubduedForeground();
			}

			return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity(TextColorAttribute)
				.Text(RoleAttribute)
			];
		}
		else if (ColumnName == UE::LiveLink::DefaultColumn::TranslatedRole)
		{
			TAttribute<FText> TranslatedRoleAttribute = MakeAttributeSP(this, &SLiveLinkClientPanelSubjectRow::GetItemTranslatedRole);

			return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SLiveLinkClientPanelSubjectRow::GetSubjectTextColor)
					.Text(EntryPtr->IsSubject() ? TranslatedRoleAttribute : FText::GetEmpty())
			];
		}
		else if (ColumnName == UE::LiveLink::DefaultColumn::Status)
		{
			if (EntryPtr->IsDevice())
			{
				if (TStrongObjectPtr<ULiveLinkDevice> Device = EntryPtr->WeakDevice.Pin())
				{
					FLiveLinkDeviceWidgetArguments Args;
					Args.IsRowSelected = IsRowSelected;
					return Device->GenerateWidgetForColumn(UE::LiveLink::DefaultColumn::Status, Args);
				}
			}
			else if (EntryPtr->IsPlaybackSource())
			{
				return SNew(SBox)
					.WidthOverride(12.0)
					.HeightOverride(12.0)
					[
						SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.ToolTipText(LOCTEXT("PlaybackSourceToolTip", "This source is being driven by a recording."))
							.Image(FAppStyle::Get().GetBrush("Animation.Forward"))
					];
			}
			else
			{
				return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
						.Image(this, &SLiveLinkClientPanelSubjectRow::GetSubjectIcon)
						.ToolTipText(this, &SLiveLinkClientPanelSubjectRow::GetSubjectIconToolTip)
				];
			}
		}

		return SNullWidget::NullWidget;
	}

private:
	ECheckBoxState GetSubjectEnabled() const { return EntryPtr->IsSubjectEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }
	void OnEnabledChanged(ECheckBoxState NewState) { EntryPtr->SetSubjectEnabled(NewState == ECheckBoxState::Checked); }
	FText GetItemText() const { return EntryPtr->GetItemText(); }
	FText GetItemMachineName() const { return EntryPtr->GetMachineName(); }
	FText GetItemRole() const
	{
		TSubclassOf<ULiveLinkRole> Role = EntryPtr->GetItemRole();
		if (Role)
		{
			return Role->GetDefaultObject<ULiveLinkRole>()->GetDisplayName();
		}
		return FText::GetEmpty();
	}

	// If role is translated, put the original role in the extra info.
	FText GetItemTranslatedRole() const
	{
		TSubclassOf<ULiveLinkRole> TranslatedRole = EntryPtr->GetItemTranslatedRole();

		if (TranslatedRole.Get())
		{
			return TranslatedRole->GetDefaultObject<ULiveLinkRole>()->GetDisplayName();
		}
		return FText::GetEmpty();
	}

	/** Get widget visibility according to whether or not the panel is in read-only mode. */
	EVisibility GetVisibilityFromReadOnly() const
	{
		return bReadOnly.Get() ? EVisibility::Hidden : EVisibility::Visible;
	}

	/** Get the icon for a subject's status. */
	const FSlateBrush* GetSubjectIcon() const
	{
		const FSlateBrush* IconBrush = nullptr;

		if (EntryPtr->IsSubjectEnabled())
		{
			if (!EntryPtr->IsPaused())
			{
				IconBrush = EntryPtr->IsSubjectValid() ? OkayIcon : WarningIcon;
			}
			else
			{
				IconBrush = PauseIcon;
			}
		}
		else
		{
			// No icon for disabled subjects, we rely on setting the text color to subdued foreground.
		}

		return IconBrush;
	}

	/** Get the tooltip for a subject's status icon. */
	FText GetSubjectIconToolTip() const
	{
		FText IconToolTip;

		if (EntryPtr->IsSubjectEnabled())
		{
			IconToolTip = EntryPtr->IsSubjectValid() ? LOCTEXT("ValidSubjectToolTip", "Subject is operating normally.") : LOCTEXT("InvalidSubjectToolTip", "Subject is invalid.");
		}
		else
		{
			IconToolTip = LOCTEXT("SubjectDisabledToolTip", "Subject is disabled.");
		}

		return IconToolTip;
	}

	/** Get the text color for a subject. */
	FSlateColor GetSubjectTextColor() const
	{
		FSlateColor TextColor = FSlateColor::UseForeground();

		if (EntryPtr->IsSubject() && !EntryPtr->IsSubjectEnabled())
		{
			TextColor = FSlateColor::UseSubduedForeground();
		}

		return TextColor;
	}

	FLiveLinkSubjectUIEntryPtr EntryPtr;

	/** Returns whether the panel is in read-only mode. */
	TAttribute<bool> bReadOnly;

	//~ Status icons
	const FSlateBrush* OkayIcon = nullptr;
	const FSlateBrush* WarningIcon = nullptr;
	const FSlateBrush* PauseIcon = nullptr;
};

class SLiveLinkClientPanelSourcesRow : public SMultiColumnTableRow<FLiveLinkSourceUIEntryPtr>
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkClientPanelSourcesRow) {}
	/** The list item for this row */
		SLATE_ARGUMENT(FLiveLinkSourceUIEntryPtr, Entry)
		SLATE_ATTRIBUTE(bool, ReadOnly)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		EntryPtr = Args._Entry;
		bReadOnly = Args._ReadOnly;

		SMultiColumnTableRow<FLiveLinkSourceUIEntryPtr>::Construct(
			FSuperRowType::FArguments()
			.Style(&FAppStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			.Padding(1.0f),
			OwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == SourceListUI::TypeColumnName)
		{
			return SNew(STextBlock)
				.Text(EntryPtr->GetSourceType());
		}
		else if (ColumnName == SourceListUI::MachineColumnName)
		{
			return SNew(STextBlock)
				.Text(MakeAttributeSP(this, &SLiveLinkClientPanelSourcesRow::GetMachineName));
		}
		else if (ColumnName == SourceListUI::StatusColumnName)
		{
			return SNew(STextBlock)
				.Text(MakeAttributeSP(this, &SLiveLinkClientPanelSourcesRow::GetSourceStatus));
		}
		else if (ColumnName == SourceListUI::ActionsColumnName)
		{
			return SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility(this, &SLiveLinkClientPanelSourcesRow::GetVisibilityFromReadOnly)
				.OnClicked(this, &SLiveLinkClientPanelSourcesRow::OnRemoveClicked)
				.ToolTipText(LOCTEXT("RemoveSource", "Remove selected live link source"))
				.ContentPadding(0.f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				];
		}

		return SNullWidget::NullWidget;
	}

	FText GetToolTipText() const
	{
		return EntryPtr->GetToolTip();
	}

private:
	FText GetMachineName() const
	{
		return EntryPtr->GetMachineName();
	}

	FText GetSourceStatus() const
	{
		return EntryPtr->GetStatus();
	}

	FReply OnRemoveClicked()
	{
		EntryPtr->RemoveFromClient();
		return FReply::Handled();
	}

	/** Get widget visibility according to whether or not the panel is in read-only mode. */
	EVisibility GetVisibilityFromReadOnly() const
	{
		return bReadOnly.Get() ? EVisibility::Collapsed : EVisibility::Visible;
	}

private:
	FLiveLinkSourceUIEntryPtr EntryPtr;

	/** Attribute used to query whether the panel is in read only mode or not. */
	TAttribute<bool> bReadOnly;
};

FLiveLinkSourcesView::FLiveLinkSourcesView(FLiveLinkClient* InLiveLinkClient, TSharedPtr<FUICommandList> InCommandList, TAttribute<bool> bInReadOnly, FOnSourceSelectionChanged InOnSourceSelectionChanged)
	: Client(InLiveLinkClient)
	, OnSourceSelectionChangedDelegate(MoveTemp(InOnSourceSelectionChanged))
	, bReadOnly(MoveTemp(bInReadOnly))
{
	Factories = UE::LiveLink::GetAllSourceFactories();
	CreateSourcesListView(InCommandList);
}

FLiveLinkSourcesView::~FLiveLinkSourcesView()
{
	FTSTicker::RemoveTicker(TickHandle);
}

TSharedRef<ITableRow> FLiveLinkSourcesView::MakeSourceListViewWidget(FLiveLinkSourceUIEntryPtr Entry, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SLiveLinkClientPanelSourcesRow, OwnerTable)
		.Entry(Entry)
		.ReadOnly(bReadOnly)
		.ToolTipText_Lambda([Entry]() { return Entry->GetToolTip(); });
}

void FLiveLinkSourcesView::OnSourceListSelectionChanged(FLiveLinkSourceUIEntryPtr Entry, ESelectInfo::Type SelectionType) const
{
	OnSourceSelectionChangedDelegate.Execute(Entry, SelectionType);
}

void FLiveLinkSourcesView::CreateSourcesListView(const TSharedPtr<FUICommandList>& InCommandList)
{
	SAssignNew(HostWidget, SVerticalBox)
	+SVerticalBox::Slot()
	.Padding(FMargin(4.0, 4.0, 4.0, 6.0))
	.MinHeight(29)
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0, 2.0))
		.FillWidth(1.0f)
		[
			SAssignNew(FilterSearchBox, SLiveLinkFilterSearchBox<FLiveLinkSourceUIEntryPtr>)
			.ItemSource(&SourceData)
			.OnUpdateFilteredList_Lambda([this](const TArray<FLiveLinkSourceUIEntryPtr>& FilteredItems)
				{
					FilteredList = FilteredItems;
					SourcesListView->RequestListRefresh();
				})
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0, 2.0))
		.AutoWidth()
		[
			SNew(SPositiveActionButton)
			.OnGetMenuContent_Raw(this, &FLiveLinkSourcesView::OnGenerateSourceMenu)
			.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			.Text(LOCTEXT("AddSource", "Add Source"))
			.ToolTipText(LOCTEXT("AddSource_ToolTip", "Add a new Live Link source"))
		]
	]
	+SVerticalBox::Slot()
	.VAlign(VAlign_Fill)
	[
		SAssignNew(SourcesListView, SLiveLinkSourceListView, bReadOnly)
			.ListItemsSource(&FilteredList)
			.SelectionMode(ESelectionMode::Single)
			.ScrollbarVisibility(EVisibility::Visible)
			.OnGenerateRow_Raw(this, &FLiveLinkSourcesView::MakeSourceListViewWidget)
			.OnContextMenuOpening_Raw(this, &FLiveLinkSourcesView::OnSourceConstructContextMenu, InCommandList)
			.OnSelectionChanged_Raw(this, &FLiveLinkSourcesView::OnSourceListSelectionChanged)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(SourceListUI::TypeColumnName)
				.FillWidth(25.f)
				.DefaultLabel(LOCTEXT("TypeColumnHeaderName", "Source Type"))
				+ SHeaderRow::Column(SourceListUI::MachineColumnName)
				.FillWidth(25.f)
				.DefaultLabel(LOCTEXT("MachineColumnHeaderName", "Source Machine"))
				+ SHeaderRow::Column(SourceListUI::StatusColumnName)
				.FillWidth(50.f)
				.DefaultLabel(LOCTEXT("StatusColumnHeaderName", "Status"))
				+ SHeaderRow::Column(SourceListUI::ActionsColumnName)
				.ManualWidth(20.f)
				.DefaultLabel(FText())
			)
	];
}

TSharedPtr<SWidget> FLiveLinkSourcesView::OnSourceConstructContextMenu(TSharedPtr<FUICommandList> InCommandList)
{
	if (bReadOnly.Get())
	{
		return nullptr;
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	MenuBuilder.BeginSection(TEXT("Remove"));
	{
		if (CanRemoveSource())
		{
			MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().RemoveSource);
		}
		MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().RemoveAllSources);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FLiveLinkSourcesView::GetWidget()
{
	return HostWidget.ToSharedRef();
}

void FLiveLinkSourcesView::RefreshSourceData(bool bRefreshUI)
{
	SourceData.Reset();
	FilteredList.Reset();

	for (FGuid SourceGuid : Client->GetDisplayableSources())
	{
		SourceData.Add(MakeShared<FLiveLinkSourceUIEntry>(SourceGuid, Client));
	}
	SourceData.Sort([](const FLiveLinkSourceUIEntryPtr& LHS, const FLiveLinkSourceUIEntryPtr& RHS) { return LHS->GetMachineName().CompareTo(RHS->GetMachineName()) < 0; });

	if (bRefreshUI && FilterSearchBox)
	{
		FilterSearchBox->Update();
	}
}

void FLiveLinkSourcesView::HandleRemoveSource()
{
	TArray<FLiveLinkSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);

	if (Selected.Num() > 0)
	{
		Selected[0]->RemoveFromClient();
	}
}

bool FLiveLinkSourcesView::CanRemoveSource()
{
	return SourcesListView->GetNumItemsSelected() > 0;
}

void FLiveLinkSourcesView::OnPropertyChanged(const FPropertyChangedEvent& InEvent)
{
	TArray<FLiveLinkSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	for (FLiveLinkSourceUIEntryPtr Item : Selected)
	{
		Client->OnPropertyChanged(Item->GetGuid(), InEvent);
	}
}

FLiveLinkSubjectsView::FLiveLinkSubjectsView(FOnSubjectSelectionChanged InOnSubjectSelectionChanged, const TSharedPtr<FUICommandList>& InCommandList, TAttribute<bool> bInReadOnly)
	: SubjectSelectionChangedDelegate(InOnSubjectSelectionChanged)
	, bReadOnly(MoveTemp(bInReadOnly))
	, bShowDevices(false)
{
	CreateCombinedTreeView(InCommandList);
}

FLiveLinkSubjectsView::FLiveLinkSubjectsView(FOnSubjectSelectionChanged InOnSubjectSelectionChanged, const TSharedPtr<FUICommandList>& InCommandList, const FSubjectsViewArgs& InArgs)
	: SubjectSelectionChangedDelegate(InOnSubjectSelectionChanged)
	, bReadOnly(InArgs.bInReadOnly)
	, bShowDevices(InArgs.bInShowDevices)
{
	CreateCombinedTreeView(InCommandList);
}

void FLiveLinkSubjectsView::OnSubjectSelectionChanged(FLiveLinkSubjectUIEntryPtr SubjectEntry, ESelectInfo::Type SelectInfo)
{
	SubjectSelectionChangedDelegate.Execute(SubjectEntry, SelectInfo);
}

TSharedPtr<SWidget> FLiveLinkSubjectsView::OnCombinedListOpenContextMenu(TSharedPtr<FUICommandList> InCommandList)
{
	if (bReadOnly.Get())
	{
		return nullptr;
	}

	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	MenuBuilder.BeginSection(TEXT("Remove"));
	{
		if (CanRemoveSubject())
		{
			MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().RemoveSubject);
		}

		if (CanRemoveDevice())
		{
			MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().RemoveDevice);
		}

		if (CanRemoveSource())
		{
			MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().RemoveSource);
		}

		if (HasAnySources())
		{
			MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().RemoveAllSources);
		}
	}
	MenuBuilder.EndSection();

	if (CanPauseSubject())
	{
		MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().PauseSubject, NAME_None, TAttribute<FText>::CreateSP(this, &FLiveLinkSubjectsView::GetPauseSubjectLabel), TAttribute<FText>::CreateSP(this, &FLiveLinkSubjectsView::GetPauseSubjectToolTip));
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<ITableRow> FLiveLinkSubjectsView::MakeTreeRowWidget(FLiveLinkSubjectUIEntryPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SLiveLinkClientPanelSubjectRow, OwnerTable)
		.Entry(InInfo)
		.ReadOnly(bReadOnly);
}

void FLiveLinkSubjectsView::GetChildrenForInfo(FLiveLinkSubjectUIEntryPtr InInfo, TArray< FLiveLinkSubjectUIEntryPtr >& OutChildren)
{
	if (InInfo)
	{
		OutChildren.Reserve(InInfo->Children.Num());

		for (const FLiveLinkSubjectUIEntryPtr& Entry : InInfo->Children)
		{
			if (Entry && !Entry->bFilteredOut)
			{
				OutChildren.Add(Entry);
			}
		}
	}
}

TSharedPtr<SWidget> FLiveLinkSubjectsView::OnOpenVirtualSubjectContextMenu(TSharedPtr<FUICommandList> InCommandList)
{
	if (bReadOnly.Get())
	{
		return nullptr;
	}

	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	MenuBuilder.BeginSection(TEXT("Remove"));
	{
		if (CanRemoveSubject())
		{
			MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().RemoveSubject);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddMenuEntry(FLiveLinkClientCommands::Get().PauseSubject, NAME_None, TAttribute<FText>::CreateSP(this, &FLiveLinkSubjectsView::GetPauseSubjectLabel), TAttribute<FText>::CreateSP(this, &FLiveLinkSubjectsView::GetPauseSubjectToolTip));

	return MenuBuilder.MakeWidget();
}

bool FLiveLinkSubjectsView::CanRemoveSubject() const
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsTreeView->GetSelectedItems(Selected);

	for (const FLiveLinkSubjectUIEntryPtr& Entry : Selected)
	{
		if (Entry && Entry->IsVirtualSubject())
		{
			return true;
		}
	}

	return false;
}

bool FLiveLinkSubjectsView::CanRemoveDevice() const
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsTreeView->GetSelectedItems(Selected);

	for (const FLiveLinkSubjectUIEntryPtr& Entry : Selected)
	{
		if (Entry && Entry->IsDevice())
		{
			return true;
		}
	}

	return false;
}

bool FLiveLinkSubjectsView::CanRemoveSource() const
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsTreeView->GetSelectedItems(Selected);

	for (const FLiveLinkSubjectUIEntryPtr& Entry : Selected)
	{
		// Allow removing souce when right clicking a subject.
		if (Entry && (Entry->IsSubject() || Entry->IsSource()))
		{
			return true;
		}
	}

	return false;
}


bool FLiveLinkSubjectsView::HasAnySources() const
{
	for (const FLiveLinkSubjectUIEntryPtr& Entry : FilteredList)
	{
		if (Entry && Entry->IsSource())
		{
			return true;
		}
	}

	return false;
}

void FLiveLinkSubjectsView::RefreshSubjects()
{
	TArray<FSavedSelection> SavedSelection;
	SaveSelection(SavedSelection);

	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		if (ILiveLinkClient* Client = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName))
		{
			FLiveLinkSubjectUIEntryPtr SourcesHeader;
			FLiveLinkSubjectUIEntryPtr DevicesHeader;

			TArray<FLiveLinkSubjectKey> SubjectKeys = Client->GetSubjects(true, true);
			TArray<FGuid> Sources = Client->GetSources();

			TMap<FGuid, FLiveLinkSubjectUIEntryPtr> SourceItems;
			FLiveLinkClient* ClientPtr = static_cast<FLiveLinkClient*>(Client);

			SubjectData.Reset();

			// Lazily create a header (if it doesn't exist yet) and add Child under it.
			auto AddChildUnderHeader = [this](FLiveLinkSubjectUIEntryPtr& Header, ELiveLinkEntryType HeaderType, FLiveLinkClient* InClient, const FLiveLinkSubjectUIEntryPtr& Child)
			{
				if (!Header)
				{
					Header = MakeShared<FLiveLinkSubjectUIEntry>(InClient, HeaderType);
					SubjectData.Add(Header);
				}
				Header->Children.Add(Child);
			};

			// Find an existing source entry or create a new one under the SourcesHeader.
			auto GetOrCreateSource = [&](const FGuid& SourceGuid, const FLiveLinkSubjectKey& SubjectKey) -> FLiveLinkSubjectUIEntryPtr
			{
				if (FLiveLinkSubjectUIEntryPtr* Existing = SourceItems.Find(SourceGuid))
				{
					return *Existing;
				}
				constexpr bool bIsSource = true;
				FLiveLinkSubjectUIEntryPtr Source = MakeShared<FLiveLinkSubjectUIEntry>(SubjectKey, ClientPtr, bIsSource);
				AddChildUnderHeader(SourcesHeader, ELiveLinkEntryType::SourcesHeader, ClientPtr, Source);
				SourceItems.Add(SourceGuid, Source);
				SubjectsTreeView->SetItemExpansion(Source, true);
				return Source;
			};

			// Add Subjects & Sources
			for (const FLiveLinkSubjectKey& SubjectKey : SubjectKeys)
			{
				FLiveLinkSubjectUIEntryPtr Source = GetOrCreateSource(SubjectKey.Source, SubjectKey);

				FLiveLinkSubjectUIEntryPtr SubjectEntry = MakeShared<FLiveLinkSubjectUIEntry>(SubjectKey, ClientPtr);
				Source->Children.Add(SubjectEntry);
			}

			// Add sources that haven't been added through subjects.
			// @Note: This pass must come after subjects were added, because subjects can override
			// their source name, so we add them in the previous pass to avoid having to rename them.
			for (const FGuid& SourceId : Sources)
			{
				GetOrCreateSource(SourceId, FLiveLinkSubjectKey{ SourceId, NAME_None });
			}

			// Add Devices (Only visible in LiveLinkHub)
			if (bShowDevices)
			{
				if (ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>())
				{
					const TMap<FGuid, TObjectPtr<ULiveLinkDevice>>& DeviceMap = Subsystem->GetDeviceMap();
					for (const TPair<FGuid, TObjectPtr<ULiveLinkDevice>>& DevicePair : DeviceMap)
					{
						FLiveLinkSubjectUIEntryPtr Entry = MakeShared<FLiveLinkSubjectUIEntry>(ClientPtr, ELiveLinkEntryType::Device);
						Entry->WeakDevice = DevicePair.Value;
						AddChildUnderHeader(DevicesHeader, ELiveLinkEntryType::DevicesHeader, ClientPtr, Entry);
					}
				}
			}

			// Sort all levels of the tree.
			auto SortPredicate = [](const FLiveLinkSubjectUIEntryPtr& LHS, const FLiveLinkSubjectUIEntryPtr& RHS) { return LHS->GetItemText().CompareTo(RHS->GetItemText()) < 0; };
			SubjectData.Sort(SortPredicate);
			for (FLiveLinkSubjectUIEntryPtr& Header : SubjectData)
			{
				Header->Children.Sort(SortPredicate);
				for (const FLiveLinkSubjectUIEntryPtr& Child : Header->Children)
				{
					Child->Children.Sort(SortPredicate);
				}
			}

			RestoreSelection(SavedSelection);

			if (FilterSearchBox)
			{
				FilterSearchBox->Update();
			}
		}
	}
}

void FLiveLinkSubjectsView::SaveSelection(TArray<FSavedSelection>& OutSavedSelection) const
{
	TArray<FLiveLinkSubjectUIEntryPtr> SelectedItems = SubjectsTreeView->GetSelectedItems();
	OutSavedSelection.Reserve(SelectedItems.Num());
	for (const FLiveLinkSubjectUIEntryPtr& SelectedItem : SelectedItems)
	{
		FSavedSelection Entry;
		Entry.SubjectKey = SelectedItem->SubjectKey;
		Entry.Type = SelectedItem->Type;
		Entry.WeakDevice = SelectedItem->WeakDevice;
		OutSavedSelection.Add(MoveTemp(Entry));
	}
}

void FLiveLinkSubjectsView::RestoreSelection(const TArray<FSavedSelection>& InSavedSelection)
{
	SubjectsTreeView->ClearSelection();

	for (const FLiveLinkSubjectUIEntryPtr& HeaderEntry : SubjectData)
	{
		for (const FLiveLinkSubjectUIEntryPtr& Child : HeaderEntry->Children)
		{
			RestoreSelectionForItem(Child, InSavedSelection);
			for (const FLiveLinkSubjectUIEntryPtr& GrandChild : Child->Children)
			{
				RestoreSelectionForItem(GrandChild, InSavedSelection);
			}
		}
	}
}

void FLiveLinkSubjectsView::RestoreSelectionForItem(const FLiveLinkSubjectUIEntryPtr& Item, const TArray<FSavedSelection>& InSavedSelection)
{
	for (const FSavedSelection& Selection : InSavedSelection)
	{
		if (Item->Type != Selection.Type)
		{
			continue;
		}

		bool bMatch = false;
		if (Item->IsSubject())
		{
			bMatch = (Item->SubjectKey == Selection.SubjectKey);
		}
		else if (Item->IsSource())
		{
			bMatch = (Item->SubjectKey.Source == Selection.SubjectKey.Source);
		}
		else if (Item->IsDevice())
		{
			bMatch = (Item->WeakDevice == Selection.WeakDevice);
		}

		if (bMatch)
		{
			SubjectsTreeView->SetItemSelection(Item, true);
			break;
		}
	}
}

bool FLiveLinkSubjectsView::CanPauseSubject() const
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsTreeView->GetSelectedItems(Selected);

	for (const FLiveLinkSubjectUIEntryPtr& Entry : Selected)
	{
		if (Entry && Entry->IsSubjectValid() && Entry->IsSubjectEnabled())
		{
			return true;
		}
	}

	return false;
}

void FLiveLinkSubjectsView::HandlePauseSubject()
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsTreeView->GetSelectedItems(Selected);

	for (const FLiveLinkSubjectUIEntryPtr& Entry : Selected)
	{
		if (Entry && Entry->IsSubjectValid() && Entry->IsSubjectEnabled())
		{
			Entry->PauseSubject();
		}
	}
}

TSharedRef<SWidget> FLiveLinkSubjectsView::GetWidget()
{
	InitializeFilters();

	TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
		.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
		.ColorAndOpacity(FSlateColor::UseForeground());

	FilterBar = SNew(SLiveLinkFilterBar<FFilterType>)
		.CustomFilters(CustomFilters)
		.OnFilterChanged(this, &FLiveLinkSubjectsView::OnFilterChanged);

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(4.0, 8.0, 4.0, 6.0))
		.MinHeight(29)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.0, 2.0))
			.AutoWidth()
			[
				SNew(SPositiveActionButton)
					.OnGetMenuContent(this, &FLiveLinkSubjectsView::OnGenerateAddMenuEntries)
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.Text(LOCTEXT("AddButtonLabel", "Add"))
					.ToolTipText(LOCTEXT("AddButton_ToolTip", "Add a new Live Link data device."))
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.0, 2.0))
			.FillWidth(1.0f)
			[
				SAssignNew(FilterSearchBox, SLiveLinkFilterSearchBox<FLiveLinkSubjectUIEntryPtr>, FilterBar)
					.ItemSource(&SubjectData)
					.OnUpdateFilteredList_Lambda([this](const TArray<FLiveLinkSubjectUIEntryPtr>& FilteredItems)
						{
							FilteredList = FilteredItems;

							for (const FLiveLinkSubjectUIEntryPtr& Item : FilteredList)
							{
								SubjectsTreeView->SetItemExpansion(Item, true);
								for (const FLiveLinkSubjectUIEntryPtr& Child : Item->Children)
								{
									SubjectsTreeView->SetItemExpansion(Child, true);
								}
							}

							SubjectsTreeView->RequestListRefresh();
						})
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.0, 2.0))
			.AutoWidth()
			[
				SLiveLinkFilterBar<FFilterType>::MakeAddFilterButton(FilterBar.ToSharedRef())
			]
		]
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Fill)
		[
			SubjectsTreeView.ToSharedRef()
		];
}

void FLiveLinkSubjectsView::CreateCombinedTreeView(const TSharedPtr<FUICommandList>& InCommandList)
{
	SAssignNew(SubjectsTreeView, SLiveLinkSubjectsTreeView, bReadOnly)
		.TreeItemsSource(&FilteredList)
		.OnGenerateRow_Raw(this, &FLiveLinkSubjectsView::MakeTreeRowWidget)
		.OnGetChildren_Raw(this, &FLiveLinkSubjectsView::GetChildrenForInfo)
		.OnSelectionChanged_Raw(this, &FLiveLinkSubjectsView::OnSubjectSelectionChanged)
		.OnContextMenuOpening_Raw(this, &FLiveLinkSubjectsView::OnCombinedListOpenContextMenu, InCommandList)
		.SelectionMode(ESelectionMode::Multi)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true)
			.HiddenColumnsList({ UE::LiveLink::DefaultColumn::TranslatedRole })
			+ SHeaderRow::Column(UE::LiveLink::DefaultColumn::Action)
			.DefaultTooltip(LOCTEXT("ActionWidgetToolTip", "Action that can be applied to the device or subject."))
			.DefaultLabel(FText())
			.FixedWidth(28)
			+ SHeaderRow::Column(UE::LiveLink::DefaultColumn::Name)
			.DefaultLabel(LOCTEXT("ItemName", "Name"))
			.DefaultTooltip(LOCTEXT("ItemNameToolTip", "Name of this Subject/Source/Device."))
			.FillWidth(0.60f)
			+ SHeaderRow::Column(UE::LiveLink::DefaultColumn::Machine)
			.DefaultLabel(LOCTEXT("MachineName", "Machine"))
			.DefaultTooltip(LOCTEXT("MachineNameToolTip", "Name of the machine that hosts this Subject/Source/Device."))
			.HAlignCell(HAlign_Center)
			.FillWidth(0.30f)
			+ SHeaderRow::Column(UE::LiveLink::DefaultColumn::Role)
			.DefaultLabel(LOCTEXT("StatusRoleName", "Status / Role"))
			.ManualWidth(90.f)
			.DefaultTooltip(LOCTEXT("RoleToolTip", "Role of this subject."))
			+ SHeaderRow::Column(UE::LiveLink::DefaultColumn::TranslatedRole)
			.DefaultLabel(LOCTEXT("TranslatedRoleName", "Translated"))
			.DefaultTooltip(LOCTEXT("TranslatedRoleToolTip", "Get the translated role of a subject if it's being translated before being rebroadcast. This should only be relevant in Live Link Hub."))
			.ManualWidth(70.f)
			+ SHeaderRow::Column(UE::LiveLink::DefaultColumn::Status)
			.ManualWidth(20.f)
			.HAlignCell(HAlign_Center)
			.DefaultLabel(FText())
		);
}

FText FLiveLinkSubjectsView::GetPauseSubjectLabel() const
{
	FText Label = LOCTEXT("PauseSubjectLabel", "Pause Subject");

	if (IsSelectedSubjectPaused())
	{
		Label = LOCTEXT("UnpauseSubjectLabel", "Unpause Subject");
	}

	return Label;
}

FText FLiveLinkSubjectsView::GetPauseSubjectToolTip() const
{
	FText ToolTip = LOCTEXT("PauseSubjectToolTip", "Pause a subject, the last received data will be used for evaluation.");

	if (IsSelectedSubjectPaused())
	{
		ToolTip = LOCTEXT("UnpauseSubjectToolTip", "Unpause Subject and resume operating on live data.");
	}

	return ToolTip;
}

bool FLiveLinkSubjectsView::IsSelectedSubjectPaused() const
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsTreeView->GetSelectedItems(Selected);

	for (const FLiveLinkSubjectUIEntryPtr& EntryPtr : Selected)
	{
		if (EntryPtr && !EntryPtr->IsPaused())
		{
			return false;
		}
	}

	return true;
}

TSharedRef<SWidget> FLiveLinkSubjectsView::OnGenerateAddMenuEntries()
{
	bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	ILiveLinkDeviceModule& DeviceModule = ILiveLinkDeviceModule::Get();

	if (bShowDevices)
	{
		DeviceModule.CreateDeviceMenuEntries(MenuBuilder);
	}

	TArray<TObjectPtr<ULiveLinkSourceFactory>> Factories = UE::LiveLink::GetAllSourceFactories();
	UE::LiveLink::GenerateSourceMenuEntries(Factories, MenuBuilder);

	return MenuBuilder.MakeWidget();
}


TSharedRef<SWidget> FLiveLinkSourcesView::OnGenerateSourceMenu()
{
	bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	UE::LiveLink::GenerateSourceMenuEntries(Factories, MenuBuilder);

	return MenuBuilder.MakeWidget();
}

void FLiveLinkSubjectsView::InitializeFilters()
{
	CustomFilters.Reset();

	// Basic filters, for filtering out sources, subjects and devices.
	{
		TSharedPtr<FFilterCategory> BasicFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("BasicFiltersCategory", "Basic"), FText::GetEmpty());

		TSharedRef<FGenericFilter<FFilterType>> SourcesFilter = MakeShared<FGenericFilter<FFilterType>>(
			BasicFiltersCategory,
			TEXT("SourcesFilter"),
			LOCTEXT("SourcesFilterName", "Sources"),
			FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
				{
					return InItem && InItem->IsSource();
				}));

		SourcesFilter->SetToolTipText(LOCTEXT("SourcesFilter", "Only show sources"));
		CustomFilters.Add(SourcesFilter);

		TSharedRef<FGenericFilter<FFilterType>> SubjectsFilter = MakeShared<FGenericFilter<FFilterType>>(
			BasicFiltersCategory,
			TEXT("SubjectsFilter"),
			LOCTEXT("SubjectsFilterName", "Subjects"),
			FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
				{
					return InItem && InItem->IsSubject();
				}));

		SubjectsFilter->SetToolTipText(LOCTEXT("SubjectsFilter", "Only show subjects"));
		CustomFilters.Add(SubjectsFilter);

		TSharedRef<FGenericFilter<FFilterType>> DevicesFilter = MakeShared<FGenericFilter<FFilterType>>(
			BasicFiltersCategory,
			TEXT("DevicesFilter"),
			LOCTEXT("DevicesFilterName", "Devices"),
			FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
				{
					return InItem && InItem->IsDevice();
				}));

		SubjectsFilter->SetToolTipText(LOCTEXT("DevicesFilter", "Only show devices"));
		CustomFilters.Add(DevicesFilter);
	}
}

void FLiveLinkSubjectsView::OnFilterChanged()
{
	FilterSearchBox->Update();
}

#undef LOCTEXT_NAMESPACE /**LiveLinkClientPanel*/
