// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkComponentDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LiveLinkComponentController.h"
#include "LiveLinkControllerBase.h"
#include "LiveLinkEditorPrivate.h"
#include "ScopedTransaction.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "LiveLinkComponentDetailsCustomization"


void FLiveLinkComponentDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayout = &DetailBuilder;

	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailBuilder.GetSelectedObjects();

	//Hide everything when more than one are selected
	if (SelectedObjects.Num() != 1)
	{
		for (TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
		{
			if (ULiveLinkComponentController* SelectedPtr = Cast<ULiveLinkComponentController>(SelectedObject.Get()))
			{
				TSharedRef<IPropertyHandle> ControllersProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkComponentController, ControllerMap));
				ControllersProperty->MarkHiddenByCustomization();
			}
		}

		return;
	}

	if (ULiveLinkComponentController* SelectedPtr = Cast<ULiveLinkComponentController>(SelectedObjects[0].Get()))
	{
		// Force "LiveLink" to be the topmost category in the details panel
		DetailBuilder.EditCategory(TEXT("LiveLink"));

		//Hide the Map default UI
		TSharedRef<IPropertyHandle> ControllersProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkComponentController, ControllerMap));
		ControllersProperty->MarkHiddenByCustomization();

		//Get hook to the controller map. If that fails, early exit
		TSharedPtr<IPropertyHandleMap> MapHandle = ControllersProperty->AsMap();
		if (MapHandle.IsValid())
		{
			EditedObject = SelectedPtr;

			//Register callback when LiveLinkSubjectRepresentation selection has changed to refresh the UI and update controller
			TSharedRef<IPropertyHandle> SubjectRepresentationPropertyRef = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkComponentController, SubjectRepresentation));
			SubjectRepresentationPropertyRef->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FLiveLinkComponentDetailCustomization::OnSubjectRepresentationPropertyChanged));
			SubjectRepresentationPropertyRef->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FLiveLinkComponentDetailCustomization::OnSubjectRepresentationPropertyChanged));

			//Listen to controller map modifications to refresh UI when a change comes through MU
			FSimpleDelegate RefreshDelegate = FSimpleDelegate::CreateSP(this, &FLiveLinkComponentDetailCustomization::ForceRefreshDetails);
			MapHandle->SetOnNumElementsChanged(RefreshDelegate);

			//Loop for each entry in the map. 
			//Fetch the LiveLinkRole name (key) and display its name
			//Add a menu to select a ControllerClass for it
			//If a Controller is picked, display its properties
			uint32 NumEntry = 0;
			ControllersProperty->GetNumChildren(NumEntry);
			for (uint32 EntryIndex = 0; EntryIndex < NumEntry; ++EntryIndex)
			{
				TSharedPtr<IPropertyHandle> EntryHandle = ControllersProperty->GetChildHandle(EntryIndex);
				if (!EntryHandle.IsValid())
				{
					continue;
				}

				TSharedPtr<IPropertyHandle> KeyHandle = EntryHandle->GetKeyHandle();

				//Map has a TSubClassof Key type. Make sure we were able to query the UClass and that it's not null. 
				UObject* LiveLinkRoleClassObj = nullptr;
				KeyHandle->GetValue(LiveLinkRoleClassObj);

				TSubclassOf<ULiveLinkRole> LiveLinkRoleClass = Cast<UClass>(LiveLinkRoleClassObj);
				if (LiveLinkRoleClass == nullptr)
				{
					continue;
				}

				FText ControllerName = FText::Format(LOCTEXT("No Controller", "{0}"), FText::FromName(NAME_None));

				//Map value is a pointer to the Controller. Can be null so it could be empty
				TArray<UObject*> ExternalObjects;
				UObject* ControllerPtr = nullptr;
				EntryHandle->GetValue(ControllerPtr);
				if (ControllerPtr != nullptr)
				{
					ControllerName = ControllerPtr->GetClass()->GetDisplayNameText();
					ExternalObjects.Add(ControllerPtr);
				}

				//Since we're displaying properties of another object, add it as external to the current one being edited. 
				IDetailPropertyRow* ThisRoleRow = DetailBuilder.EditCategory(TEXT("Role Controllers")).AddExternalObjects(ExternalObjects, EPropertyLocation::Type::Default, FAddPropertyParams().UniqueId("LiveLinkControllerRow"));

				//As external objects, each map entry will generate a custom widget for its detail property row like this: 
				//(NameWidget) | (ValueWidget)
				//Role Name    | Dropdown menu with available controllers
				if (ThisRoleRow)
				{
					ThisRoleRow->CustomWidget()
					.NameContent()
					[
						BuildControllerNameWidget(ControllersProperty, LiveLinkRoleClass)
					]
					.ValueContent()
					[
						BuildControllerValueWidget(LiveLinkRoleClass, ControllerName)
					];
				}
			}

			// Start by looking if data is dirty as we enter. Can happen when component lives in a blueprint
			OnSubjectRepresentationPropertyChanged();
		}
	}
}

void FLiveLinkComponentDetailCustomization::OnSubjectRepresentationPropertyChanged()
{
	if (ULiveLinkComponentController* EditedObjectPtr = EditedObject.Get())
	{
		//Verify if Role has changed
		if (EditedObjectPtr->IsControllerMapOutdated())
		{
			const FScopedTransaction Transaction(LOCTEXT("OnChangedSubjectRepresentation", "Subject Representation Changed"));
			EditedObjectPtr->Modify();

			EditedObjectPtr->OnSubjectRoleChanged();
		}
	}
}

TSharedRef<SWidget> FLiveLinkComponentDetailCustomization::HandleControllerComboButton(TSubclassOf<ULiveLinkRole> RoleClass) const
{
	// Generate menu
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("SupportedControllers", LOCTEXT("SupportedControllers", "Controllers"));
	{
		if(RoleClass)
		{
			TArray<TSubclassOf<ULiveLinkControllerBase>> NewControllerClasses = ULiveLinkControllerBase::GetControllersForRole(RoleClass);
			if (NewControllerClasses.Num() > 0)
			{
				//Always add a None entry
				MenuBuilder.AddMenuEntry(
					FText::FromName(NAME_None),
					FText::FromName(NAME_None),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FLiveLinkComponentDetailCustomization::HandleControllerSelection, RoleClass, TWeakObjectPtr<UClass>()),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &FLiveLinkComponentDetailCustomization::IsControllerItemSelected, FName(), RoleClass)
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);

				for (const TSubclassOf<ULiveLinkControllerBase>& ControllerClass : NewControllerClasses)
				{
					if (UClass* ControllerClassPtr = ControllerClass.Get())
					{
						MenuBuilder.AddMenuEntry(
							FText::Format(LOCTEXT("Controller Label", "{0}"), ControllerClassPtr->GetDisplayNameText()),
							FText::Format(LOCTEXT("Controller ToolTip", "{0}"), ControllerClassPtr->GetDisplayNameText()),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(this, &FLiveLinkComponentDetailCustomization::HandleControllerSelection, RoleClass, MakeWeakObjectPtr(ControllerClassPtr)),
								FCanExecuteAction(),
								FIsActionChecked::CreateSP(this, &FLiveLinkComponentDetailCustomization::IsControllerItemSelected, ControllerClassPtr->GetFName(), RoleClass)
							),
							NAME_None,
							EUserInterfaceActionType::RadioButton
						);
					}

				}
			}
			else
			{
				MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("NoControllersFound", "No Controllers were found for this role"), false, false);
			}
		}
		else
		{
			MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("InvalidRoleClass", "Role is invalid. Can't find controllers for it"), false, false);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FLiveLinkComponentDetailCustomization::HandleControllerSelection(TSubclassOf<ULiveLinkRole> RoleClass, TWeakObjectPtr<UClass> SelectedControllerClass) const
{
 	if (ULiveLinkComponentController* EditedObjectPtr = EditedObject.Get())
 	{
 		const FScopedTransaction Transaction(LOCTEXT("OnChangedController", "Property Controller Selection"));
 		EditedObjectPtr->Modify();
 
 		UClass* SelectedControllerClassPtr = SelectedControllerClass.Get();
 		EditedObjectPtr->SetControllerClassForRole(RoleClass, SelectedControllerClassPtr);
 	}
}

bool FLiveLinkComponentDetailCustomization::IsControllerItemSelected(FName Item, TSubclassOf<ULiveLinkRole> RoleClass) const
{
	if (RoleClass == nullptr || RoleClass.Get() == nullptr)
	{
		return false;
	}

	if (ULiveLinkComponentController* EditedObjectPtr = EditedObject.Get())
	{
		TObjectPtr<ULiveLinkControllerBase>* CurrentClass = EditedObjectPtr->ControllerMap.Find(RoleClass);
		if (CurrentClass == nullptr || *CurrentClass == nullptr)
		{
			return Item.IsNone();
		}
		else
		{
			return (*CurrentClass)->GetClass()->GetFName() == Item;
		}
	}

	return false;
}

EVisibility FLiveLinkComponentDetailCustomization::HandleControllerWarningVisibility(TSubclassOf<ULiveLinkRole> RoleClassEntry) const
{
	if (ULiveLinkComponentController* EditorObjectPtr = EditedObject.Get())
	{
		TObjectPtr<ULiveLinkControllerBase>* AssociatedControllerPtr = EditorObjectPtr->ControllerMap.Find(RoleClassEntry);
		if (AssociatedControllerPtr && *AssociatedControllerPtr)
		{
			const ULiveLinkControllerBase* AssociatedController = *AssociatedControllerPtr;
			if (UActorComponent* SelectedComponent = AssociatedController->GetAttachedComponent())
			{
				if (!SelectedComponent->IsA(AssociatedController->GetDesiredComponentClass()))
				{
					//Controller exists for RoleClass and desired component is not the kind it wants
					return EVisibility::Visible;
				}
			}
			else
			{
				//Component is not valid
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Hidden;
}

TSharedRef<SWidget> FLiveLinkComponentDetailCustomization::BuildControllerNameWidget(TSharedPtr<IPropertyHandle> ControllersProperty, TSubclassOf<ULiveLinkRole> RoleClass) const
{
	return ControllersProperty->CreatePropertyNameWidget(RoleClass.Get()->GetDisplayNameText());
}

TSharedRef<SWidget> FLiveLinkComponentDetailCustomization::BuildControllerValueWidget(TSubclassOf<ULiveLinkRole> RoleClass, const FText& ControllerName) const
{
	TSharedRef<SWidget> ValueWidget = 
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FLiveLinkComponentDetailCustomization::HandleControllerComboButton, RoleClass)
			.ContentPadding(FMargin(4.0, 2.0))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(ControllerName)
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.Image(FLiveLinkEditorPrivate::GetStyleSet()->GetBrush("LiveLinkController.WarningIcon"))
			.ToolTipText(LOCTEXT("ControllerWarningToolTip", "The selected component to control does not match this controller's desired component class"))
 			.Visibility(this, &FLiveLinkComponentDetailCustomization::HandleControllerWarningVisibility, RoleClass)
		];

	return ValueWidget;
}

void FLiveLinkComponentDetailCustomization::ForceRefreshDetails()
{
	DetailLayout->ForceRefreshDetails();
}

#undef LOCTEXT_NAMESPACE
