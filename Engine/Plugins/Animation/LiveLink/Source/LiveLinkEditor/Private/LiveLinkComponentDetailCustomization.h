// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "UObject/WeakObjectPtr.h"

class ULiveLinkComponentController;
class ULiveLinkRole;
struct EVisibility;
template <class TClass> class TSubclassOf;

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;

/**
* Customizes a ULiveLinkComponentController details
*/
class FLiveLinkComponentDetailCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FLiveLinkComponentDetailCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

protected:
	void OnSubjectRepresentationPropertyChanged();
	TSharedRef<SWidget> HandleControllerComboButton(TSubclassOf<ULiveLinkRole> RoleClass) const;
	void HandleControllerSelection(TSubclassOf<ULiveLinkRole> RoleClass, TWeakObjectPtr<UClass> SelectedControllerClass) const;
	bool IsControllerItemSelected(FName Item, TSubclassOf<ULiveLinkRole> RoleClass) const;
	EVisibility HandleControllerWarningVisibility(TSubclassOf<ULiveLinkRole> RoleClassEntry) const;
	TSharedRef<SWidget> BuildControllerNameWidget(TSharedPtr<IPropertyHandle> ControllersProperty, TSubclassOf<ULiveLinkRole> RoleClass) const;
	TSharedRef<SWidget> BuildControllerValueWidget(TSubclassOf<ULiveLinkRole> RoleClass, const FText& ControllerName) const;
	void ForceRefreshDetails();

protected:
	/** LiveLinkComponent on which we're acting */
	TWeakObjectPtr<ULiveLinkComponentController> EditedObject;

	/** Keep a reference to force refresh the layout */
	IDetailLayoutBuilder* DetailLayout = nullptr;
};
