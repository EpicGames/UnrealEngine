// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkControllerBaseDetailCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "LiveLinkControllerBase.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "LiveLinkControllerBaseDetailsCustomization"

void FLiveLinkControllerBaseDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);

	if (SelectedObjects.Num() < 1)
	{
		return;
	}

	// The controller is not selectable in the details panel, the the controller map is hidden when multiple livelink components are selected
	// Safe to assume there is only one selected controller being customized
	LiveLinkControllerWeak = Cast<ULiveLinkControllerBase>(SelectedObjects[0].Get());

	// Collapse the properties of each controller by default (cleaner look for controllers with a lot of settings)
	TArray<FName> CategoryNames;
	DetailBuilder.GetCategoryNames(CategoryNames);
	for (FName CategoryName : CategoryNames)
	{
		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(CategoryName);
		Category.InitiallyCollapsed(true);
	}

	DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
	{
		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : CategoryMap)
		{
			int32 SortOrder = Pair.Value->GetSortOrder();
			const FName CategoryName = Pair.Key;

			if (CategoryName == "ComponentToControl")
			{
				SortOrder = 0;
			}
			else
			{
				const int32 ValueSortOrder = Pair.Value->GetSortOrder();
				if (ValueSortOrder >= SortOrder && ValueSortOrder < SortOrder + 10)
				{
					SortOrder += 10;
				}
				else
				{
					continue;
				}
			}

			Pair.Value->SetSortOrder(SortOrder);
		}
	});

	// Try to get the ComponentToControl property from the category
	IDetailCategoryBuilder& ComponentToControlCategory = DetailBuilder.EditCategory(TEXT("ComponentToControl"));
	TArray<TSharedRef<IPropertyHandle>> ComponentToControlProperties;
	ComponentToControlCategory.GetDefaultProperties(ComponentToControlProperties);
	if (ComponentToControlProperties.Num() > 0)
	{
		// Set callback when the user makes a change to the component picker
		TSharedRef<IPropertyHandle> ComponentToControlProperty = ComponentToControlProperties[0];
		ComponentToControlProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FLiveLinkControllerBaseDetailCustomization::OnComponentChanged));
	}
 }

void FLiveLinkControllerBaseDetailCustomization::OnComponentChanged()
{
	if (ULiveLinkControllerBase* LiveLinkController = LiveLinkControllerWeak.Get())
	{
		LiveLinkController->SetAttachedComponent(LiveLinkController->GetAttachedComponent());
	}
}

#undef LOCTEXT_NAMESPACE
