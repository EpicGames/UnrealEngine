// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "UObject/WeakObjectPtr.h"

class ULiveLinkControllerBase;

class IDetailLayoutBuilder;

/**
* Customizes ULiveLinkControllerBase details
*/
class FLiveLinkControllerBaseDetailCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FLiveLinkControllerBaseDetailCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

private:
	void OnComponentChanged();

protected:
	TWeakObjectPtr<ULiveLinkControllerBase> LiveLinkControllerWeak;
};
