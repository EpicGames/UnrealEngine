// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetToolsModule.h"
#include "Interfaces/IPluginManager.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Features/IModularFeatures.h"
#include "Brushes/SlateNoResource.h"
#include "ISettingsModule.h"
#include "IPlacementModeModule.h"
#include "LevelEditor.h"
#include "LiveLinkSourceSettings.h"
#include "PropertyEditorModule.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "LiveLinkBoneAttachmentDetailCustomization.h"
#include "LiveLinkBroadcastComponent.h"
#include "LiveLinkBroadcastComponentDetailCustomization.h"
#include "LiveLinkComponentController.h"
#include "LiveLinkComponentDetailCustomization.h"
#include "LiveLinkComponentSettings.h"
#include "LiveLinkControllerBase.h"
#include "LiveLinkControllerBaseDetailCustomization.h"
#include "LiveLinkClient.h"
#include "LiveLinkClientPanel.h"
#include "LiveLinkClientCommands.h"
#include "LiveLinkEditorPrivate.h"
#include "LiveLinkEditorSettings.h"
#include "LiveLinkGraphPanelPinFactory.h"
#include "LiveLinkHubSubjectSettings.h"
#include "LiveLinkHubSubjectSettingsDetailsCustomization.h"
#include "LiveLinkSettings.h"
#include "LiveLinkSourceSettingsDetailCustomization.h"
#include "LiveLinkSubjectKeyDetailCustomization.h"
#include "LiveLinkSubjectNameDetailCustomization.h"
#include "LiveLinkSubjectRepresentationDetailCustomization.h"
#include "LiveLinkVirtualSubjectDetailCustomization.h"
#include "LiveLinkVirtualSubjectBoneAttachment.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "VirtualSubjects/LiveLinkBlueprintVirtualSubject.h"
#include "Visualizers/LiveLinkDataPreview.h"



/**
 * Implements the LiveLinkEditor module.
 */

LLM_DEFINE_TAG(LiveLink_LiveLinkEditor);
DEFINE_LOG_CATEGORY(LogLiveLinkEditor);


#define LOCTEXT_NAMESPACE "LiveLinkModule"

static const FName LiveLinkClientTabName(TEXT("LiveLink"));
static const FName LevelEditorModuleName(TEXT("LevelEditor"));


namespace LiveLinkEditorModuleUtils
{
	FString InPluginContent(const FString& RelativePath, const ANSICHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("LiveLink"))->GetContentDir();
		return (ContentDir / RelativePath) + Extension;
	}
}

#define IMAGE_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(StyleSet->RootToCoreContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)
#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( LiveLinkEditorModuleUtils::InPluginContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( LiveLinkEditorModuleUtils::InPluginContent( RelativePath, ".svg" ), __VA_ARGS__ )


class FLiveLinkEditorModule : public IModuleInterface
{
public:
	static TSharedPtr<FSlateStyleSet> StyleSet;

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(LiveLink_LiveLinkEditor);

		static FName LiveLinkStyle(TEXT("LiveLinkStyle"));
		StyleSet = MakeShared<FSlateStyleSet>(LiveLinkStyle);

		bHasRegisteredTabSpawners = false;

		if (FModuleManager::Get().IsModuleLoaded(LevelEditorModuleName))
		{
			RegisterTabSpawner();
		}

		ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FLiveLinkEditorModule::ModulesChangesCallback);

		FLiveLinkClientCommands::Register();

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));

		const FVector2D Icon8x8(8.0f, 8.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);

		StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		StyleSet->Set("LiveLinkClient.Common.Icon", new IMAGE_PLUGIN_BRUSH(TEXT("LiveLink_40x"), Icon40x40));
		StyleSet->Set("LiveLinkClient.Common.Icon.Small", new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));

		StyleSet->Set("LiveLinkHub.Clients.Icon", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Clients"), Icon16x16));
		StyleSet->Set("LiveLinkHub.Sources.Icon", new IMAGE_PLUGIN_BRUSH(TEXT("Starship/Sources"), Icon16x16));
		StyleSet->Set("LiveLinkHub.Layout.Icon", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Layout"), Icon16x16));
		StyleSet->Set("LiveLinkHub.Playback.Icon", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Playback"), Icon16x16));
		StyleSet->Set("LiveLinkHub.Subjects.Icon", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Subjects"), Icon16x16));

		StyleSet->Set("LiveLinkHub.AutoConnect.Disabled", new IMAGE_PLUGIN_BRUSH_SVG("Starship/AutoConnectDisabled", Icon16x16));
		StyleSet->Set("LiveLinkHub.AutoConnect.Global", new IMAGE_PLUGIN_BRUSH_SVG("Starship/AutoConnectGlobal", Icon16x16));
		StyleSet->Set("LiveLinkHub.AutoConnect.Local", new IMAGE_PLUGIN_BRUSH_SVG("Starship/AutoConnectLocal", Icon16x16));

		StyleSet->Set("LiveLinkHub.Icon.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Starship/LiveLinkHub_16x16"), Icon16x16));
		StyleSet->Set("LiveLinkHub.Icon", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/LiveLinkHub_Icon"), Icon40x40));
		StyleSet->Set("LiveLinkHub.EllipsisIcon", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/Ellipsis"), Icon16x16));
		StyleSet->Set("LiveLinkHub.TimecodeGenlock", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/TimecodeGenlock"), Icon16x16));
		StyleSet->Set("LiveLinkHub.SpokeMode", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/SpokeMode"), Icon16x16, FSlateColor(FLinearColor(1.000f, 0.168f, 0.350f))));
		StyleSet->Set("LiveLinkHub.HubMode", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/HubMode"), Icon16x16, FSlateColor(FLinearColor(0.271f, 0.554f, 0.067f))));


		StyleSet->Set("ClassIcon.LiveLinkPreset",                      new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));
		StyleSet->Set("ClassIcon.LiveLinkFrameInterpolationProcessor", new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));
		StyleSet->Set("ClassIcon.LiveLinkFramePreProcessor",           new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));
		StyleSet->Set("ClassIcon.LiveLinkFrameTranslator",             new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));
		StyleSet->Set("ClassIcon.LiveLinkPreset",                      new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));
		StyleSet->Set("ClassIcon.LiveLinkRole",                        new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));
		StyleSet->Set("ClassIcon.LiveLinkVirtualSubject",              new IMAGE_PLUGIN_BRUSH_SVG("Starship/LiveLink", Icon16x16));

		StyleSet->Set("ClassThumbnail.LiveLinkPreset", new IMAGE_PLUGIN_BRUSH("LiveLink_40x", Icon40x40));

		StyleSet->Set("LiveLinkClient.Common.AddSource", new IMAGE_PLUGIN_BRUSH(TEXT("icon_AddSource_40x"), Icon40x40));
		StyleSet->Set("LiveLinkClient.Common.RemoveSource", new IMAGE_PLUGIN_BRUSH(TEXT("icon_RemoveSource_40x"), Icon40x40));
		StyleSet->Set("LiveLinkClient.Common.RemoveAllSources", new IMAGE_PLUGIN_BRUSH(TEXT("icon_RemoveSource_40x"), Icon40x40));

		StyleSet->Set("PlacementBrowser.Icons.VirtualProduction", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Starship/VirtualProduction"), Icon16x16));

		StyleSet->Set("LiveLinkController.WarningIcon", new IMAGE_BRUSH_SVG("Starship/Common/alert-circle", Icon16x16, FStyleColors::Warning));
		
		FButtonStyle Button = FButtonStyle()
			.SetNormal(FSlateBoxBrush(StyleSet->RootToContentDir("Common/ButtonHoverHint.png"), FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 0.15f)))
			.SetHovered(FSlateBoxBrush(StyleSet->RootToContentDir("Common/ButtonHoverHint.png"), FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 0.25f)))
			.SetPressed(FSlateBoxBrush(StyleSet->RootToContentDir("Common/ButtonHoverHint.png"), FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 0.30f)))
			.SetNormalPadding(FMargin(0, 0, 0, 1))
			.SetPressedPadding(FMargin(0, 1, 0, 0));
		FComboButtonStyle ComboButton = FComboButtonStyle()
			.SetButtonStyle(Button.SetNormal(FSlateNoResource()))
			.SetDownArrowImage(FSlateImageBrush(StyleSet->RootToCoreContentDir(TEXT("Common/ComboArrow.png")), Icon8x8))
			.SetMenuBorderBrush(FSlateBoxBrush(StyleSet->RootToCoreContentDir(TEXT("Old/Menu_Background.png")), FMargin(8.0f / 64.0f)))
			.SetMenuBorderPadding(FMargin(0.0f));
		StyleSet->Set("ComboButton", ComboButton);

		FEditableTextBoxStyle EditableTextStyle = FEditableTextBoxStyle()
			.SetTextStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 9))
			.SetBackgroundImageNormal(FSlateNoResource())
			.SetBackgroundImageHovered(FSlateNoResource())
			.SetBackgroundImageFocused(FSlateNoResource())
			.SetBackgroundImageReadOnly(FSlateNoResource())
			.SetBackgroundColor(FLinearColor::Transparent)
			.SetForegroundColor(FSlateColor::UseForeground());

		StyleSet->Set("EditableTextBox", EditableTextStyle);

		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

		RegisterSettings();
		RegisterCustomizations();
		RegisterAutoGeneratedDefaultEvents();
		RegisterPlacementModeItems();
	}

	void ModulesChangesCallback(FName ModuleName, EModuleChangeReason ReasonForChange)
	{
		if (ReasonForChange == EModuleChangeReason::ModuleLoaded && ModuleName == LevelEditorModuleName)
		{
			RegisterTabSpawner();
		}
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(LiveLink_LiveLinkEditor);

		if (!IsEngineExitRequested() && GEditor && UObjectInitialized())
		{
			UnregisterPlacementModeItems();
		}

		UnregisterCustomizations();

		UnregisterTabSpawner();

		FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);

		if (LevelEditorTabManagerChangedHandle.IsValid() && FModuleManager::Get().IsModuleLoaded(LevelEditorModuleName))
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName);
			LevelEditorModule.OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
		}

		StyleSet.Reset();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	static TSharedRef<SDockTab> SpawnLiveLinkTab(const FSpawnTabArgs& SpawnTabArgs, TSharedPtr<FSlateStyleSet> InStyleSet)
	{
		FLiveLinkClient* Client = &IModularFeatures::Get().GetModularFeature<FLiveLinkClient>(FLiveLinkClient::ModularFeatureName);

		const TSharedRef<SDockTab> MajorTab =
			SNew(SDockTab)
			.TabRole(ETabRole::NomadTab);

		MajorTab->SetContent(SNew(SLiveLinkClientPanel, Client));

		return MajorTab;
	}

private:

	void RegisterTabSpawner()
	{
		if (bHasRegisteredTabSpawners)
		{
			UnregisterTabSpawner();
		}

		FTabSpawnerEntry& SpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(LiveLinkClientTabName, FOnSpawnTab::CreateStatic(&FLiveLinkEditorModule::SpawnLiveLinkTab, StyleSet))
			.SetDisplayName(LOCTEXT("LiveLinkTabTitle", "Live Link"))
			.SetTooltipText(LOCTEXT("LiveLinkTabTooltipText", "Open the Live Link streaming manager tab."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory())
			.SetIcon(FSlateIcon(StyleSet->GetStyleSetName(), "LiveLinkClient.Common.Icon.Small"));

		bHasRegisteredTabSpawners = true;
	}

	void UnregisterTabSpawner()
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(LiveLinkClientTabName);
		bHasRegisteredTabSpawners = false;
	}

	void RegisterSettings()
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Editor", "Plugins", "Live Link",
				LOCTEXT("EditorSettingsName", "Live Link"),
				LOCTEXT("EditorSettingsDescription", "Configure Live Link."),
				GetMutableDefault<ULiveLinkEditorSettings>()
			);

			SettingsModule->RegisterSettings("Project", "Plugins", "Live Link",
				LOCTEXT("LiveLinkSettingsName", "Live Link"),
				LOCTEXT("LiveLinkDescription", "Configure the Live Link plugin."),
				GetMutableDefault<ULiveLinkSettings>()
			);

			SettingsModule->RegisterSettings("Project", "Plugins", "Live Link Component",
				LOCTEXT("LiveLinkComponentSettingsName", "Live Link Component"),
				LOCTEXT("LiveLinkComponentDescription", "Configure the Live Link Component."),
				GetMutableDefault<ULiveLinkComponentSettings>()
			);
		}
	}

	void UnregisterSettings()
	{
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "Live Link Component");
			SettingsModule->UnregisterSettings("Project", "Plugins", "Live Link");
			SettingsModule->UnregisterSettings("Editor", "Plugins", "Live Link");
		}
	}

	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout(ULiveLinkVirtualSubject::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkVirtualSubjectDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FLiveLinkSubjectKey::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLiveLinkSubjectKeyDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FLiveLinkSubjectRepresentation::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLiveLinkSubjectRepresentationDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FLiveLinkSubjectName::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLiveLinkSubjectNameDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FLiveLinkVirtualSubjectBoneAttachment::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLiveLinkBoneAttachmentDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomClassLayout(ULiveLinkSourceSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkSourceSettingsDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomClassLayout(ULiveLinkComponentController::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkComponentDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomClassLayout(ULiveLinkBroadcastComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkBroadcastComponentDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomClassLayout(ULiveLinkControllerBase::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkControllerBaseDetailCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomClassLayout(ULiveLinkHubSubjectSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkHubSubjectSettingsDetailsCustomization::MakeInstance));

		LiveLinkGraphPanelPinFactory = MakeShared<FLiveLinkGraphPanelPinFactory>();
		FEdGraphUtilities::RegisterVisualPinFactory(LiveLinkGraphPanelPinFactory);
	}

	void UnregisterCustomizations()
	{
		if (UObjectInitialized() && !IsEngineExitRequested())
		{
			FEdGraphUtilities::UnregisterVisualPinFactory(LiveLinkGraphPanelPinFactory);
			FPropertyEditorModule* PropertyEditorModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor");
			if (PropertyEditorModule)
			{
				PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkHubSubjectSettings::StaticClass()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkBroadcastComponent::StaticClass()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkComponentController::StaticClass()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkControllerBase::StaticClass()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkSourceSettings::StaticClass()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(FLiveLinkVirtualSubjectBoneAttachment::StaticStruct()->GetFName());
				PropertyEditorModule->UnregisterCustomPropertyTypeLayout(FLiveLinkSubjectName::StaticStruct()->GetFName());
				PropertyEditorModule->UnregisterCustomPropertyTypeLayout(FLiveLinkSubjectRepresentation::StaticStruct()->GetFName());
				PropertyEditorModule->UnregisterCustomPropertyTypeLayout(FLiveLinkSubjectKey::StaticStruct()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkVirtualSubject::StaticClass()->GetFName());
			}
		}
	}

	// Set up default nodes for live link blueprint classes
	void RegisterAutoGeneratedDefaultEvents()
	{
		FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, ULiveLinkBlueprintVirtualSubject::StaticClass(), "OnInitialize");
		FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, ULiveLinkBlueprintVirtualSubject::StaticClass(), "OnUpdate");
	}

	// Register the livelink data preview actor.
	void RegisterPlacementModeItems()
	{
		auto RegisterPlaceActors = [&]() -> void
			{
				if (!GEditor)
				{
					return;
				}

				const FPlacementCategoryInfo* Info = GetVirtualProductionCategoryRegisteredInfo();

				if (!Info)
				{
					UE_LOGF(LogLiveLinkEditor, Warning, "Could not find or create VirtualProduction Place Actor Category");
					return;
				}

				/*Register the Live Link Data Prevew Actor*/
				PlaceActors.Add(IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
					*ALiveLinkDataPreview::StaticClass(),
					FAssetData(ALiveLinkDataPreview::StaticClass()),
					NAME_None,
					NAME_None,
					TOptional<FLinearColor>(),
					TOptional<int32>(),
					NSLOCTEXT("PlacementMode", "LiveLinkDataPreview", "Live Link Data Preview")
				)));
			};

		if (FApp::CanEverRender())
		{
			if (GEngine && GEngine->IsInitialized())
			{
				RegisterPlaceActors();
			}
			else
			{
				FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterPlaceActors);
			}
		}
	}

	// Unregister the data preview actor placement item.
	void UnregisterPlacementModeItems()
	{
		IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();

		for (TOptional<FPlacementModeID>& PlaceActor : PlaceActors)
		{
			if (PlaceActor.IsSet())
			{
				PlacementModeModule.UnregisterPlaceableItem(*PlaceActor);
			}
		}

		PlaceActors.Empty();
	}

	// Get the virtual production actor category info.
	const FPlacementCategoryInfo* GetVirtualProductionCategoryRegisteredInfo() const
	{
		IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();

		if (const FPlacementCategoryInfo* RegisteredInfo = PlacementModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction()))
		{
			return RegisteredInfo;
		}
		else
		{
			FPlacementCategoryInfo Info(
				LOCTEXT("VirtualProductionCategoryName", "Virtual Production"),
				FSlateIcon(StyleSet->GetStyleSetName(), "PlacementBrowser.Icons.VirtualProduction"),
				FLevelEditorOutlinerBuiltInCategories::VirtualProduction(),
				TEXT("PMVirtualProduction"),
				25 // Determines where the category shows up in the list with respect to the others.
			);
			Info.ShortDisplayName = LOCTEXT("VirtualProductionShortCategoryName", "VP");
			IPlacementModeModule::Get().RegisterPlacementCategory(Info);

			return PlacementModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction());
		}
	}

private:
	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle ModulesChangedHandle;

	TSharedPtr<FLiveLinkGraphPanelPinFactory> LiveLinkGraphPanelPinFactory;

	// Track if we have registered
	bool bHasRegisteredTabSpawners;

	// Registered livelink data preview actor.
	TArray<TOptional<FPlacementModeID>> PlaceActors;
};

TSharedPtr<FSlateStyleSet> FLiveLinkEditorModule::StyleSet;

IMPLEMENT_MODULE(FLiveLinkEditorModule, LiveLinkEditor);


TSharedPtr< class ISlateStyle > FLiveLinkEditorPrivate::GetStyleSet()
{
	return FLiveLinkEditorModule::StyleSet;
}

#undef LOCTEXT_NAMESPACE
