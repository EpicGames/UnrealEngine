// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkEditor, Log, All);


class ISlateStyle;

struct FLiveLinkEditorPrivate
{
	static TSharedPtr< class ISlateStyle > GetStyleSet();
};
