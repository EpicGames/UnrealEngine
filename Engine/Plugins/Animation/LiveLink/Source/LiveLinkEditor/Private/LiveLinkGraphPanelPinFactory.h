// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "EdGraphSchema_K2.h"

#include "SLiveLinkSubjectNameGraphPin.h"
#include "SLiveLinkSubjectRepresentationGraphPin.h"


class FLiveLinkGraphPanelPinFactory : public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override
	{
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			if (UScriptStruct* PinStructType = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()))
			{
				if (PinStructType == FLiveLinkSubjectRepresentation::StaticStruct())
				{
					return SNew(SLiveLinkSubjectRepresentationGraphPin, InPin);
				}
				else if (PinStructType == FLiveLinkSubjectName::StaticStruct())
				{
					return SNew(SLiveLinkSubjectNameGraphPin, InPin);
				}
			}
		}

		return nullptr;
	}
};
