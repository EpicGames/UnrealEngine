// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubSubjectSettingsDetailsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "LiveLinkHubSubjectSettings.h"
#include "LiveLinkSettings.h"
#include "Misc/ConfigCacheIni.h"


#define LOCTEXT_NAMESPACE "LiveLinkHubSubjectSettingsDetailCustomization"

void FLiveLinkHubSubjectSettingsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	//Get the current settings object being edited
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() != 1)
	{
		return;
	}

	const bool bIsInLiveLinkHubApp = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni);

	if (bIsInLiveLinkHubApp)
	{
		TSharedRef<IPropertyHandle> OutboundProperty = InDetailBuilder.GetProperty(ULiveLinkHubSubjectSettings::GetOutboundNamePropertyName());
		if (IDetailPropertyRow* PropertyRow = InDetailBuilder.EditDefaultProperty(OutboundProperty))
		{
			FResetToDefaultOverride ResetOverride;

			const FName SubjectNamePropertyName = ULiveLinkHubSubjectSettings::GetSubjectNamePropertyName();

			PropertyRow->CustomWidget()
				.OverrideResetToDefault(FResetToDefaultOverride::Create(
					FIsResetToDefaultVisible::CreateLambda([SubjectNamePropertyName](TSharedPtr<IPropertyHandle> PropertyHandle)
						{
							TSharedPtr<IPropertyHandle> SubjectNameProperty = PropertyHandle->GetParentHandle()->GetChildHandle(SubjectNamePropertyName);
						
							if (SubjectNameProperty && SubjectNameProperty->IsValidHandle())
							{
								FText OutboundName;
								FText SubjectName;
								PropertyHandle->GetValueAsDisplayText(OutboundName);
								SubjectNameProperty->GetValueAsDisplayText(SubjectName);
								return !OutboundName.EqualTo(SubjectName);
							}
							return false;
						}),
					FResetToDefaultHandler::CreateLambda([SubjectNamePropertyName](TSharedPtr<IPropertyHandle> PropertyHandle)
						{
							TSharedPtr<IPropertyHandle> SubjectNameProperty = PropertyHandle->GetParentHandle()->GetChildHandle(SubjectNamePropertyName);

							if (SubjectNameProperty && SubjectNameProperty->IsValidHandle())
							{
								FString SubjectName;
								SubjectNameProperty->GetValueAsDisplayString(SubjectName);
								PropertyHandle->SetValue(*SubjectName);
							}
						})
				))
				.NameContent()
				[
					OutboundProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					OutboundProperty->CreatePropertyValueWidget()
				];
		}

		InDetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, FrameRate), ULiveLinkSubjectSettings::StaticClass());
		InDetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, bRebroadcastSubject), ULiveLinkSubjectSettings::StaticClass());
		InDetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkHubSubjectSettings, Translators), ULiveLinkSubjectSettings::StaticClass());
	}
	else
	{
		InDetailBuilder.HideProperty(ULiveLinkHubSubjectSettings::GetOutboundNamePropertyName(), ULiveLinkHubSubjectSettings::StaticClass());
	}
}

#undef LOCTEXT_NAMESPACE
