// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class ULiveLinkHubSubjectSettings;


/**
* Customizes a ULiveLinkHubSubjectSettings object
*/
class FLiveLinkHubSubjectSettingsDetailsCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FLiveLinkHubSubjectSettingsDetailsCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

private:
};
