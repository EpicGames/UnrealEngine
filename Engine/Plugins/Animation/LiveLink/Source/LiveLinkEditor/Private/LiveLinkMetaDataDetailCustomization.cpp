// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMetaDataDetailCustomization.h"

#include "LiveLinkTypes.h"

#include "CommonFrameRates.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "LiveLinkMetaDataDetailCustomization"


void FLiveLinkMetaDataDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FLiveLinkMetaData, SceneTime)
			&& CastField<FStructProperty>(ChildHandle->GetProperty()))
		{
			SceneTimeHandle = ChildHandle;
			check(CastFieldChecked<FStructProperty>(SceneTimeHandle->GetProperty())->Struct->GetName() == TEXT("QualifiedFrameTime"));

			IDetailGroup& SceneTimeGroup = StructBuilder.AddGroup("SceneTime", LOCTEXT("SceneTimeLabel", "Scene Time"));

			SceneTimeGroup.AddWidgetRow()
				.NameContent()
				[
					SNew(SBox)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FrameNumber", "Frame Number"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				.ValueContent()
				.MinDesiredWidth(166.0f)
				[
					SNew(STextBlock)
					.Text(this, &FLiveLinkMetaDataDetailCustomization::GetFrameNumber)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];

			SceneTimeGroup.AddWidgetRow()
				.NameContent()
				[
					SNew(SBox)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FrameRate", "Timecode Frame Rate"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				.ValueContent()
				.MinDesiredWidth(166.0f)
				[
					SNew(STextBlock)
					.Text(this, &FLiveLinkMetaDataDetailCustomization::GetTimecodeFrameRate)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];

			SceneTimeGroup.AddWidgetRow()
				.NameContent()
				[
					SNew(SBox)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Timecode", "Timecode"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				.ValueContent()
				.MinDesiredWidth(166.0f)
				[
					SNew(STextBlock)
					.Text(this, &FLiveLinkMetaDataDetailCustomization::GetTimecodeValue)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];
		}
		else
		{
			StructBuilder.AddProperty(ChildHandle).ShowPropertyButtons(false);
		}
	}
}

FText FLiveLinkMetaDataDetailCustomization::GetFrameNumber() const
{
	void* Data;
	FPropertyAccess::Result Result = SceneTimeHandle->GetValueData(Data);
	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	if (Result == FPropertyAccess::Success)
	{
		FQualifiedFrameTime* QualifiedFrameTimePtr = reinterpret_cast<FQualifiedFrameTime*>(Data);
		return FText::AsNumber(QualifiedFrameTimePtr->Time.FrameNumber.Value);
	}

	return LOCTEXT("Failed", "Failed");
}

FText FLiveLinkMetaDataDetailCustomization::GetTimecodeFrameRate() const
{
	void* Data;
	FPropertyAccess::Result Result = SceneTimeHandle->GetValueData(Data);
	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	if (Result == FPropertyAccess::Success)
	{
		FQualifiedFrameTime* QualifiedFrameTimePtr = reinterpret_cast<FQualifiedFrameTime*>(Data);
		if (const FCommonFrameRateInfo* Found = FCommonFrameRates::Find(QualifiedFrameTimePtr->Rate))
		{
			return Found->DisplayName;
		}
		else
		{
			QualifiedFrameTimePtr->Rate.ToPrettyText();
		}
	}

	return LOCTEXT("Failed", "Failed");
}

FText FLiveLinkMetaDataDetailCustomization::GetTimecodeValue() const
{
	void* Data;
	FPropertyAccess::Result Result = SceneTimeHandle->GetValueData(Data);
	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	if (Result == FPropertyAccess::Success)
	{
		FQualifiedFrameTime* QualifiedFrameTimePtr = reinterpret_cast<FQualifiedFrameTime*>(Data);
		return FText::FromString(FTimecode::FromFrameNumber(QualifiedFrameTimePtr->Time.FrameNumber, QualifiedFrameTimePtr->Rate, false).ToString());
	}

	return LOCTEXT("Failed", "Failed");
}

#undef LOCTEXT_NAMESPACE
