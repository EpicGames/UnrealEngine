// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

class FLiveLinkMetaDataDetailCustomization : public IPropertyTypeCustomization
{
public:
	// IDetailCustomization interface
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	FText GetFrameNumber() const;
	FText GetTimecodeValue() const;
	FText GetTimecodeFrameRate() const;
	TSharedPtr<IPropertyHandle> SceneTimeHandle;
};
