// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPanelController.h"
#include "Async/TaskGraphInterfaces.h"
#include "Framework/Commands/UICommandList.h"
#include "Engine/Engine.h"
#include "Features/IModularFeatures.h"
#include "IDetailsView.h"
#include "ILiveLinkDeviceModule.h"
#include "LiveLinkClient.h"
#include "LiveLinkClientCommands.h"
#include "LiveLinkClientPanelViews.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceSubsystem.h"
#include "Misc/ConfigCacheIni.h"
#include "SLiveLinkDataView.h"
#include "Widgets/Layout/SWidgetSwitcher.h"


FLiveLinkPanelController::FLiveLinkPanelController(TAttribute<bool> bInReadOnly)
{
	Client = (FLiveLinkClient*)&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

	OnSourcesChangedHandle = Client->OnLiveLinkSourcesChanged().AddRaw(this, &FLiveLinkPanelController::OnSourcesChangedHandler);
	OnSubjectsChangedHandle = Client->OnLiveLinkSubjectsChanged().AddRaw(this, &FLiveLinkPanelController::OnSubjectsChangedHandler);

	FLiveLinkClientCommands::Register();
	CommandList = MakeShared<FUICommandList>();
	BindCommands();

	FLiveLinkSubjectsView::FSubjectsViewArgs Args;
	Args.bInReadOnly = bInReadOnly;
	Args.bInShowDevices = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni);

	SubjectsView = MakeShared<FLiveLinkSubjectsView>(FLiveLinkSubjectsView::FOnSubjectSelectionChanged::CreateRaw(this, &FLiveLinkPanelController::OnSubjectSelectionChangedHandler), CommandList, Args);

	if (Args.bInShowDevices)
	{
		ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

		TWeakPtr<FLiveLinkSubjectsView> WeakSubjectsView = SubjectsView;

		auto OnDevicesChangedHandler = [WeakSubjectsView](FGuid InDeviceId, ULiveLinkDevice* InDevice)
		{
			if (TSharedPtr<FLiveLinkSubjectsView> View = WeakSubjectsView.Pin())
			{
				View->RefreshSubjects();
			}
		};

		DeviceAddedHandle = Subsystem->OnDeviceAdded().AddLambda(OnDevicesChangedHandler);
		DeviceRemovedHandle = Subsystem->OnDeviceRemoved().AddLambda(OnDevicesChangedHandler);
	}

	FOnFinishedChangingProperties SourcesDelegate;
	SourcesDelegate.AddRaw(this, &FLiveLinkPanelController::OnFinishedChangingSourceProperties);

	SourcesDetailsView = UE::LiveLink::CreateSourcesDetailsView(SourcesDelegate, bInReadOnly);
	SubjectsDetailsView = UE::LiveLink::CreateSubjectsDetailsView(Client, bInReadOnly);
	FOnFinishedChangingProperties Delegate;
	Delegate.AddRaw(this, &FLiveLinkPanelController::OnFinishedChangingDeviceProperties);

	DevicesDetailsView = UE::LiveLink::CreateDevicesDetailsView(MoveTemp(Delegate), bInReadOnly);

	RebuildSubjectList();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Needed until SourcesView is removed in 5.10
FLiveLinkPanelController::~FLiveLinkPanelController()
{
	if (GEngine)
	{
		if (ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>())
		{
			Subsystem->OnDeviceAdded().Remove(DeviceAddedHandle);
			Subsystem->OnDeviceRemoved().Remove(DeviceRemovedHandle);
		}
	}

	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = (FLiveLinkClient*)&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		if (LiveLinkClient)
		{
			LiveLinkClient->OnLiveLinkSourcesChanged().Remove(OnSourcesChangedHandle);
			OnSourcesChangedHandle.Reset();

			LiveLinkClient->OnLiveLinkSubjectsChanged().Remove(OnSubjectsChangedHandle);
			OnSubjectsChangedHandle.Reset();
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FLiveLinkPanelController::OnSubjectSelectionChangedHandler(FLiveLinkSubjectUIEntryPtr SubjectEntry, ESelectInfo::Type SelectInfo)
{
	if (bSelectionChangedGuard)
	{
		return;
	}
	TGuardValue<bool> ReentrantGuard(bSelectionChangedGuard, true);

	SourcesDetailsView->SetObject(nullptr);
	DevicesDetailsView->SetObject(nullptr);
	SubjectsDetailsView->SetSubjectKey(FLiveLinkSubjectKey());

	// Resolve the selected device (nullptr if deselected or non-device selected)
	ULiveLinkDevice* SelectedDevice = nullptr;

	if (SubjectEntry.IsValid())
	{
		if (SubjectEntry->IsSource())
		{
			SourcesDetailsView->SetObject(SubjectEntry->GetSettings());
		}
		else if (SubjectEntry->IsSubject())
		{
			SubjectsDetailsView->SetSubjectKey(SubjectEntry->SubjectKey);
		}
		else if (SubjectEntry->IsDevice())
		{
			if (TStrongObjectPtr<ULiveLinkDevice> Device = SubjectEntry->WeakDevice.Pin())
			{
				DevicesDetailsView->SetObject(Device->GetDeviceSettings());
				SelectedDevice = Device.Get();
			}
		}

		SubjectSelectionChangedDelegate.Broadcast(SubjectEntry->SubjectKey);
	}

	ILiveLinkDeviceModule::Get().OnSelectionChanged().Broadcast(SelectedDevice);
}

void FLiveLinkPanelController::OnFinishedChangingDeviceProperties(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (DevicesDetailsView)
	{
		ULiveLinkDevice* Device = nullptr;

		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DevicesDetailsView->GetSelectedObjects();
		if (ensure(SelectedObjects.Num() > 0))
		{
			if (UObject* SelectedObject = SelectedObjects[0].Get(); ensure(SelectedObject))
			{
				Device = SelectedObject->GetTypedOuter<ULiveLinkDevice>();
			}
		}

		if (ensure(Device))
		{
			Device->OnSettingChanged(InPropertyChangedEvent);
		}
	}
}

void FLiveLinkPanelController::OnFinishedChangingSourceProperties(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	TArray<FLiveLinkSubjectUIEntryPtr> SelectedItems = SubjectsView->SubjectsTreeView->GetSelectedItems();
	for (const FLiveLinkSubjectUIEntryPtr& Item : SelectedItems)
	{
		if (Item->IsSource())
		{
			Client->OnPropertyChanged(Item->SubjectKey.Source, InPropertyChangedEvent);
		}
	}
}

int32 FLiveLinkPanelController::GetCombinedDetailWidgetIndex() const
{
	if (SourcesDetailsView->GetSelectedObjects().Num() > 0)
	{
		return 0;
	}
	
	if (!SubjectsDetailsView->GetSubjectKey().SubjectName.IsNone())
	{
		return 1;
	}

	return 2;
}

TSharedRef<SWidget> FLiveLinkPanelController::GetCombinedDetailsWidget()
{
	return SNew(SWidgetSwitcher)
		.WidgetIndex(this, &FLiveLinkPanelController::GetCombinedDetailWidgetIndex)
		+ SWidgetSwitcher::Slot()
		[
			//[0] Detail view for Source
			SourcesDetailsView.ToSharedRef()
		]
		+ SWidgetSwitcher::Slot()
		[
			// [1] Detail view for Subject, Frame data & Static data
			SubjectsDetailsView.ToSharedRef()
		]
		+ SWidgetSwitcher::Slot()
		[
			// [2] Detail view for Devices
			DevicesDetailsView.ToSharedRef()
		];
}

void FLiveLinkPanelController::BindCommands()
{
	CommandList->MapAction(FLiveLinkClientCommands::Get().RemoveSource,
		FExecuteAction::CreateRaw(this, &FLiveLinkPanelController::HandleRemoveSource),
		FCanExecuteAction::CreateRaw(this, &FLiveLinkPanelController::CanRemoveSource)
	);

	CommandList->MapAction(FLiveLinkClientCommands::Get().RemoveAllSources,
		FExecuteAction::CreateRaw(this, &FLiveLinkPanelController::HandleRemoveAllSources),
		FCanExecuteAction::CreateRaw(this, &FLiveLinkPanelController::HasSource)
	);

	CommandList->MapAction(FLiveLinkClientCommands::Get().RemoveSubject,
		FExecuteAction::CreateRaw(this, &FLiveLinkPanelController::HandleRemoveSubject),
		FCanExecuteAction::CreateRaw(this, &FLiveLinkPanelController::CanRemoveSubject)
	);

	CommandList->MapAction(FLiveLinkClientCommands::Get().RemoveDevice,
		FExecuteAction::CreateRaw(this, &FLiveLinkPanelController::HandleRemoveDevice),
		FCanExecuteAction::CreateRaw(this, &FLiveLinkPanelController::CanRemoveDevice)
	);

	CommandList->MapAction(FLiveLinkClientCommands::Get().PauseSubject,
		FExecuteAction::CreateRaw(this, &FLiveLinkPanelController::HandlePauseSubject),
		FCanExecuteAction::CreateRaw(this, &FLiveLinkPanelController::CanPauseSubject)
	);
}

void FLiveLinkPanelController::OnSourcesChangedHandler()
{
	// Since this can be called from any thread, make sure we only update slate on the game thread.
	FFunctionGraphTask::CreateAndDispatchWhenReady([this]()
	{
		RebuildSubjectList();
	}, TStatId(), nullptr, ENamedThreads::GameThread);
}

void FLiveLinkPanelController::OnSubjectsChangedHandler()
{
	// Since this can be called from any thread, make sure we only update slate on the game thread.
	FFunctionGraphTask::CreateAndDispatchWhenReady([this]()
	{
		RebuildSubjectList();
	}, TStatId(), nullptr, ENamedThreads::GameThread);
}

void FLiveLinkPanelController::RebuildSubjectList()
{
	if (SubjectsView)
	{
		SubjectsView->RefreshSubjects();
	}
}

bool FLiveLinkPanelController::HasSource() const
{
	constexpr bool bIncludeVirtualSources = true;
	return Client->GetDisplayableSources(bIncludeVirtualSources).Num() > 0;
}

bool FLiveLinkPanelController::CanRemoveSource() const
{
	return true;
}

void FLiveLinkPanelController::HandleRemoveSource()
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsView->SubjectsTreeView->GetSelectedItems(Selected);
	for (const FLiveLinkSubjectUIEntryPtr& Entry : Selected)
	{
		if (Entry->IsSource())
		{
			Entry->RemoveFromClient();
		}
	}
}

void FLiveLinkPanelController::HandleRemoveAllSources()
{
	Client->RemoveAllSources();
}

bool FLiveLinkPanelController::CanRemoveSubject() const
{
	return SubjectsView->CanRemoveSubject();
}

bool FLiveLinkPanelController::CanRemoveDevice() const
{
	return true;
}

void FLiveLinkPanelController::HandleRemoveSubject()
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsView->SubjectsTreeView->GetSelectedItems(Selected);
	if (Selected.Num() > 0 && Selected[0])
	{
		Selected[0]->RemoveFromClient();
	}
}

void FLiveLinkPanelController::HandleRemoveDevice()
{
	TArray<FLiveLinkSubjectUIEntryPtr> Selected;
	SubjectsView->SubjectsTreeView->GetSelectedItems(Selected);
	
	for (const FLiveLinkSubjectUIEntryPtr& Entry : Selected)
	{
		if (Entry)
		{
			Entry->RemoveFromClient();
		}
	}
}

bool FLiveLinkPanelController::CanPauseSubject() const
{
	return SubjectsView->CanPauseSubject();
}

void FLiveLinkPanelController::HandlePauseSubject()
{
	SubjectsView->HandlePauseSubject();
}
