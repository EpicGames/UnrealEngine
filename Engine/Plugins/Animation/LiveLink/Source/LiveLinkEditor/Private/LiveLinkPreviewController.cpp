// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPreviewController.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "CameraController.h"
#include "ILiveLinkClient.h"
#include "IPersonaPreviewScene.h"
#include "LiveLinkClientReference.h"
#include "LiveLinkCustomVersion.h"
#include "LiveLinkInstance.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkPreviewController)

const FName EditorCamera(TEXT("EditorActiveCamera"));

class FLiveLinkCameraController : public FEditorCameraController
{
	FLiveLinkClientReference ClientRef;

public:

	virtual void UpdateSimulation(
		const FCameraControllerUserImpulseData& UserImpulseData,
		const float DeltaTime,
		const bool bAllowRecoilIfNoImpulse,
		const float MovementSpeedScale,
		FVector& InOutCameraPosition,
		FVector& InOutCameraEuler,
		float& InOutCameraFOV)
	{
		if (ILiveLinkClient* Client = ClientRef.GetClient())
		{
			TSubclassOf<ULiveLinkRole> SubjectRole = Client->GetSubjectRole_AnyThread(EditorCamera);
			if (SubjectRole)
			{
				//Old plugin will stream EditorCamera as AnimationRole through backward compatibility path. Otherwise, it should be of the Camera Role
				FLiveLinkSubjectFrameData CurrentFrameData;
				if (SubjectRole->IsChildOf(ULiveLinkAnimationRole::StaticClass()))
				{
					if (Client->EvaluateFrame_AnyThread(EditorCamera, ULiveLinkAnimationRole::StaticClass(), CurrentFrameData))
					{
						FLiveLinkAnimationFrameData* FrameData = CurrentFrameData.FrameData.Cast<FLiveLinkAnimationFrameData>();

						if (FrameData->Transforms.Num() > 0)
						{
							FTransform Camera = FrameData->Transforms[0];
							InOutCameraPosition = Camera.GetLocation();
							InOutCameraEuler = Camera.GetRotation().Euler();
							return;
						}
					}
				}
				else
				{
					if (Client->EvaluateFrame_AnyThread(EditorCamera, ULiveLinkTransformRole::StaticClass(), CurrentFrameData))
					{
						FLiveLinkTransformFrameData* FrameData = CurrentFrameData.FrameData.Cast<FLiveLinkTransformFrameData>();

						FTransform Camera = FrameData->Transform;
						InOutCameraPosition = Camera.GetLocation();
						InOutCameraEuler = Camera.GetRotation().Euler();
						return;
					}
				}
			}
		}

		InOutCameraPosition = FVector(0.f);
		InOutCameraEuler = FVector(0.f);
	}

};

void ULiveLinkPreviewController::InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{
	PreviewScene->GetPreviewMeshComponent()->SetAnimInstanceClass(ULiveLinkInstance::StaticClass());

	if (ULiveLinkInstance* LiveLinkInstance = Cast<ULiveLinkInstance>(PreviewScene->GetPreviewMeshComponent()->GetAnimInstance()))
	{
		LiveLinkInstance->SetSubject(LiveLinkSubjectName);
		LiveLinkInstance->SetRetargetAsset(RetargetAsset);
	}
	if (bEnableCameraSync)
	{
		PreviewScene->SetCameraOverride(MakeShared<FLiveLinkCameraController>());
	}
}

void ULiveLinkPreviewController::UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{
	PreviewScene->GetPreviewMeshComponent()->SetAnimInstanceClass(UAnimPreviewInstance::StaticClass());
	PreviewScene->SetCameraOverride(nullptr);
}

void ULiveLinkPreviewController::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	Ar.UsingCustomVersion(FLiveLinkCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		const int32 LiveLinkVersion = Ar.CustomVer(FLiveLinkCustomVersion::GUID);

		if (LiveLinkVersion < FLiveLinkCustomVersion::NewLiveLinkRoleSystem)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			LiveLinkSubjectName.Name = SubjectName_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif //WITH_EDITORONLY_DATA
}

