// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaPreviewSceneController.h"

#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

#include "LiveLinkPreviewController.generated.h"

class ULiveLinkRetargetAsset;

UCLASS()
class ULiveLinkPreviewController : public UPersonaPreviewSceneController
{
public:
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(4.23, "FName SubjectName is deprecated. Use the SubjectName of type FLiveLinkSubjectName instead.")
	UPROPERTY()
	FName SubjectName_DEPRECATED;
#endif //WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Live Link")
	FLiveLinkSubjectName LiveLinkSubjectName;

	UPROPERTY(EditAnywhere, Category = "Live Link")
	bool bEnableCameraSync;

	UPROPERTY(EditAnywhere, NoClear, Category = "Live Link")
	TSubclassOf<ULiveLinkRetargetAsset> RetargetAsset;

	virtual void InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;
	virtual void UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar);
	//~ End UObject Interface

};
