// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSourceSettingsDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ILiveLinkClient.h"
#include "LiveLinkSourceSettings.h"
#include "LiveLinkSubjectSettings.h"
#include "Misc/ConfigCacheIni.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LiveLinkSourceSettingsDetailCustomization"


void FLiveLinkSourceSettingsDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	DetailBuilder = &InDetailBuilder;

	//Get the current settings object being edited
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() != 1)
	{
		return;
	}

	//Make sure it's valid and keep a ref on it
	ULiveLinkSourceSettings* SourceSettings = Cast<ULiveLinkSourceSettings>(ObjectsBeingCustomized[0].Get());
	if (SourceSettings == nullptr)
	{
		return;
	}
	WeakSourceSettings = SourceSettings;

	//Find associated source
	EditedSourceGuid.Invalidate();
	const TArray<FGuid> Sources = LiveLinkClient.GetClient()->GetSources();
	for (const FGuid& SourceGuid : Sources)
	{
		if (SourceSettings == LiveLinkClient.GetClient()->GetSourceSettings(SourceGuid))
		{
			EditedSourceGuid = SourceGuid;
			break;
		}
	}

	const bool bIsVirtualSource = LiveLinkClient.GetClient()->GetVirtualSources().Contains(EditedSourceGuid) || EditedSourceGuid == FGuid();

	TSharedPtr<IPropertyHandle> BufferSettingsPropertyHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkSourceSettings, BufferSettings));
	InDetailBuilder.HideProperty(BufferSettingsPropertyHandle);

	TSharedRef<IPropertyHandle> ModePropertyHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkSourceSettings, Mode));

	void* ModeValuePtr = nullptr;
	FPropertyAccess::Result ModeResult = ModePropertyHandle->GetValueData(ModeValuePtr);
	if (ModeResult == FPropertyAccess::MultipleValues || ModeResult == FPropertyAccess::Fail || ModeValuePtr == nullptr)
	{
		return;
	}

	bool bSupportsBuffering = true;
	TSharedRef<IPropertyHandle> TransmitEvalutedDataHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkSourceSettings, bTransmitEvaluatedData));

	const bool bIsInLiveLinkHubApp = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bCreateLiveLinkHubInstance"), false, GEngineIni);
	if (bIsInLiveLinkHubApp)
	{
		if (bIsVirtualSource)
		{
			// Virtual subject transmit evaluated data by default
			InDetailBuilder.HideProperty(TransmitEvalutedDataHandle);
		}
		else
		{
			bSupportsBuffering = SourceSettings->bTransmitEvaluatedData;
			TransmitEvalutedDataHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FLiveLinkSourceSettingsDetailCustomization::ForceRefresh));
		}
	}
	else
	{
		InDetailBuilder.HideProperty(TransmitEvalutedDataHandle);
	}

	if (!bSupportsBuffering)
	{
		InDetailBuilder.HideProperty(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkSourceSettings, Mode)));
		InDetailBuilder.HideProperty(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkSourceSettings, BufferSettings)));

		// Don't add buffering properties if the source doesn't support it. 
		return;
	}

	const ELiveLinkSourceMode SourceMode = *reinterpret_cast<ELiveLinkSourceMode*>(ModeValuePtr);

	InDetailBuilder.AddPropertyToCategory(ModePropertyHandle);
	ModePropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FLiveLinkSourceSettingsDetailCustomization::ForceRefresh));

	bool bEnableParentSubjects = GConfig->GetBoolOrDefault(TEXT("LiveLink"), TEXT("bEnableParentSubjects"), false, GEngineIni);

	if (!bEnableParentSubjects)
	{
		InDetailBuilder.HideProperty(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkSourceSettings, ParentSubject)));
	}

	IDetailCategoryBuilder& CategoryBuilder = InDetailBuilder.EditCategory("Buffer - Settings");

	CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, MaxNumberOfFrameToBuffered)));

	if (SourceMode == ELiveLinkSourceMode::Timecode)
	{
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, ValidTimecodeFrame)))
			.DisplayName(LOCTEXT("ValidTimecodeFrameDisplayName", "Valid Buffer"));
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, TimecodeFrameOffset)))
			.DisplayName(LOCTEXT("TimecodeFrameOffsetDisplayName", "Offset"));

		TSharedPtr<IPropertyHandle> TimecodeHandle = BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, DetectedFrameRate));
		CategoryBuilder.AddCustomRow(TimecodeHandle->GetPropertyDisplayName(), false)
		.NameContent()
		[
			TimecodeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &FLiveLinkSourceSettingsDetailCustomization::GetFrameRateText)
				.ColorAndOpacity(this, &FLiveLinkSourceSettingsDetailCustomization::GetFrameRateColor)
				.ToolTipText(this, &FLiveLinkSourceSettingsDetailCustomization::GetFrameRateTooltip)
			]
		];
		
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, bUseTimecodeSmoothLatest)));

		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, bKeepAtLeastOneFrame))
			, EPropertyLocation::Advanced);

		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, TimecodeClockOffset))
			, EPropertyLocation::Advanced);

		IDetailCategoryBuilder& SubFrameCategoryBuilder = InDetailBuilder.EditCategory("Sub Frame");
		SubFrameCategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, bGenerateSubFrame)));
		SubFrameCategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, SourceTimecodeFrameRate)));
	}
	else if (SourceMode == ELiveLinkSourceMode::EngineTime)
	{
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, ValidEngineTime)))
			.DisplayName(LOCTEXT("ValidEngineTimeDisplayName", "Valid Buffer"));
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, EngineTimeOffset)))
			.DisplayName(LOCTEXT("EngineTimeOffsetDisplayName", "Offset"));
		
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, bKeepAtLeastOneFrame))
			, EPropertyLocation::Advanced);
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, EngineTimeClockOffset))
			, EPropertyLocation::Advanced);
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, SmoothEngineTimeOffset))
			, EPropertyLocation::Advanced);
	}
	else if (SourceMode == ELiveLinkSourceMode::Latest)
	{
		CategoryBuilder.AddProperty(BufferSettingsPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, LatestOffset)))
			.DisplayName(LOCTEXT("LatestOffsetDisplayName", "Offset"));
	}
}

void FLiveLinkSourceSettingsDetailCustomization::ForceRefresh()
{
	if (DetailBuilder)
	{
		DetailBuilder->ForceRefreshDetails();
	}
}

FText FLiveLinkSourceSettingsDetailCustomization::GetFrameRateText() const
{
	ULiveLinkSourceSettings* CurrentSettings = WeakSourceSettings.Get();
	if (CurrentSettings)
	{
		return CurrentSettings->BufferSettings.DetectedFrameRate.ToPrettyText();
	}

	return FText::GetEmpty();
}

FSlateColor FLiveLinkSourceSettingsDetailCustomization::GetFrameRateColor() const
{
	if (!DoSubjectsHaveSameTimecode())
	{
		return FLinearColor::Yellow;
	}

	return FLinearColor::White;
}

FText FLiveLinkSourceSettingsDetailCustomization::GetFrameRateTooltip() const
{
	if (!DoSubjectsHaveSameTimecode())
	{
		return LOCTEXT("FrameRateTooltip", "Warning - Not all enabled subjects from this source have the same Timecode FrameRate.");
	}

	return FText::GetEmpty();
}

bool FLiveLinkSourceSettingsDetailCustomization::DoSubjectsHaveSameTimecode() const
{
	ULiveLinkSourceSettings* CurrentSettings = WeakSourceSettings.Get();
	if (CurrentSettings && EditedSourceGuid.IsValid())
	{
		FFrameRate FoundFrameRate(1, -1); //Start with an invalid frame rate;
		const bool bIncludeDisabled = false;
		const bool bIncludeVirtuals = false;
		const TArray<FLiveLinkSubjectKey> Subjects = LiveLinkClient.GetClient()->GetSubjects(bIncludeDisabled, bIncludeVirtuals);
		for (const FLiveLinkSubjectKey& Key : Subjects)
		{
			if (Key.Source == EditedSourceGuid)
			{
				const ULiveLinkSubjectSettings* SubjectSettings = Cast<ULiveLinkSubjectSettings>(LiveLinkClient.GetClient()->GetSubjectSettings(Key));
				if (SubjectSettings)
				{
					if (FoundFrameRate.IsValid())
					{
						if (FoundFrameRate != SubjectSettings->FrameRate)
						{
							return false;
						}
					}
					else
					{
						//Stamp first found associated frame rate
						FoundFrameRate = SubjectSettings->FrameRate;
					}
				}
			}
		}
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
