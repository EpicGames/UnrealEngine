// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"


#include "LiveLinkClientReference.h"
#include "Misc/Guid.h"
#include "UObject/WeakObjectPtr.h"

class ULiveLinkSourceSettings;
struct FSlateColor;


/**
* Customizes a FLiveLinkSourceSettingsDetailCustomization
*/
class FLiveLinkSourceSettingsDetailCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance() 
	{
		return MakeShared<FLiveLinkSourceSettingsDetailCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

private:
	void ForceRefresh();

	/** Returns the current frame rate associated with this source */
	FText GetFrameRateText() const;

	/** If all subjects from this source don't have the same timecode, FrameRate will be displayed yellow */
	FSlateColor GetFrameRateColor() const;

	/** Tooltip to advise the user that subjects from this source don't have same FrameRate */
	FText GetFrameRateTooltip() const;

	/** Returns true if enabled subjects from this Source have the same FrameRate  */
	bool DoSubjectsHaveSameTimecode() const;

	IDetailLayoutBuilder* DetailBuilder = nullptr;
	TWeakObjectPtr<ULiveLinkSourceSettings> WeakSourceSettings;
	FGuid EditedSourceGuid;
	FLiveLinkClientReference LiveLinkClient;
};
