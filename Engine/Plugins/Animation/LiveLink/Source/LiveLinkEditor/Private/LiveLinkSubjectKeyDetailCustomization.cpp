// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSubjectKeyDetailCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"


#define LOCTEXT_NAMESPACE "LiveLinkSubjectKeyDetailCustomization"


TSharedRef<IPropertyTypeCustomization> FLiveLinkSubjectKeyDetailCustomization::MakeInstance()
{
	return MakeShareable(new FLiveLinkSubjectKeyDetailCustomization);
}


void FLiveLinkSubjectKeyDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FLiveLinkSubjectKey::StaticStruct());

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125.f)
	.MaxDesiredWidth(400.f)
	[
		SNew(SLiveLinkSubjectRepresentationPicker)
		.ShowRole(false)
		.ShowSource(true)
		.Font(CustomizationUtils.GetRegularFont())
		.HasMultipleValues(this, &FLiveLinkSubjectKeyDetailCustomization::HasMultipleValues)
		.Value(this, &FLiveLinkSubjectKeyDetailCustomization::GetValue)
		.OnValueChanged(this, &FLiveLinkSubjectKeyDetailCustomization::SetValue)
	].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}

void FLiveLinkSubjectKeyDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();
	uint32 NumberOfChild;
	if (PropertyHandle->GetNumChildren(NumberOfChild) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumberOfChild; ++Index)
		{
			TSharedRef<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(Index).ToSharedRef();
			ChildBuilder.AddProperty(ChildPropertyHandle)
				.ShowPropertyButtons(true)
				.IsEnabled(MakeAttributeLambda([=] { return !PropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
		}
	}
}

SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole FLiveLinkSubjectKeyDetailCustomization::GetValue() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	for (const void* RawPtr : RawData)
	{
		if (RawPtr)
		{
			return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole (*reinterpret_cast<const FLiveLinkSubjectKey*>(RawPtr));
		}
	}

	return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole();
}

void FLiveLinkSubjectKeyDetailCustomization::SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue)
{
	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty());

	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);
	FLiveLinkSubjectKey* PreviousValue = reinterpret_cast<FLiveLinkSubjectKey*>(RawData[0]);
	FLiveLinkSubjectKey NewSubjectKey = NewValue.ToSubjectKey();

	FString TextValue;
	StructProperty->Struct->ExportText(TextValue, &NewSubjectKey, PreviousValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	ensure(StructPropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
}

bool FLiveLinkSubjectKeyDetailCustomization::HasMultipleValues() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	TOptional<FLiveLinkSubjectKey> CompareAgainst;
	for (const void* RawPtr : RawData)
	{
		if (RawPtr == nullptr)
		{
			if (CompareAgainst.IsSet())
			{
				return false;
			}
		}
		else
		{
			FLiveLinkSubjectKey ThisValue = *reinterpret_cast<const FLiveLinkSubjectKey*>(RawPtr);

			if (!CompareAgainst.IsSet())
			{
				CompareAgainst = ThisValue;
			}
			else if (!(ThisValue == CompareAgainst.GetValue()))
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
