// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "SLiveLinkSubjectRepresentationPicker.h"

class FLiveLinkSubjectKeyDetailCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:
	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole GetValue() const;
	void SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue);
	bool HasMultipleValues() const;

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};
