// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSubjectNameDetailCustomization.h"

#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"


#define LOCTEXT_NAMESPACE "LiveLinkSubjectNameDetailCustomization"


TSharedRef<IPropertyTypeCustomization> FLiveLinkSubjectNameDetailCustomization::MakeInstance()
{
	return MakeShareable(new FLiveLinkSubjectNameDetailCustomization);
}


void FLiveLinkSubjectNameDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FLiveLinkSubjectName::StaticStruct());

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SLiveLinkSubjectRepresentationPicker)
		.ShowRole(false)
		.Font(CustomizationUtils.GetRegularFont())
		.HasMultipleValues(this, &FLiveLinkSubjectNameDetailCustomization::HasMultipleValues)
		.Value(this, &FLiveLinkSubjectNameDetailCustomization::GetValue)
		.OnValueChanged(this, &FLiveLinkSubjectNameDetailCustomization::SetValue)
	].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}


SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole FLiveLinkSubjectNameDetailCustomization::GetValue() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	for (const void* RawPtr : RawData)
	{
		if (RawPtr)
		{
			FLiveLinkSubjectRepresentation Representation;
			Representation.Subject = *reinterpret_cast<const FLiveLinkSubjectName *>(RawPtr);
			return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole(Representation);
		}
	}

	return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole();
}

void FLiveLinkSubjectNameDetailCustomization::SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue)
{
	if (!StructPropertyHandle->IsValidHandle() || StructPropertyHandle->GetNumOuterObjects() == 0)
	{
		return;
	}

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty());

	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	if (ensure(RawData.Num()))
	{
		FLiveLinkSubjectName* PreviousValue = reinterpret_cast<FLiveLinkSubjectName*>(RawData[0]);
		FLiveLinkSubjectName NewSubjectNameValue = NewValue.Subject;

		FString TextValue;
		StructProperty->Struct->ExportText(TextValue, &NewSubjectNameValue, PreviousValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
		ensure(StructPropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
	}
}

bool FLiveLinkSubjectNameDetailCustomization::HasMultipleValues() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	TOptional<FLiveLinkSubjectName> CompareAgainst;
	for (const void* RawPtr : RawData)
	{
		if (RawPtr == nullptr)
		{
			if (CompareAgainst.IsSet())
			{
				return false;
			}
		}
		else
		{
			FLiveLinkSubjectName ThisValue = *reinterpret_cast<const FLiveLinkSubjectName*>(RawPtr);

			if (!CompareAgainst.IsSet())
			{
				CompareAgainst = ThisValue;
			}
			else if (!(ThisValue == CompareAgainst.GetValue()))
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
