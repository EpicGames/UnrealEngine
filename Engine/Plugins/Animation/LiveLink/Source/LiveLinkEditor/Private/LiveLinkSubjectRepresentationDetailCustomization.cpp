// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSubjectRepresentationDetailCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"


#define LOCTEXT_NAMESPACE "LiveLinkSubjectRepresentationDetailCustomization"


TSharedRef<IPropertyTypeCustomization> FLiveLinkSubjectRepresentationDetailCustomization::MakeInstance()
{
	return MakeShareable(new FLiveLinkSubjectRepresentationDetailCustomization);
}


void FLiveLinkSubjectRepresentationDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	check(CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct == FLiveLinkSubjectRepresentation::StaticStruct());

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125.f)
	.MaxDesiredWidth(400.f)
	[
		SNew(SLiveLinkSubjectRepresentationPicker)
		.ShowRole(true)
		.Font(CustomizationUtils.GetRegularFont())
		.HasMultipleValues(this, &FLiveLinkSubjectRepresentationDetailCustomization::HasMultipleValues)
		.Value(this, &FLiveLinkSubjectRepresentationDetailCustomization::GetValue)
		.OnValueChanged(this, &FLiveLinkSubjectRepresentationDetailCustomization::SetValue)
	].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}

void FLiveLinkSubjectRepresentationDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();
	uint32 NumberOfChild;
	if (PropertyHandle->GetNumChildren(NumberOfChild) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumberOfChild; ++Index)
		{
			TSharedRef<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(Index).ToSharedRef();
			ChildBuilder.AddProperty(ChildPropertyHandle)
				.ShowPropertyButtons(true)
				.IsEnabled(MakeAttributeLambda([=] { return !PropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
		}
	}
}

SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole FLiveLinkSubjectRepresentationDetailCustomization::GetValue() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	for (const void* RawPtr : RawData)
	{
		if (RawPtr)
		{
			return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole(*reinterpret_cast<const FLiveLinkSubjectRepresentation *>(RawPtr));
		}
	}

	return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole();
}

void FLiveLinkSubjectRepresentationDetailCustomization::SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue)
{
	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty());

	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);
	FLiveLinkSubjectRepresentation* PreviousValue = reinterpret_cast<FLiveLinkSubjectRepresentation*>(RawData[0]);
	FLiveLinkSubjectRepresentation NewSubRep = NewValue.ToSubjectRepresentation();

	FString TextValue;
	StructProperty->Struct->ExportText(TextValue, &NewSubRep, PreviousValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	ensure(StructPropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
}

bool FLiveLinkSubjectRepresentationDetailCustomization::HasMultipleValues() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	TOptional<FLiveLinkSubjectRepresentation> CompareAgainst;
	for (const void* RawPtr : RawData)
	{
		if (RawPtr == nullptr)
		{
			if (CompareAgainst.IsSet())
			{
				return false;
			}
		}
		else
		{
			FLiveLinkSubjectRepresentation ThisValue = *reinterpret_cast<const FLiveLinkSubjectRepresentation*>(RawPtr);

			if (!CompareAgainst.IsSet())
			{
				CompareAgainst = ThisValue;
			}
			else if (!(ThisValue == CompareAgainst.GetValue()))
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
