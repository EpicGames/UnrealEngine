// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkTest.h"
#include "InterpolationProcessor/LiveLinkBasicFrameInterpolateProcessor.h"
#include "Misc/AutomationTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkTest)

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLiveLinkInterpolationTest, "System.Engine.Animation.LiveLink.Interpolation", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

/** */
bool FLiveLinkInterpolationTest::RunTest(const FString& Parameters)
{
	const int32 IntA = 1;
	const int32 IntB = 4;
	const int32 IntR = 2;
	const float FloatA = 2.f;
	const float FloatB = 8.f;
	const float FloatR = 4.f;
	const FVector VectorA = FVector(11, 21, 31);
	const FVector VectorB = FVector(14, 24, 34);
	const FVector VectorR = FVector(12, 22, 32);

	FLiveLinkInnerTestInternal InternalA;
	FLiveLinkInnerTestInternal InternalB;

	InternalA.InnerSingleFloat = FloatA * -1.f;
	InternalB.InnerSingleFloat = FloatB * -1.f;
	InternalA.InnerSingleInt = IntA * -1.f;
	InternalB.InnerSingleInt = IntB * -1.f;
	InternalA.InnerVectorDim[0] = VectorA * -1.f;
	InternalB.InnerVectorDim[0] = VectorB * -1.f;
	InternalA.InnerVectorDim[1] = VectorA * -1.f - 100.f;
	InternalB.InnerVectorDim[1] = VectorB * -1.f - 100.f;
	InternalA.InnerFloatDim[0] = FloatA * -1.f;
	InternalB.InnerFloatDim[0] = FloatB * -1.f;
	InternalA.InnerFloatDim[1] = FloatA * -1.f - 100.f;
	InternalB.InnerFloatDim[1] = FloatB * -1.f - 100.f;
	InternalA.InnerIntDim[0] = IntA * -1.f;
	InternalB.InnerIntDim[0] = IntB * -1.f;
	InternalA.InnerIntDim[1] = IntA * -1.f - 100;
	InternalB.InnerIntDim[1] = IntB * -1.f - 100;
	InternalA.InnerIntArray.Add(IntA * -1.f);
	InternalB.InnerIntArray.Add(IntB * -1.f);
	InternalA.InnerIntArray.Add(IntA * -1.f - 100);
	InternalB.InnerIntArray.Add(IntB * -1.f - 100);

	FLiveLinkFrameDataStruct FrameDataStructA(FLiveLinkTestFrameDataInternal::StaticStruct());
	FLiveLinkFrameDataStruct FrameDataStructB(FLiveLinkTestFrameDataInternal::StaticStruct());
	FLiveLinkTestFrameDataInternal& FrameA = *FrameDataStructA.Cast<FLiveLinkTestFrameDataInternal>();
	FLiveLinkTestFrameDataInternal& FrameB = *FrameDataStructB.Cast<FLiveLinkTestFrameDataInternal>();

	FrameA.PropertyValues.Add(FloatA);
	FrameB.PropertyValues.Add(FloatB);
	FrameA.PropertyValues.Add(FloatA +100.f);
	FrameB.PropertyValues.Add(FloatB +100.f);
	FrameA.MetaData.StringMetaData.Add("StringKeyA", "StringValueA");
	FrameB.MetaData.StringMetaData.Add("StringKeyB", "StringValueB");
	FrameA.MetaData.SceneTime.Rate = FFrameRate(24, 1);
	FrameB.MetaData.SceneTime.Rate = FFrameRate(24, 1);
	FrameA.MetaData.SceneTime.Time.FrameNumber = IntA;
	FrameB.MetaData.SceneTime.Time.FrameNumber = IntB;
	FrameA.WorldTime = FLiveLinkWorldTime(1000.0, 0.0);
	FrameB.WorldTime = FLiveLinkWorldTime(4000.0, 0.0);

	FrameA.NotInterpolated = IntA;
	FrameB.NotInterpolated = IntB;
	FrameA.SingleVector = VectorA;
	FrameB.SingleVector = VectorB;
	FrameA.SingleStruct = InternalA;
	FrameB.SingleStruct = InternalB;
	FrameA.SingleFloat = FloatA;
	FrameB.SingleFloat = FloatB;
	FrameA.SingleInt = IntA;
	FrameB.SingleInt = IntB;
	FrameA.VectorArray.Add(VectorA);
	FrameB.VectorArray.Add(VectorB);
	FrameA.VectorArray.Add(VectorA + 100.f);
	FrameB.VectorArray.Add(VectorB + 100.f);
	FrameA.StructArray.Add(InternalA);
	FrameB.StructArray.Add(InternalB);
	FrameA.FloatArray.Add(FloatA);
	FrameB.FloatArray.Add(FloatB);
	FrameA.FloatArray.Add(FloatA + 100.f);
	FrameB.FloatArray.Add(FloatB + 100.f);
	FrameA.IntArray.Add(IntA);
	FrameB.IntArray.Add(IntB);
	FrameA.IntArray.Add(IntA + 100);
	FrameB.IntArray.Add(IntB + 100);

	ULiveLinkBasicFrameInterpolationProcessor* Interpolator = NewObject<ULiveLinkBasicFrameInterpolationProcessor>();
	Interpolator->bInterpolatePropertyValues = true;

	ULiveLinkFrameInterpolationProcessor::FWorkerSharedPtr Worker = Interpolator->FetchWorker();
	TSharedPtr<ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker, ESPMode::ThreadSafe> BasicWorker = StaticCastSharedPtr<ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker>(Worker);

	FLiveLinkFrameDataStruct FrameDataStructR(FLiveLinkTestFrameDataInternal::StaticStruct());
	double BlendFactor = BasicWorker->GetBlendFactor(2000.0, FrameDataStructA, FrameDataStructB);
	ULiveLinkBasicFrameInterpolationProcessor::FLiveLinkBasicFrameInterpolationProcessorWorker::FGenericInterpolateOptions InterpolationOptions;
	BasicWorker->GenericInterpolate(BlendFactor, InterpolationOptions, FrameDataStructA, FrameDataStructB, FrameDataStructR);
	const FLiveLinkTestFrameDataInternal& FrameR = *FrameDataStructR.Cast<FLiveLinkTestFrameDataInternal>();

	auto TestStruct = [=, this](const FLiveLinkInnerTestInternal& FrameR, int32 OuterIndex)
		{
			TestEqual(TEXT("InnerSingleFloat"), FrameR.InnerSingleFloat, FloatR * -1.f - 100 * OuterIndex);
			TestEqual(TEXT("InnerSingleInt"), FrameR.InnerSingleInt, IntR * -1 - 100 * OuterIndex);

			for (int32 Index = 0; Index < 2; ++Index)
			{
				TestEqual(TEXT("InnerVectorDim"), FrameR.InnerVectorDim[Index], VectorR * -1.f - 100 * Index);
				TestEqual(TEXT("InnerFloatDim"), FrameR.InnerFloatDim[Index], FloatR * -1.f - 100 * Index);
				TestEqual(TEXT("InnerIntDim"), FrameR.InnerIntDim[Index], IntR * -1 - 100 * Index);
				TestEqual(TEXT("InnerIntArray"), FrameR.InnerIntArray[Index], IntR * -1 - 100 * Index);
			}
		};

	TestEqual(TEXT("PropertyValues.Num"), FrameR.PropertyValues.Num(), 2);
	for (int32 Index = 0; Index < 2; ++Index)
	{
		TestEqual(TEXT("PropertyValues.At"), FrameR.PropertyValues[Index], FloatR + 100*Index);
	}

	//TestEqual(TEXT("MetaData.StringMetaData"), FrameR.MetaData.StringMetaData, FrameA.MetaData.StringMetaData);
	//TestEqual(TEXT("MetaData.SceneTime"), FrameR.MetaData.SceneTime, FrameA.MetaData.SceneTime);
	TestEqual(TEXT("WorldTime"), FrameR.WorldTime.GetOffsettedTime(), 2000.0);

	TestEqual(TEXT("NotInterpolated"), FrameR.NotInterpolated, FrameA.NotInterpolated);
	TestEqual(TEXT("SingleVector"), FrameR.SingleVector, VectorR);
	TestStruct(FrameR.SingleStruct, 0);
	TestEqual(TEXT("SingleFloat"), FrameR.SingleFloat, FloatR);
	TestEqual(TEXT("SingleInt"), FrameR.SingleInt, IntR);

	TestStruct(FrameR.StructArray[0], 0);
	for(int32 Index = 0; Index < 2; ++Index)
	{
		TestEqual(TEXT("VectorArray"), FrameR.VectorArray[Index], VectorR + 100 * Index);
		TestEqual(TEXT("FloatArray"), FrameR.FloatArray[Index], FloatR + 100 * Index);
		TestEqual(TEXT("IntArray"), FrameR.IntArray[Index], IntR + 100 * Index);
	}

	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS

