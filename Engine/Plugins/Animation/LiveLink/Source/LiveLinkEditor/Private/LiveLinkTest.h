// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"


#include "LiveLinkTest.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FLiveLinkInnerTestInternal
{
	GENERATED_BODY()

	UPROPERTY()
	float InnerSingleFloat = 0.f;

	UPROPERTY()
	int32 InnerSingleInt = 0;

	UPROPERTY()
	FVector InnerVectorDim[2] = {FVector::ZeroVector, FVector::ZeroVector };

	UPROPERTY()
	float InnerFloatDim[2] = {0.f, 0.f};

	UPROPERTY()
	int32 InnerIntDim[2] = {0, 0};

	UPROPERTY()
	TArray<int32> InnerIntArray;
};

USTRUCT()
struct FLiveLinkTestFrameDataInternal : public FLiveLinkBaseFrameData
{
	GENERATED_BODY()

	UPROPERTY()
	float NotInterpolated = 0.f;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	FVector SingleVector = FVector::ZeroVector;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	FLiveLinkInnerTestInternal SingleStruct;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	float SingleFloat = 0.f;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	int32 SingleInt = 0;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	TArray<FVector> VectorArray;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	TArray<FLiveLinkInnerTestInternal> StructArray;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	TArray<float> FloatArray;

	UPROPERTY(BlueprintReadOnly, Category = "Test", Interp)
	TArray<int32> IntArray;
};
