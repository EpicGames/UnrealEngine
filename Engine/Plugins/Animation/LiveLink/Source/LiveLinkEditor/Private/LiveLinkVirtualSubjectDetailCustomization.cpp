// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkVirtualSubjectDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Views/TableViewMetadata.h"
#include "ILiveLinkClient.h"
#include "LiveLinkRole.h"
#include "LiveLinkVirtualSubject.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkBasicRole.h"
#include "ScopedTransaction.h"
#include "SLiveLinkSubjectRepresentationPicker.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "LiveLinkVirtualSubjectDetailsCustomization"


void FLiveLinkVirtualSubjectDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	MyDetailsBuilder = &DetailBuilder;

	for (const TWeakObjectPtr<UObject>& SelectedObject : MyDetailsBuilder->GetSelectedObjects())
	{
		if (ULiveLinkVirtualSubject* Selection = Cast<ULiveLinkVirtualSubject>(SelectedObject.Get()))
		{
			SubjectPtr = Selection;
			break;
		}
	}

	ULiveLinkVirtualSubject* Subject = SubjectPtr.Get();
	if (Subject == nullptr)
	{
		return;
	}

	Client = Subject->GetClient();

	SubjectsPropertyHandle = DetailBuilder.GetProperty(TEXT("Subjects"));

	{
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(SubjectsPropertyHandle->GetProperty());
		check(ArrayProperty);
		FStructProperty* StructProperty = CastField<FStructProperty>(ArrayProperty->Inner);
		check(StructProperty);
		check(StructProperty->Struct == FLiveLinkSubjectName::StaticStruct());
	}

	DetailBuilder.HideProperty(SubjectsPropertyHandle);

	UpdateSubjectList();
	UpdateSelectedSubjects();
	
	TSharedRef<SComboButton> ComboButton = SNew(SComboButton)
		.ButtonContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("SubjectsPickerButtonLabel", "Subjects..."))
		]
		.OnGetMenuContent(this, &FLiveLinkVirtualSubjectDetailCustomization::OnGetVirtualSubjectsMenu);

	IDetailCategoryBuilder& SubjectPropertyGroup = DetailBuilder.EditCategory(*SubjectsPropertyHandle->GetMetaData(TEXT("Category")));

	SubjectPropertyGroup.AddCustomRow(LOCTEXT("SelectedSubjectsLabel", "Selected Subjects"))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("SelectedSubjectsLabel", "Selected Subjects"))
		]
		.ValueContent()
		[
			SAssignNew(SelectedSubjectsListView, SListView<FSubjectEntryPtr>)
				.ListItemsSource(&SelectedSubjectsListItems)
				.SelectionMode(ESelectionMode::None)
				.OnGenerateRow(this, &FLiveLinkVirtualSubjectDetailCustomization::OnGenerateWidgetForSelectedSubjectItem)
		];

	SubjectPropertyGroup.AddCustomRow(LOCTEXT("SubjectsTitleLabel", "Subjects"))
		.ValueContent()
		[
			ComboButton
		];

	TSharedRef<IPropertyHandle> SyncSubjectProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkVirtualSubject, SyncSubject));
	IDetailCategoryBuilder& SyncSubjectPropertyGroup = DetailBuilder.EditCategory(*SyncSubjectProperty->GetMetaData(TEXT("Category")));

	DetailBuilder.HideProperty(SyncSubjectProperty);

	FIsResetToDefaultVisible IsResetToDefaultVisible = FIsResetToDefaultVisible::CreateLambda(
		[](TSharedPtr<IPropertyHandle> PropertyHandle)
		{
			FName Value;
			PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSubjectName, Name))->GetValue(Value);
			return Value != NAME_None;
		});

	FResetToDefaultHandler ResetToDefaultHandler = FResetToDefaultHandler::CreateLambda(
		[](TSharedPtr<IPropertyHandle> PropertyHandle)
		{
			PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLiveLinkSubjectName, Name))->SetValue(NAME_None);
		});

	const FResetToDefaultOverride Override = FResetToDefaultOverride::Create(IsResetToDefaultVisible, ResetToDefaultHandler);

	const FText ToolTipText = LOCTEXT("SyncSubjectToolTipText", "Sync subject will be used to decide when to send this Virtual Subject's data. For smoothest result, choose the subject with the highest rate of update.");

	SyncSubjectPropertyGroup.AddCustomRow(LOCTEXT("SyncSubjectLabel", "Sync Subject"))
		.PropertyHandleList({ SyncSubjectProperty })
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("SyncSubjectLabel", "Sync Subject"))
				.ToolTipText(ToolTipText)
		]
		.ValueContent()
		[
			SAssignNew(SyncSubjectWidget, SLiveLinkSubjectRepresentationPicker)
				.ShowRole(false)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.HasMultipleValues(false)
				.OnGetSubjects(this, &FLiveLinkVirtualSubjectDetailCustomization::OnGetSubjects)
				.Value(this, &FLiveLinkVirtualSubjectDetailCustomization::GetSyncSubject)
				.OnValueChanged(this, &FLiveLinkVirtualSubjectDetailCustomization::SetSyncSubject)
				.ToolTipText(ToolTipText)
		]
		.OverrideResetToDefault(Override);
}

int32 GetArrayPropertyIndex(TSharedPtr<IPropertyHandleArray> ArrayProperty, FName ItemToSearchFor, uint32 NumItems)
{
	for (uint32 Index = 0; Index < NumItems; ++Index)
	{
		TArray<void*> RawData;
		ArrayProperty->GetElement(Index)->AccessRawData(RawData);
		FLiveLinkSubjectName* SubjectNamePtr = reinterpret_cast<FLiveLinkSubjectName*>(RawData[0]);
		if (SubjectNamePtr && SubjectNamePtr->Name == ItemToSearchFor)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

bool FLiveLinkVirtualSubjectDetailCustomization::IsEntrySelected(FSubjectEntryPtr EntryPtr)
{
	const TSharedPtr<IPropertyHandleArray> SubjectsArrayPropertyHandle = SubjectsPropertyHandle->AsArray();
	uint32 NumItems;
	SubjectsArrayPropertyHandle->GetNumElements(NumItems);

	int32 SubjectIndex = GetArrayPropertyIndex(SubjectsArrayPropertyHandle, *EntryPtr, NumItems);

	return SubjectIndex != INDEX_NONE;
}

TSharedRef<ITableRow> FLiveLinkVirtualSubjectDetailCustomization::OnGenerateWidgetForSelectedSubjectItem(FSubjectEntryPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FSlateFontInfo Font = IDetailLayoutBuilder::GetDetailFont();
	Font.Size -= 0.5;

	return SNew(STableRow<FSubjectEntryPtr>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(4.0, 0.0)
			[
				SNew(STextBlock)
					.Text(FText::FromName(*InItem))
					.Font(Font)
					.ColorAndOpacity(this, &FLiveLinkVirtualSubjectDetailCustomization::HandleSubjectItemColor, InItem)
					.ToolTipText(this, &FLiveLinkVirtualSubjectDetailCustomization::HandleSubjectItemToolTip, InItem)
			]
		];
}

TSharedRef<ITableRow> FLiveLinkVirtualSubjectDetailCustomization::OnGenerateWidgetForSubjectItem(FSubjectEntryPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FLiveLinkVirtualSubjectDetailCustomization* CaptureThis = this;

	return SNew(STableRow<FSubjectEntryPtr>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([CaptureThis, InItem]()
				{
					return CaptureThis->IsEntrySelected(InItem) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([CaptureThis, InItem](const ECheckBoxState NewState)
				{
					const TSharedPtr<IPropertyHandleArray> SubjectsArrayPropertyHandle = CaptureThis->SubjectsPropertyHandle->AsArray();
					uint32 NumItems;
					SubjectsArrayPropertyHandle->GetNumElements(NumItems);

					if (NewState == ECheckBoxState::Checked)
					{
						FFormatNamedArguments Arguments;
						Arguments.Add(TEXT("SubjectName"), FText::FromName(*InItem));
						FScopedTransaction Transaction(FText::Format(LOCTEXT("AddSourceToVirtualSubject", "Add {SubjectName} to virtual subject"), Arguments));

						FString TextValue;
						FLiveLinkSubjectName NewSubjectName = *InItem;
						FLiveLinkSubjectName::StaticStruct()->ExportText(TextValue, &NewSubjectName, &NewSubjectName, nullptr, EPropertyPortFlags::PPF_None, nullptr);

						FPropertyAccess::Result Result = SubjectsArrayPropertyHandle->AddItem();
						check(Result == FPropertyAccess::Success);
						Result = SubjectsArrayPropertyHandle->GetElement(NumItems)->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::NotTransactable);
						check(Result == FPropertyAccess::Success);

						// Sync subject handling
						if (TStrongObjectPtr<ULiveLinkVirtualSubject> VSubject = CaptureThis->SubjectPtr.Pin())
						{
							// Automatically set a sync subject if none has been selected.
							if (VSubject->SyncSubject == FLiveLinkSubjectName())
							{
								VSubject->SyncSubject = NewSubjectName;
							}
						}
					}
					else
					{
						int32 RemoveIndex = GetArrayPropertyIndex(SubjectsArrayPropertyHandle, *InItem, NumItems);
						if (RemoveIndex != INDEX_NONE)
						{
							SubjectsArrayPropertyHandle->DeleteItem(RemoveIndex);
						}

						if (TStrongObjectPtr<ULiveLinkVirtualSubject> VSubject = CaptureThis->SubjectPtr.Pin())
						{
							// Clear out the sync subject if we disable it.
							if (VSubject->SyncSubject == *InItem)
							{
								VSubject->SyncSubject = FLiveLinkSubjectName();
							}
						}
					}

					CaptureThis->UpdateSelectedSubjects();
				})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(STextBlock)
					.Text(FText::FromName(*InItem))
					.ColorAndOpacity(this, &FLiveLinkVirtualSubjectDetailCustomization::HandleSubjectItemColor, InItem)
					.ToolTipText(this, &FLiveLinkVirtualSubjectDetailCustomization::HandleSubjectItemToolTip, InItem)
			]
		];
}

FSlateColor FLiveLinkVirtualSubjectDetailCustomization::HandleSubjectItemColor(FSubjectEntryPtr InItem) const
{
	FSlateColor Result = FSlateColor::UseForeground();

	if (ULiveLinkVirtualSubject* Subject = SubjectPtr.Get())
	{
		const FName ThisItem = *InItem.Get();

		TArray<FLiveLinkSubjectKey> SubjectKeys = Client->GetSubjects(/*bIncludeDisabledSubject*/ false, /*bIncludeVirtualSubject*/ false);
		if (!SubjectKeys.ContainsByPredicate([ThisItem](const FLiveLinkSubjectKey& Other) { return Other.SubjectName.Name == ThisItem; }))
		{
			Result = FLinearColor::Red;
		}
	}

	return Result;
}

FText FLiveLinkVirtualSubjectDetailCustomization::HandleSubjectItemToolTip(FSubjectEntryPtr InItem) const
{
	FText Result;

	if (ULiveLinkVirtualSubject* Subject = SubjectPtr.Get())
	{
		const FName ThisItem = *InItem.Get();
		TArray<FLiveLinkSubjectKey> SubjectKeys = Client->GetSubjects(/*bIncludeDisabledSubject*/ false, /*bIncludeVirtualSubject*/ false);
		if (!SubjectKeys.ContainsByPredicate([ThisItem](const FLiveLinkSubjectKey& Other) { return Other.SubjectName.Name == ThisItem; }))
		{
			Result = LOCTEXT("LinkedSubjectToolTip", "This subject was not found in the list of available LiveLink subjects. VirtualSubject might not work properly.");
		}
	}

	return Result;
}

void FLiveLinkVirtualSubjectDetailCustomization::UpdateSubjectList()
{
	SubjectsListItems.Reset();

	if (Client)
	{
		TArray<FLiveLinkSubjectKey> SubjectKeys = Client->GetSubjects(/*bIncludeDisabledSubject*/ false, /*bIncludeVirtualSubject*/ false);
		for (const FLiveLinkSubjectKey& SubjectKey : SubjectKeys)
		{
			TSubclassOf<ULiveLinkRole> LiveLinkRole = Client->GetSubjectRole_AnyThread(SubjectKey);
			if (LiveLinkRole && (LiveLinkRole->IsChildOf(ULiveLinkAnimationRole::StaticClass()) || LiveLinkRole->IsChildOf(ULiveLinkBasicRole::StaticClass())))
			{
				SubjectsListItems.AddUnique(MakeShared<FName>(SubjectKey.SubjectName.Name));
			}
		}
	}

	if (ULiveLinkVirtualSubject* Subject = SubjectPtr.Get())
	{
		// In case one of the associated subject linked to this virtual one doesn't exist anymore, add it to the list to display it red
		for (const FLiveLinkSubjectName& SelectedSubject : Subject->GetSubjects())
		{
			if (!SubjectsListItems.ContainsByPredicate([SelectedSubject](const FSubjectEntryPtr& Other) { return *Other == SelectedSubject.Name; }))
			{
				SubjectsListItems.AddUnique(MakeShared<FName>(SelectedSubject.Name));
			}
		}
	}
}

TSharedRef<SWidget> FLiveLinkVirtualSubjectDetailCustomization::OnGetVirtualSubjectsMenu()
{
	UpdateSubjectList();
	UpdateSelectedSubjects();

	return SNew(SListView<FSubjectEntryPtr>)
		.ListItemsSource(&SubjectsListItems)
		.OnGenerateRow(this, &FLiveLinkVirtualSubjectDetailCustomization::OnGenerateWidgetForSubjectItem);
}

void FLiveLinkVirtualSubjectDetailCustomization::UpdateSelectedSubjects()
{
	SelectedSubjectsListItems.Reset();
	for (FSubjectEntryPtr EntryPtr : SubjectsListItems)
	{
		if (IsEntrySelected(EntryPtr))
		{
			SelectedSubjectsListItems.Add(EntryPtr);
		}
	}

	if (TStrongObjectPtr<ULiveLinkVirtualSubject> VSubject = SubjectPtr.Pin())
	{
		if (!VSubject->GetSubjects().Contains(VSubject->SyncSubject))
		{
			VSubject->SyncSubject = FLiveLinkSubjectName();
		}
	}

	if (SelectedSubjectsListView)
	{
		SelectedSubjectsListView->RequestListRefresh();
	}
}

void FLiveLinkVirtualSubjectDetailCustomization::OnGetSubjects(TArray<FLiveLinkSubjectKey>& OutSubjectsList) const
{
	if (TStrongObjectPtr<ULiveLinkVirtualSubject> VSubject = SubjectPtr.Pin())
	{
		const TArray<FLiveLinkSubjectName>& Subjects = VSubject->GetSubjects();
		OutSubjectsList.Reset(Subjects.Num());
		Algo::Transform(Subjects, OutSubjectsList, [](FLiveLinkSubjectName Name) { return FLiveLinkSubjectKey{ FGuid(), Name }; });
	}
}

SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole FLiveLinkVirtualSubjectDetailCustomization::GetSyncSubject() const
{
	SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole Role;
	if (TStrongObjectPtr<ULiveLinkVirtualSubject> VSubject = SubjectPtr.Pin())
	{
		Role.Subject = VSubject->SyncSubject;
	}

	return Role;
}

void FLiveLinkVirtualSubjectDetailCustomization::SetSyncSubject(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole Role)
{
	if (TStrongObjectPtr<ULiveLinkVirtualSubject> VSubject = SubjectPtr.Pin())
	{
		VSubject->Modify();
		VSubject->SyncSubject = Role.Subject;
	}
}

#undef LOCTEXT_NAMESPACE
