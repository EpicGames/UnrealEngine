// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "SLiveLinkSubjectRepresentationPicker.h"

#include "Widgets/Views/SListView.h"

class ULiveLinkVirtualSubject;


class ILiveLinkClient;
class IPropertyHandle;
class ITableRow;
class STableViewBase;


/**
* Customizes a ULiveLinkVirtualSubjectDetails
*/
class FLiveLinkVirtualSubjectDetailCustomization : public IDetailCustomization
{
public:

	// Data type of out subject tree UI
	typedef TSharedPtr<FName> FSubjectEntryPtr;

	static TSharedRef<IDetailCustomization> MakeInstance() 
	{
		return MakeShared<FLiveLinkVirtualSubjectDetailCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

private:
	// Generates a row for a selected subject.
	TSharedRef<ITableRow> OnGenerateWidgetForSelectedSubjectItem(FSubjectEntryPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

	// Creates subject tree entry widget
	TSharedRef<ITableRow> OnGenerateWidgetForSubjectItem(FSubjectEntryPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

	// If Item doesn't exist in Client subject's list, mark it as red
	FSlateColor HandleSubjectItemColor(FSubjectEntryPtr InItem) const;

	FText HandleSubjectItemToolTip(FSubjectEntryPtr InItem) const;

	// Generate the combo box menu to display subjects to pick.
	TSharedRef<SWidget> OnGetVirtualSubjectsMenu();

	// Returns whether an entry is currently selected.
	bool IsEntrySelected(FSubjectEntryPtr EntryPtr);

	// Update the list of LiveLink subjects.
	void UpdateSubjectList();

	// Update the list of selected subjects displayed above the subject picker.
	void UpdateSelectedSubjects();

	// Returns the list of subjects that constitute the virtual subject.
	void OnGetSubjects(TArray<FLiveLinkSubjectKey>& OutSubjectsList) const;

	// Returns the subject that synchronizes the virtual subject.
	SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole GetSyncSubject() const;

	// Sets the SyncSubject property on the virtual subject,
	void SetSyncSubject(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole Role);
private:
	// The tile set being edited
	TWeakObjectPtr<ULiveLinkVirtualSubject> SubjectPtr;

	ILiveLinkClient* Client;

	// Cached reference to our details builder so we can force refresh
	IDetailLayoutBuilder* MyDetailsBuilder;

	// Cached property pointers
	TSharedPtr<IPropertyHandle> SubjectsPropertyHandle;

	// Cached data for the subject tree UI
	TArray<FSubjectEntryPtr> SubjectsListItems;
	TArray<FSubjectEntryPtr> SelectedSubjectsListItems;
	TSharedPtr< SListView< FSubjectEntryPtr > > SelectedSubjectsListView;

	// LiveLink subject picker that restricts to the subjects that make up the virtual subject.
	TSharedPtr<class SLiveLinkSubjectRepresentationPicker> SyncSubjectWidget;
};
