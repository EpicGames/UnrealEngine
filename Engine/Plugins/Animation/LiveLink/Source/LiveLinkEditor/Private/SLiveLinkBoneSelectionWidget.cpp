// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkBoneSelectionWidget.h"

#include "DetailLayoutBuilder.h"
#include "Engine/SkeletalMeshSocket.h"
#include "IEditableSkeleton.h"
#include "ILiveLinkClient.h"
#include "LiveLinkSubjectSettings.h"
#include "LiveLinkVirtualSubject.h"
#include "Features/IModularFeatures.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Styling/AppStyle.h"
#include "Translator/LiveLinkTransformRoleToAnimation.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STreeView.h"


#define LOCTEXT_NAMESPACE "SLiveLinkBoneSelectionWidget"

/////////////////////////////////////////////////////
void SLiveLinkBoneTreeMenu::Construct(const FArguments& InArgs, TOptional<FLiveLinkSkeletonStaticData> InSkeletonStaticData)
{
	OnSelectionChangedDelegate = InArgs._OnBoneSelectionChanged;
	
	if (InSkeletonStaticData)
	{
		SkeletonStaticData = MoveTemp(*InSkeletonStaticData);
	}

	FText TitleToUse = !InArgs._Title.IsEmpty() ? InArgs._Title  : LOCTEXT("BonePickerTitle", "Select...");

	SAssignNew(TreeView, STreeView<TSharedPtr<FBoneNameInfo>>)
		.TreeItemsSource(&SkeletonTreeInfo)
		.OnGenerateRow(this, &SLiveLinkBoneTreeMenu::MakeTreeRowWidget)
		.OnGetChildren(this, &SLiveLinkBoneTreeMenu::GetChildrenForInfo)
		.OnSelectionChanged(this, &SLiveLinkBoneTreeMenu::OnSelectionChanged)
		.SelectionMode(ESelectionMode::Single);

	RebuildBoneList(InArgs._SelectedBone);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(6.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Content()
		[
			SNew(SBox)
			.WidthOverride(300.f)
			.HeightOverride(512.f)
			.Content()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("BoldFont"))
					.Text(TitleToUse)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.SeparatorImage(FAppStyle::GetBrush("Menu.Separator"))
					.Orientation(Orient_Horizontal)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(FilterTextWidget, SSearchBox)
					.SelectAllTextWhenFocused(true)
					.OnTextChanged(this, &SLiveLinkBoneTreeMenu::OnFilterTextChanged)
					.HintText(NSLOCTEXT("BonePicker", "Search", "Search..."))
				]
				+ SVerticalBox::Slot()
				[
					TreeView->AsShared()
				]
			]
		]
	];
}

TSharedPtr<SWidget> SLiveLinkBoneTreeMenu::GetFilterTextWidget()
{
	return FilterTextWidget;
}

TSharedRef<ITableRow> SLiveLinkBoneTreeMenu::MakeTreeRowWidget(TSharedPtr<FBoneNameInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FBoneNameInfo>>, OwnerTable)
		.Content()
		[
			SNew(STextBlock)
			.HighlightText(FilterText)
			.Text(FText::FromName(InInfo->BoneName))
		];
}

void SLiveLinkBoneTreeMenu::GetChildrenForInfo(TSharedPtr<FBoneNameInfo> InInfo, TArray< TSharedPtr<FBoneNameInfo> >& OutChildren)
{
	OutChildren = InInfo->Children;
}

void SLiveLinkBoneTreeMenu::OnFilterTextChanged(const FText& InFilterText)
{
	FilterText = InFilterText;

	RebuildBoneList(NAME_None);
}

void SLiveLinkBoneTreeMenu::OnSelectionChanged(TSharedPtr<SLiveLinkBoneTreeMenu::FBoneNameInfo> BoneInfo, ESelectInfo::Type SelectInfo)
{
	//Because we recreate all our items on tree refresh we will get a spurious null selection event initially.
	if (BoneInfo.IsValid() && SelectInfo == ESelectInfo::OnMouseClick)
	{
		SelectBone(BoneInfo);
	}
}

void SLiveLinkBoneTreeMenu::SelectBone(TSharedPtr<SLiveLinkBoneTreeMenu::FBoneNameInfo> BoneInfo)
{
	OnSelectionChangedDelegate.ExecuteIfBound(BoneInfo->BoneName);
}

FReply SLiveLinkBoneTreeMenu::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(InKeyEvent.GetKey() == EKeys::Enter)
	{
		TArray<TSharedPtr<SLiveLinkBoneTreeMenu::FBoneNameInfo>> SelectedItems = TreeView->GetSelectedItems();
		if(SelectedItems.Num() > 0)
		{
			SelectBone(SelectedItems[0]);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SLiveLinkBoneTreeMenu::RebuildBoneList(const FName& SelectedBone)
{
	SkeletonTreeInfo.Empty();
	SkeletonTreeInfoFlat.Empty();

	const int32 MaxBone = SkeletonStaticData.BoneNames.Num(); 

	for (int32 BoneIdx = 0; BoneIdx < MaxBone; ++BoneIdx)
	{
		const FName BoneName = SkeletonStaticData.BoneNames[BoneIdx];
		TSharedRef<FBoneNameInfo> BoneInfo = MakeShared<FBoneNameInfo>(BoneName);

		// Filter if Necessary
		if (!FilterText.IsEmpty() && !BoneInfo->BoneName.ToString().Contains(FilterText.ToString()))
		{
			continue;
		}

		int32 ParentIdx = SkeletonStaticData.BoneParents[BoneIdx];
		bool bAddToParent = false;

		if (ParentIdx != INDEX_NONE && FilterText.IsEmpty())
		{
			// We have a parent, search for it in the flat list
			FName ParentName = SkeletonStaticData.BoneNames[ParentIdx];

			for (int32 FlatListIdx = 0; FlatListIdx < SkeletonTreeInfoFlat.Num(); ++FlatListIdx)
			{
				TSharedPtr<FBoneNameInfo> InfoEntry = SkeletonTreeInfoFlat[FlatListIdx];
				if (InfoEntry->BoneName == ParentName)
				{
					bAddToParent = true;
					ParentIdx = FlatListIdx;
					break;
				}
			}

			if (bAddToParent)
			{
				SkeletonTreeInfoFlat[ParentIdx]->Children.Add(BoneInfo);
			}
			else
			{
				SkeletonTreeInfo.Add(BoneInfo);
			}
		}
		else
		{
			SkeletonTreeInfo.Add(BoneInfo);
		}

		SkeletonTreeInfoFlat.Add(BoneInfo);
		TreeView->SetItemExpansion(BoneInfo, true);
		if (BoneName == SelectedBone)
		{
			TreeView->SetItemSelection(BoneInfo, true);
			TreeView->RequestScrollIntoView(BoneInfo);
		}
	}

	TreeView->RequestTreeRefresh();
}

/////////////////////////////////////////////////////

void SLiveLinkBoneSelectionWidget::Construct(const FArguments& InArgs, const FLiveLinkSubjectKey& InSubjectKey)
{
	OnBoneSelectionChanged = InArgs._OnBoneSelectionChanged;
	OnGetSelectedBone = InArgs._OnGetSelectedBone;
	SubjectKey = InSubjectKey;

	ChildSlot
	[
		SAssignNew(BonePickerButton, SComboButton)
		.OnGetMenuContent(FOnGetContent::CreateSP(this, &SLiveLinkBoneSelectionWidget::CreateSkeletonWidgetMenu))
		.ContentPadding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SLiveLinkBoneSelectionWidget::GetCurrentBoneName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

TSharedRef<SWidget> SLiveLinkBoneSelectionWidget::CreateSkeletonWidgetMenu()
{
	FName CurrentBoneName;
	if (OnGetSelectedBone.IsBound())
	{
		CurrentBoneName = OnGetSelectedBone.Execute();
	}

	ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	bool bIncludeVirtualSubject = true;
	bool bIncludeDisabledSubject = true;

	FName TranslatorOutputBoneName;
	TOptional<FLiveLinkSkeletonStaticData> SkeletonData;

	/** Get a list of name of subjects supporting a certain role */
	TArray<FLiveLinkSubjectKey> Subjects = LiveLinkClient->GetSubjectsSupportingRole(ULiveLinkAnimationRole::StaticClass(), bIncludeDisabledSubject, bIncludeVirtualSubject);

	if (FLiveLinkSubjectKey* Key = Subjects.FindByPredicate([this](const FLiveLinkSubjectKey& Other) { return Other.SubjectName == SubjectKey.SubjectName; }))
	{
		if (const FLiveLinkStaticDataStruct* StaticData = LiveLinkClient->GetSubjectStaticData_AnyThread(*Key))
		{
			if (const FLiveLinkSkeletonStaticData* SkeletonStaticData = StaticData->Cast<FLiveLinkSkeletonStaticData>())
			{
				SkeletonData = *SkeletonStaticData;
			}
		}

		if (!SkeletonData)
		{
			SkeletonData = MakeStaticDataFromTranslator(*Key);
		}
	}

	TSharedRef<SLiveLinkBoneTreeMenu> MenuWidget = SNew(SLiveLinkBoneTreeMenu, MoveTemp(SkeletonData))
		.OnBoneSelectionChanged(this, &SLiveLinkBoneSelectionWidget::OnSelectionChanged)
		.SelectedBone(CurrentBoneName);

	BonePickerButton->SetMenuContentWidgetToFocus(MenuWidget->GetFilterTextWidget());

	return MenuWidget;
}

void SLiveLinkBoneSelectionWidget::OnSelectionChanged(FName BoneName)
{
	//Because we recreate all our items on tree refresh we will get a spurious null selection event initially.
	if (OnBoneSelectionChanged.IsBound())
	{
		OnBoneSelectionChanged.Execute(BoneName);
	}

	BonePickerButton->SetIsOpen(false);
}

FText SLiveLinkBoneSelectionWidget::GetCurrentBoneName() const
{
	if (OnGetSelectedBone.IsBound())
	{
		FName Name = OnGetSelectedBone.Execute();
		return FText::FromName(Name);
	}

	return FText::GetEmpty();
}

void SLiveLinkBoneSelectionWidget::SetSubject(const FLiveLinkSubjectKey& InSubjectKey)
{
	SubjectKey = InSubjectKey;
}


FLiveLinkSkeletonStaticData SLiveLinkBoneSelectionWidget::MakeStaticDataFromTranslator(const FLiveLinkSubjectKey& InSubjectKey) const
{
	ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	UObject* Settings = LiveLinkClient->GetSubjectSettings(InSubjectKey);
	TArray<ULiveLinkFrameTranslator*> Translators;

	if (ULiveLinkSubjectSettings* SubjectSettings = Cast<ULiveLinkSubjectSettings>(Settings))
	{
		Translators = SubjectSettings->Translators;
	}
	else if (ULiveLinkVirtualSubject* VirtualSubject = Cast<ULiveLinkVirtualSubject>(Settings))
	{
		Translators = VirtualSubject->GetTranslators();
	}

	FName OutputBoneName;
	for (ULiveLinkFrameTranslator* Translator : Translators)
	{
		if (ULiveLinkTransformRoleToAnimation* TransformTranslator = Cast<ULiveLinkTransformRoleToAnimation>(Translator))
		{
			OutputBoneName = TransformTranslator->OutputBoneName;
			break;
		}
	}

	FLiveLinkSkeletonStaticData StaticData;
	StaticData.BoneParents = { INDEX_NONE };
	StaticData.BoneNames = { OutputBoneName };

	return StaticData;
}


#undef LOCTEXT_NAMESPACE
