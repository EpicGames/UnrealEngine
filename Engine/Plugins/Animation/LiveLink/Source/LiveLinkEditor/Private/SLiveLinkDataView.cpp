// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkDataView.h"

#include "DetailsViewArgs.h"
#include "LiveLinkMetaDataDetailCustomization.h"
#include "IDetailsView.h"
#include "LiveLinkClient.h"
#include "LiveLinkRole.h"
#include "LiveLinkSubjectSettings.h"

#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"


#define LOCTEXT_NAMESPACE "LiveLinkDataView"


void SLiveLinkDataView::Construct(const FArguments& Args, FLiveLinkClient* InClient)
{
	Client = InClient;
	LastUpdateSeconds = 0.0;
	UpdateDelay = 1.0;
	DetailType = EDetailType::Property;

	FMenuBuilder DetailViewOptions(true, nullptr);
	DetailViewOptions.BeginSection("Properties", LOCTEXT("Properties", "Properties"));
	{
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowSubjectDetail", "Show Subject Properties"),
			LOCTEXT("ShowSubjectDetail_ToolTip", "Displays the subject properties"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLiveLinkDataView::OnSelectDetailWidget, EDetailType::Property),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLiveLinkDataView::IsSelectedDetailWidget, EDetailType::Property)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowStaticDataDetail", "Show Static Data"),
			LOCTEXT("ShowStaticData_ToolTip", "Displays the subject static data"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLiveLinkDataView::OnSelectDetailWidget, EDetailType::StaticData),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLiveLinkDataView::IsSelectedDetailWidget, EDetailType::StaticData)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowFrameDataDetail", "Show Frame Data"),
			LOCTEXT("ShowFrameData_ToolTip", "Displays the subject frame data"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLiveLinkDataView::OnSelectDetailWidget, EDetailType::FrameData),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLiveLinkDataView::IsSelectedDetailWidget, EDetailType::FrameData)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	DetailViewOptions.EndSection();

	DetailViewOptions.BeginSection("Settings", LOCTEXT("Settings", "Settings"));
	{
		TSharedRef<SWidget> RefreshDelay = SNew(SSpinBox<double>)
			.MinValue(0.0)
			.MaxValue(10.0)
			.MinSliderValue(0.0)
			.MaxSliderValue(10.0)
			.ToolTipText(LOCTEXT("RefreshDelay_ToolTip", "Refresh delay of Static & Frame data."))
			.Value(this, &SLiveLinkDataView::GetRefreshDelay)
			.OnValueCommitted(this, &SLiveLinkDataView::SetRefreshDelayInternal)
			.IsEnabled(this, &SLiveLinkDataView::CanEditRefreshDelay);

		DetailViewOptions.AddWidget(RefreshDelay, LOCTEXT("RefreshDelay", "Refresh Delay"));
	}
	DetailViewOptions.EndSection();

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	DetailsViewArgs.bShowCustomFilterOption = false;
	DetailsViewArgs.bShowOptions = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	SettingsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	SettingsDetailsView->OnFinishedChangingProperties().AddSP(this, &SLiveLinkDataView::OnPropertyChanged);
	SettingsDetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([bReadOnly = Args._ReadOnly](){ return !bReadOnly.Get(); }));

	FStructureDetailsViewArgs StructViewArgs;
	StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructViewArgs, TSharedPtr<FStructOnScope>());
	StructureDetailsView->GetDetailsView()->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([]() { return false; }));
	StructureDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(FLiveLinkMetaData::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([](){ return MakeShared<FLiveLinkMetaDataDetailCustomization>(); })
		);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNullWidget::NullWidget
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SComboButton)
				.ContentPadding(0)
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.MenuContent()
				[
					DetailViewOptions.MakeWidget()
				]
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 2, 0)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visibility").GetIcon())
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 0, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ViewButton", "View Options"))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0, 2.0))
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &SLiveLinkDataView::GetDetailWidgetIndex)
			+ SWidgetSwitcher::Slot()
			[
				SettingsDetailsView.ToSharedRef()
			]
			+ SWidgetSwitcher::Slot()
			[
				StructureDetailsView->GetWidget().ToSharedRef()
			]
		]
	];
}

void SLiveLinkDataView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (DetailType != EDetailType::Property && InCurrentTime - LastUpdateSeconds > UpdateDelay)
	{
		LastUpdateSeconds = InCurrentTime;

		if (Client->IsSubjectEnabled(SubjectKey, true))
		{
			TSubclassOf<ULiveLinkRole> SubjectRole = Client->GetSubjectRole_AnyThread(SubjectKey.SubjectName);
			if (SubjectRole)
			{
				FLiveLinkSubjectFrameData SubjectData;
				if (Client->EvaluateFrame_AnyThread(SubjectKey.SubjectName, SubjectRole, SubjectData))
				{
					if (DetailType == EDetailType::StaticData)
					{
						TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(SubjectData.StaticData.GetStruct());
						CastChecked<UScriptStruct>(StructOnScope->GetStruct())->CopyScriptStruct(StructOnScope->GetStructMemory(), SubjectData.StaticData.GetBaseData());
						StructureDetailsView->SetStructureData(StructOnScope);
					}
					else
					{
						TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(SubjectData.FrameData.GetStruct());
						CastChecked<UScriptStruct>(StructOnScope->GetStruct())->CopyScriptStruct(StructOnScope->GetStructMemory(), SubjectData.FrameData.GetBaseData());
						StructureDetailsView->SetStructureData(StructOnScope);
					}
				}
			}
		}
	}
}

void SLiveLinkDataView::SetSubjectKey(FLiveLinkSubjectKey InSubjectKey)
{
	SubjectKey = InSubjectKey;

	UObject* SettingsObject = Client->GetSubjectSettings(SubjectKey);
	SettingsDetailsView->SetObject(SettingsObject);
}

int32 SLiveLinkDataView::GetDetailWidgetIndex() const
{
	return DetailType == EDetailType::Property ? 0 : 1;
}

void SLiveLinkDataView::OnSelectDetailWidget(EDetailType InDetailType)
{
	DetailType = InDetailType;
}

void SLiveLinkDataView::OnPropertyChanged(const FPropertyChangedEvent& InEvent)
{
	Client->OnPropertyChanged(SubjectKey.Source, InEvent);
}


#undef LOCTEXT_NAMESPACE
