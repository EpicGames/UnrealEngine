// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkSubjectNameGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "K2Node_EvaluateLiveLinkFrame.h"


void SLiveLinkSubjectNameGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget> SLiveLinkSubjectNameGraphPin::GetDefaultValueWidget()
{
	//Create widget
	return SNew(SLiveLinkSubjectRepresentationPicker)
		.ShowRole(false)
		.HasMultipleValues(false)
		.Value(this, &SLiveLinkSubjectNameGraphPin::GetValue)
		.OnValueChanged(this, &SLiveLinkSubjectNameGraphPin::SetValue)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
}

SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole SLiveLinkSubjectNameGraphPin::GetValue() const
{
	FLiveLinkSubjectRepresentation SubjectRepresentation;

	if (!GraphPinObj->GetDefaultAsString().IsEmpty())
	{
		FLiveLinkSubjectName::StaticStruct()->ImportText(*GraphPinObj->GetDefaultAsString(), &SubjectRepresentation.Subject, nullptr, EPropertyPortFlags::PPF_None, GLog, FLiveLinkSubjectName::StaticStruct()->GetName());
		SubjectRepresentation.Role = Cast<UClass>(GraphPinObj->DefaultObject);
	}

	return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole(SubjectRepresentation);
}

void SLiveLinkSubjectNameGraphPin::SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue)
{
	FString ValueString;
	FLiveLinkSubjectName::StaticStruct()->ExportText(ValueString, &NewValue.Subject, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString);

	if (UK2Node_EvaluateLiveLinkFrame* OnwingNode = Cast<UK2Node_EvaluateLiveLinkFrame>(GraphPinObj->GetOwningNode()))
	{
		UEdGraphPin* RolePin = OnwingNode->GetLiveLinkRolePin();
		RolePin->GetSchema()->TrySetDefaultObject(*RolePin, NewValue.Role.Get());
	}
}

