// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkSubjectRepresentationGraphPin.h"
#include "EdGraph/EdGraphSchema.h"

#define LOCTEXT_NAMESPACE "SubjectRepresentation"

void SLiveLinkSubjectRepresentationGraphPin::Construct( const FArguments& InArgs, UEdGraphPin* InGraphPinObj )
{
	SGraphPin::Construct( SGraphPin::FArguments(), InGraphPinObj );
}

TSharedRef<SWidget>	SLiveLinkSubjectRepresentationGraphPin::GetDefaultValueWidget()
{
	FString DefaultString = GraphPinObj->GetDefaultAsString();
	FLiveLinkSubjectRepresentation::StaticStruct()->ImportText(*DefaultString, &SubjectRepresentation, nullptr, EPropertyPortFlags::PPF_None, GLog, FLiveLinkSubjectRepresentation::StaticStruct()->GetName());

	//Create widget
	return SNew(SLiveLinkSubjectRepresentationPicker)
		.ShowRole(true)
		.HasMultipleValues(false)
		.Value(this, &SLiveLinkSubjectRepresentationGraphPin::GetValue)
		.OnValueChanged(this, &SLiveLinkSubjectRepresentationGraphPin::SetValue);
}

SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole SLiveLinkSubjectRepresentationGraphPin::GetValue() const
{
	return SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole(SubjectRepresentation);
}

void SLiveLinkSubjectRepresentationGraphPin::SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue)
{
	SubjectRepresentation = NewValue.ToSubjectRepresentation();

	FString ValueString;
	FLiveLinkSubjectRepresentation::StaticStruct()->ExportText(ValueString, &SubjectRepresentation, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString);
}

#undef LOCTEXT_NAMESPACE
