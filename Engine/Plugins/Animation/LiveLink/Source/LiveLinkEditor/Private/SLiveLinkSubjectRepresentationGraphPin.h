// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "SLiveLinkSubjectRepresentationPicker.h"


struct FLiveLinkSubjectRepresentation;

class SComboButton;

class SLiveLinkSubjectRepresentationGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkSubjectRepresentationGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:
	SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole GetValue() const;
	void SetValue(SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue);

	FLiveLinkSubjectRepresentation SubjectRepresentation;
};
