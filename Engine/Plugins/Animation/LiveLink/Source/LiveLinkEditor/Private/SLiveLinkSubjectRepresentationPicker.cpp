// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkSubjectRepresentationPicker.h"

#include "Features/IModularFeatures.h"
#include "Framework/Views/TableViewMetadata.h"
#include "ILiveLinkClient.h"
#include "LiveLinkEditorPrivate.h"
#include "LiveLinkPreset.h"
#include "LiveLinkPresetTypes.h"
#include "Misc/FeedbackContext.h"
#include "PropertyCustomizationHelpers.h"
#include "Styling/SlateIconFinder.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SLiveLinkSubjectRepresentationPicker"

namespace SubjectUI
{
	static const FName EnabledColumnName(TEXT("Enabled"));
	static const FName SourceColumnName(TEXT("Source"));
	static const FName NameColumnName(TEXT("Name"));
	static const FName RoleColumnName(TEXT("Role"));
};

struct FLiveLinkSubjectRepresentationPickerEntry
{
	FLiveLinkSubjectRepresentationPickerEntry() = default;
	FLiveLinkSubjectRepresentationPickerEntry(const SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole& InSourceSubjectRole, const FText & InSourceType, bool bInEnabled)
		: SourceSubjectRole(InSourceSubjectRole)
		, SourceType(InSourceType)
		, bEnabled(bInEnabled)
	{}

	SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole SourceSubjectRole;
	FText SourceType;
	bool bEnabled;
};

class SLiveLinkSubjectEntryRow : public SMultiColumnTableRow<FLiveLinkSubjectRepresentationPickerEntryPtr>
{
public:
	SLATE_BEGIN_ARGS(SLiveLinkSubjectEntryRow) {}
		SLATE_ARGUMENT(FLiveLinkSubjectRepresentationPickerEntryPtr, Entry)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		EntryPtr = Args._Entry;

		SMultiColumnTableRow<FLiveLinkSubjectRepresentationPickerEntryPtr>::Construct(
			FSuperRowType::FArguments()
			.Padding(0.f),
			OwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == SubjectUI::EnabledColumnName)
		{
			return SNew(SCheckBox)
				.IsChecked(EntryPtr->bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.IsEnabled(false);
		}
		else if (ColumnName == SubjectUI::SourceColumnName)
		{
			return	SNew(STextBlock)
				.Text(EntryPtr->SourceType);
		}
		else if (ColumnName == SubjectUI::NameColumnName)
		{
			return	SNew(STextBlock)
				.Text(FText::FromName(EntryPtr->SourceSubjectRole.Subject));
		}
		else if (ColumnName == SubjectUI::RoleColumnName)
		{
			if (EntryPtr->SourceSubjectRole.Role)
			{
				return SNew(STextBlock)
					.Text(EntryPtr->SourceSubjectRole.Role->GetDefaultObject<ULiveLinkRole>()->GetDisplayName());
			}
		}

		return SNullWidget::NullWidget;
	}

private:
	FLiveLinkSubjectRepresentationPickerEntryPtr EntryPtr;
};

void SLiveLinkSubjectRepresentationPicker::Construct(const FArguments& InArgs)
{
	ValueAttribute = InArgs._Value;
	OnValueChangedDelegate = InArgs._OnValueChanged;
	HasMultipleValuesAttribute = InArgs._HasMultipleValues;
	bShowSource = InArgs._ShowSource;
	bShowRole = InArgs._ShowRole;
	GetSubjectsDelegate = InArgs._OnGetSubjects;


	SubjectRepData.Reset();
	SelectedLiveLinkPreset.Reset();

	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	CachedSourceType = LiveLinkClient.GetSourceType(ValueAttribute.Get().Source);

	TSharedPtr<SWidget> ComboButtonContent;
	if (bShowRole || bShowSource)
	{
		TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox);
		if (bShowSource)
		{
			ContentBox->AddSlot()
			[
				SNew(STextBlock)
				.Font(InArgs._Font)
				.Text(this, &SLiveLinkSubjectRepresentationPicker::GetSourceNameValueText)
			];
		}

		ContentBox->AddSlot()
		[
			SNew(STextBlock)
			.Font(InArgs._Font)
			.Text(this, &SLiveLinkSubjectRepresentationPicker::GetSubjectNameValueText)
		];

		if (bShowRole)
		{
			ContentBox->AddSlot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SLiveLinkSubjectRepresentationPicker::GetRoleIcon)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(InArgs._Font)
					.Text(this, &SLiveLinkSubjectRepresentationPicker::GetRoleText)
				]
			];
		}

		SAssignNew(ComboButtonContent, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[	
			ContentBox
		];
	}
	else
	{
		ComboButtonContent = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(FMargin(0, 0, 5, 0))
		[
			SNew(SEditableTextBox)
			.Text(this, &SLiveLinkSubjectRepresentationPicker::GetSubjectNameValueText)
			.OnTextCommitted(this, &SLiveLinkSubjectRepresentationPicker::OnComboTextCommitted)
			.SelectAllTextWhenFocused(true)
			.SelectAllTextOnCommit(true)
			.ClearKeyboardFocusOnCommit(false)
			.Font(InArgs._Font)
			.Style(FLiveLinkEditorPrivate::GetStyleSet(), "EditableTextBox")
		];
	}

	ChildSlot
	[
		SAssignNew(PickerComboButton, SComboButton)
		.ComboButtonStyle(InArgs._ComboButtonStyle)
		.ButtonStyle(InArgs._ButtonStyle)
		.ForegroundColor(FSlateColor::UseForeground())
		.ContentPadding(InArgs._ContentPadding)
		.VAlign(VAlign_Center)
		.OnGetMenuContent(this, &SLiveLinkSubjectRepresentationPicker::BuildMenu)
		.ButtonContent()
		[
			ComboButtonContent.ToSharedRef()
		]
	];
}

SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole SLiveLinkSubjectRepresentationPicker::GetCurrentValue() const
{
	return ValueAttribute.Get();
}

FText SLiveLinkSubjectRepresentationPicker::GetSourceNameValueText() const
{
	const bool bHasMultipleValues = HasMultipleValuesAttribute.Get();
	if (bHasMultipleValues)
	{
		return LOCTEXT("MultipleValuesText", "<multiple values>");
	}

	return CachedSourceType;
}

FText SLiveLinkSubjectRepresentationPicker::GetSubjectNameValueText() const
{
	const bool bHasMultipleValues = HasMultipleValuesAttribute.Get();
	if (bHasMultipleValues)
	{
		return LOCTEXT("MultipleValuesText", "<multiple values>");
	}

	return FText::FromName(ValueAttribute.Get().Subject);
}

const FSlateBrush* SLiveLinkSubjectRepresentationPicker::GetRoleIcon() const
{
	const bool bHasMultipleValues = HasMultipleValuesAttribute.Get();
	FLiveLinkSourceSubjectRole CurrentSrcSubRole = ValueAttribute.Get();
	if (!bHasMultipleValues && CurrentSrcSubRole.Role != nullptr)
	{
		return FSlateIconFinder::FindIconBrushForClass(CurrentSrcSubRole.Role);
	}
	return FSlateIconFinder::FindIconBrushForClass(ULiveLinkRole::StaticClass());
}

FText SLiveLinkSubjectRepresentationPicker::GetRoleText() const
{
	const bool bHasMultipleValues = HasMultipleValuesAttribute.Get();
	if (bHasMultipleValues)
	{
		return LOCTEXT("MultipleValuesText", "<multiple values>");
	}

	FLiveLinkSourceSubjectRole CurrentSrcSubjRole = ValueAttribute.Get();
	if (CurrentSrcSubjRole.Role == nullptr)
	{
		return LOCTEXT("NoValueText", "<none>");
	}
	return CurrentSrcSubjRole.Role->GetDisplayNameText();
}

TSharedRef<SWidget> SLiveLinkSubjectRepresentationPicker::BuildMenu()
{
	SubjectRepData.Reset();
	SelectedLiveLinkPreset.Reset();
	BuildSubjectRepDataList();

	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column(SubjectUI::EnabledColumnName)
		.ManualWidth(20.f)
		.DefaultLabel(FText());

	if (bShowSource)
	{
		HeaderRow->AddColumn(SHeaderRow::Column(SubjectUI::SourceColumnName)
			.DefaultLabel(LOCTEXT("SourceColumnHeaderName", "Source"))
			.FillWidth(30));
	}
	HeaderRow->AddColumn(SHeaderRow::Column(SubjectUI::NameColumnName)
		.FillWidth(bShowSource ? 50 : 60.f)
		.DefaultLabel(LOCTEXT("SubjectColumnHeaderName", "Subject")));
	HeaderRow->AddColumn(SHeaderRow::Column(SubjectUI::RoleColumnName)
		.FillWidth(bShowSource ? 20 : 40.f)
		.DefaultLabel(LOCTEXT("RoleColumnHeaderName", "Role")));

	return SNew(SBox)
		.Padding(0)
		.WidthOverride(300.f)
		.HeightOverride(300.f)
		[
			SNew(SBorder)
			.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					// Current Preset
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					.Padding(8, 0)
					[
						SAssignNew(SelectPresetComboButton, SComboButton)
						.ContentPadding(0)
						.ForegroundColor(this, &SLiveLinkSubjectRepresentationPicker::GetSelectPresetForegroundColor)
						.ButtonStyle(FAppStyle::Get(), "ToggleButton") // Use the tool bar item style for this button
						.OnGetMenuContent(this, &SLiveLinkSubjectRepresentationPicker::BuildPresetSubMenu)
						.ButtonContent()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(FLiveLinkEditorPrivate::GetStyleSet()->GetBrush("LiveLinkClient.Common.Icon.Small"))
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.f)
							.Padding(2, 0, 0, 0)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(this, &SLiveLinkSubjectRepresentationPicker::GetPresetSelectedText)
							]
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ContentPadding(0)
						.ButtonStyle(FAppStyle::Get(), "ToggleButton") // Use the tool bar item style for this button
						.OnClicked(this, &SLiveLinkSubjectRepresentationPicker::ClearCurrentPreset)
						.IsEnabled(this, &SLiveLinkSubjectRepresentationPicker::HasCurrentPreset)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
						]
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SBorder)
					.Padding(FMargin(4.0f, 4.0f))
					[
						SAssignNew(SubjectListView, SListView<FLiveLinkSubjectRepresentationPickerEntryPtr>)
						.ListItemsSource(&SubjectRepData)
						.SelectionMode(ESelectionMode::Single)
						.OnGenerateRow(this, &SLiveLinkSubjectRepresentationPicker::MakeSubjectRepListViewWidget)
						.OnSelectionChanged(this, &SLiveLinkSubjectRepresentationPicker::OnSubjectRepListSelectionChanged)
						.HeaderRow(HeaderRow)
					]
				]
			]
		];
}

FText SLiveLinkSubjectRepresentationPicker::GetPresetSelectedText() const
{
	ULiveLinkPreset* LiveLinkPresetPtr = SelectedLiveLinkPreset.Get();
	if (LiveLinkPresetPtr)
	{
		return FText::FromName(LiveLinkPresetPtr->GetFName());
	}
	return LOCTEXT("SelectAPresetLabel", "<No Preset Selected>");
}

FSlateColor SLiveLinkSubjectRepresentationPicker::GetSelectPresetForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");
	static const FName DefaultForegroundName("DefaultForeground");
	TSharedPtr<SComboButton> SelectPresetComboButtonPin = SelectPresetComboButton.Pin();
	return (SelectPresetComboButtonPin.IsValid() && SelectPresetComboButtonPin->IsHovered()) ? FAppStyle::GetSlateColor(InvertedForegroundName) : FAppStyle::GetSlateColor(DefaultForegroundName);
}

FReply SLiveLinkSubjectRepresentationPicker::ClearCurrentPreset()
{
	SelectedLiveLinkPreset.Reset();
	BuildSubjectRepDataList();

	return FReply::Handled();
}

bool SLiveLinkSubjectRepresentationPicker::HasCurrentPreset() const
{
	return SelectedLiveLinkPreset.IsValid();
}

TSharedRef<ITableRow> SLiveLinkSubjectRepresentationPicker::MakeSubjectRepListViewWidget(FLiveLinkSubjectRepresentationPickerEntryPtr Entry, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SLiveLinkSubjectEntryRow, OwnerTable)
		.Entry(Entry);
}

void SLiveLinkSubjectRepresentationPicker::OnSubjectRepListSelectionChanged(FLiveLinkSubjectRepresentationPickerEntryPtr Entry, ESelectInfo::Type SelectionType)
{
	if (Entry.IsValid())
	{
		SetValue(Entry->SourceSubjectRole);
		CachedSourceType = Entry->SourceType;
	}
	else
	{
		SetValue(FLiveLinkSourceSubjectRole());
		CachedSourceType = FText::GetEmpty();
	}
}

TSharedRef<SWidget> SLiveLinkSubjectRepresentationPicker::BuildPresetSubMenu()
{
	ULiveLinkPreset* LiveLinkPresetPtr = SelectedLiveLinkPreset.Get();
	FAssetData CurrentAssetData = LiveLinkPresetPtr ? FAssetData(LiveLinkPresetPtr) : FAssetData();

	TArray<const UClass*> ClassFilters;
	ClassFilters.Add(ULiveLinkPreset::StaticClass());

	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.AddWidget(
		PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
			FAssetData(),
			false,
			false,
			ClassFilters,
			TArray<UFactory*>(),
			FOnShouldFilterAsset::CreateLambda([CurrentAssetData](const FAssetData& InAssetData) { return InAssetData == CurrentAssetData; }),
			FOnAssetSelected::CreateRaw(this, &SLiveLinkSubjectRepresentationPicker::NewPresetSelected),
			FSimpleDelegate()
		),
		FText::GetEmpty(),
		true,
		false
	);
	return MenuBuilder.MakeWidget();
}

void SLiveLinkSubjectRepresentationPicker::NewPresetSelected(const FAssetData& AssetData)
{
	GWarn->BeginSlowTask(LOCTEXT("MediaProfileLoadPackage", "Loading Media Profile"), true, false);
	ULiveLinkPreset* LiveLinkPresetPtr = Cast<ULiveLinkPreset>(AssetData.GetAsset());
	SelectedLiveLinkPreset = LiveLinkPresetPtr;

	BuildSubjectRepDataList();

	TSharedPtr<SComboButton> SelectPresetComboButtonPin = SelectPresetComboButton.Pin();
	if (SelectPresetComboButtonPin)
	{
		SelectPresetComboButtonPin->SetIsOpen(false);
	}

	GWarn->EndSlowTask();
}

void SLiveLinkSubjectRepresentationPicker::OnComboTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	FLiveLinkSourceSubjectRole SourceSubjectRole;
	SourceSubjectRole.Subject.Name = *NewText.ToString();
	SetValue(SourceSubjectRole);

	CachedSourceType = FText::GetEmpty();
}

void SLiveLinkSubjectRepresentationPicker::SetValue(const SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole& InValue)
{
	if (OnValueChangedDelegate.IsBound())
	{
		OnValueChangedDelegate.ExecuteIfBound(InValue);
	}
	else if (ValueAttribute.IsBound())
	{
		ValueAttribute = InValue;
	}

	TSharedPtr<SComboButton> PickerComboButtonPin = PickerComboButton.Pin();
	if (PickerComboButtonPin.IsValid())
	{
		PickerComboButtonPin->SetIsOpen(false);
	}
}

void SLiveLinkSubjectRepresentationPicker::BuildSubjectRepDataList()
{
	SubjectRepData.Reset();

	ULiveLinkPreset* LiveLinkPresetPtr = SelectedLiveLinkPreset.Get();
	if (LiveLinkPresetPtr)
	{
		for (const FLiveLinkSubjectPreset& SubjectPreset : LiveLinkPresetPtr->GetSubjectPresets())
		{
			SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole SrcSubRole;
			SrcSubRole.Source = SubjectPreset.Key.Source;
			SrcSubRole.Subject = SubjectPreset.Key.SubjectName;
			SrcSubRole.Role = SubjectPreset.Role;
			const FLiveLinkSourcePreset* FoundSourcePreset = nullptr;
			if (bShowSource)
			{
				const FGuid& SourceGuid = SubjectPreset.Key.Source;
				FoundSourcePreset = LiveLinkPresetPtr->GetSourcePresets().FindByPredicate([&SourceGuid](const FLiveLinkSourcePreset& SourcePreset) { return SourcePreset.Guid == SourceGuid; });
			}

			if (SrcSubRole.Role != nullptr && !SrcSubRole.Subject.IsNone())
			{
				if (FoundSourcePreset)
				{
					SubjectRepData.Add(MakeShared<FLiveLinkSubjectRepresentationPickerEntry>(SrcSubRole, FoundSourcePreset->SourceType, SubjectPreset.bEnabled));
				}
				else
				{
					SubjectRepData.Add(MakeShared<FLiveLinkSubjectRepresentationPickerEntry>(SrcSubRole, FText::GetEmpty(), SubjectPreset.bEnabled));
				}
			}
		}
	}
	else if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		TArray<FLiveLinkSubjectKey> SubjectKeys;

		if (GetSubjectsDelegate.IsBound())
		{
			GetSubjectsDelegate.Execute(SubjectKeys);
		}
		else
		{
			SubjectKeys = LiveLinkClient.GetSubjects(true, true);
		}

		if (bShowSource)
		{
			TMap<FGuid, FText> UniqueSources;
			for (const FLiveLinkSubjectKey& Key : SubjectKeys)
			{
				if (!UniqueSources.Contains(Key.Source))
				{
					UniqueSources.Add(Key.Source, LiveLinkClient.GetSourceType(Key.Source));
				}
			}

			for (const FLiveLinkSubjectKey& SubjectKey : SubjectKeys)
			{
				if (!SubjectKey.SubjectName.IsNone())
				{
					bool bEnabled = LiveLinkClient.IsSubjectEnabled(SubjectKey, true);
					SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole SrcSubRole;
					SrcSubRole.Source = SubjectKey.Source;
					SrcSubRole.Subject = SubjectKey.SubjectName;
					SrcSubRole.Role = LiveLinkClient.GetSubjectRole_AnyThread(SubjectKey);
					SubjectRepData.Add(MakeShared<FLiveLinkSubjectRepresentationPickerEntry>(SrcSubRole, UniqueSources[SubjectKey.Source], bEnabled));
				}
			}

			struct FEntryLess
			{
				FORCEINLINE bool operator()(const FLiveLinkSubjectRepresentationPickerEntryPtr& A, const FLiveLinkSubjectRepresentationPickerEntryPtr& B) const
				{
					return A->SourceSubjectRole.Source < B->SourceSubjectRole.Source && A->SourceSubjectRole.Subject.Name.Compare(B->SourceSubjectRole.Subject.Name) < 0;
				}
			};
			SubjectRepData.Sort(FEntryLess());
		}
		else
		{
			TMap<FLiveLinkSubjectName, int32> UniqueSubjectName;
			for(const FLiveLinkSubjectKey& Key : SubjectKeys)
			{
				if (!Key.SubjectName.IsNone())
				{
					int32* FoundValue = UniqueSubjectName.Find(Key.SubjectName);
					if (FoundValue)
					{
						++(*FoundValue);
					}
					else
					{
						UniqueSubjectName.Add(Key.SubjectName, 1);
					}
				}
			}

			for (const auto& Item : UniqueSubjectName)
			{
				bool bEnabled = LiveLinkClient.IsSubjectEnabled(Item.Key);

				SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole SrcSubRole;
				SrcSubRole.Subject = Item.Key;
				SrcSubRole.Role = LiveLinkClient.GetSubjectRole_AnyThread(Item.Key);

				if (SrcSubRole.Role == nullptr && Item.Value != 1)
				{
					bool bIsFirstEntry = true;
					for (const FLiveLinkSubjectKey& Key : SubjectKeys)
					{
						if (Key.SubjectName == Item.Key)
						{
							 TSubclassOf<ULiveLinkRole> NewRole = LiveLinkClient.GetSubjectRole_AnyThread(Key);
							 if (!bIsFirstEntry && SrcSubRole.Role != NewRole)
							 {
								 SrcSubRole.Role = nullptr;
								 break;
							 }

							 bIsFirstEntry = false;
							 SrcSubRole.Role = NewRole;
						}
					}
				}
				SubjectRepData.Add(MakeShared<FLiveLinkSubjectRepresentationPickerEntry>(SrcSubRole, FText::GetEmpty(), bEnabled));
			}

			struct FEntryLess
			{
				FORCEINLINE bool operator()(const FLiveLinkSubjectRepresentationPickerEntryPtr& A, const FLiveLinkSubjectRepresentationPickerEntryPtr& B) const
				{
					return A->SourceSubjectRole.Subject.Name.Compare(B->SourceSubjectRole.Subject.Name) < 0;
				}
			};
			SubjectRepData.Sort(FEntryLess());
		}
	}

	TSharedPtr<SListView<FLiveLinkSubjectRepresentationPickerEntryPtr>> SubjectListViewPin = SubjectListView.Pin();
	if (SubjectListViewPin.IsValid())
	{
		SubjectListViewPin->RebuildList();
	}
}

#undef LOCTEXT_NAMESPACE
