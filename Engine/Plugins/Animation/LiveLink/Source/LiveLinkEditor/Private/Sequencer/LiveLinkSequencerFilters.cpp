// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSequencerFilters.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "LiveLinkComponent.h"
#include "MovieScene/MovieSceneLiveLinkTrack.h"
#include "Styling/SlateIconFinder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkSequencerFilters)

#define LOCTEXT_NAMESPACE "LiveLinkSequencerTrackFilters"

class FSequencerTrackFilter_LiveLinkFilterCommands
	: public TCommands<FSequencerTrackFilter_LiveLinkFilterCommands>
{
public:
	FSequencerTrackFilter_LiveLinkFilterCommands()
		: TCommands<FSequencerTrackFilter_LiveLinkFilterCommands>(
			TEXT("FSequencerTrackFilter_LiveLink"),
			LOCTEXT("FSequencerTrackFilter_LiveLink", "Live Link Filters"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{}

	TSharedPtr<FUICommandInfo> ToggleFilter_LiveLink;

	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleFilter_LiveLink, "Toggle Live Link Filter", "Toggle the filter for Live Link tracks", EUserInterfaceActionType::ToggleButton, FInputChord());
	}
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_LiveLink : public FSequencerTrackFilter_ClassType<UMovieSceneLiveLinkTrack>
{
public:
	FSequencerTrackFilter_LiveLink(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr)
		:  FSequencerTrackFilter_ClassType<UMovieSceneLiveLinkTrack>(InFilterInterface, InCategory)
		, BindingCount(0)
	{
		FSequencerTrackFilter_LiveLinkFilterCommands::Register();
	}

	virtual ~FSequencerTrackFilter_LiveLink() override
	{
		BindingCount--;
		if (BindingCount < 1)
		{
			FSequencerTrackFilter_LiveLinkFilterCommands::Unregister();
		}
	}

	//~ Begin IFilter
	virtual FString GetName() const override { return TEXT("LiveLink"); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override { return LOCTEXT("SequencerTrackFilter_LiveLink", "Live Link"); }
	virtual FSlateIcon GetIcon() const override { return FSlateIconFinder::FindIconForClass(ULiveLinkComponent::StaticClass()); }
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter

	virtual FText GetDefaultToolTipText() const override
	{
		return LOCTEXT("SequencerTrackFilter_LiveLinkToolTip", "Show only Live Link tracks");
	}

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override
	{
		return FSequencerTrackFilter_LiveLinkFilterCommands::Get().ToggleFilter_LiveLink;
	}

	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override
	{
		return IsSequenceTrackSupported<UMovieSceneLiveLinkTrack>(InSequence);
	}

	//~ End FSequencerTrackFilter

private:
	mutable uint32 BindingCount;
};

//////////////////////////////////////////////////////////////////////////
//

void ULiveLinkSequencerTrackFilter::AddTrackFilterExtensions(ISequencerTrackFilters& InFilterInterface, const TSharedRef<FFilterCategory>& InPreferredCategory, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const
{
	InOutFilterList.Add(MakeShared<FSequencerTrackFilter_LiveLink>(InFilterInterface, InPreferredCategory));
}

#undef LOCTEXT_NAMESPACE
