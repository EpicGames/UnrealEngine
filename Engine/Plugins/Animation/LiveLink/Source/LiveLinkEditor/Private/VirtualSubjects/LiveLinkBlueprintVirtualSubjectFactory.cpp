// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualSubjects/LiveLinkBlueprintVirtualSubjectFactory.h"

#include "AssetToolsModule.h"
#include "ClassViewerFilter.h"
#include "Editor.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "IAssetTools.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "LiveLinkRole.h"
#include "VirtualSubjects/LiveLinkBlueprintVirtualSubject.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkBlueprintVirtualSubjectFactory)


#define LOCTEXT_NAMESPACE "LiveLinkBlueprintVirtualSubjectFactory"

ULiveLinkBlueprintVirtualSubjectFactory::ULiveLinkBlueprintVirtualSubjectFactory()
{
	SupportedClass = UBlueprint::StaticClass();
	ParentClass = ULiveLinkBlueprintVirtualSubject::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

FText ULiveLinkBlueprintVirtualSubjectFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Blueprint Virtual Subject");
}

bool ULiveLinkBlueprintVirtualSubjectFactory::ConfigureProperties()
{
	class FLiveLinkRoleClassFilter : public IClassViewerFilter
	{
	public:
		FLiveLinkRoleClassFilter()
		{
			AllowedChildrenOfClasses.Add(ULiveLinkRole::StaticClass());
			DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
		};

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override
		{
			bool bIsCorrectClass = InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
			bool bValidClassFlags = !InClass->HasAnyClassFlags(DisallowedClassFlags);

			if (bIsCorrectClass && bValidClassFlags)
			{
				ULiveLinkRole* ClassRole = InClass->GetDefaultObject<ULiveLinkRole>();
				UScriptStruct* StaticStruct = ClassRole->GetStaticDataStruct();
				UScriptStruct* FrameStruct = ClassRole->GetFrameDataStruct();
				return UEdGraphSchema_K2::IsAllowableBlueprintVariableType(StaticStruct) && UEdGraphSchema_K2::IsAllowableBlueprintVariableType(FrameStruct);
			}
			return false;
		};

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return false;
		};

	private:
		/** All children of these classes will be included unless filtered out by another setting. */
		TSet< const UClass* > AllowedChildrenOfClasses;

		/** Disallowed class flags. */
		EClassFlags DisallowedClassFlags;
	};

	class FVirtualSubjectFactoryUI : public TSharedFromThis<FVirtualSubjectFactoryUI>
	{
	public:
		FReply OnCreate()
		{
			check(ResultRole);
			if (PickerWindow.IsValid())
			{
				PickerWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}

		FReply OnCancel()
		{
			ResultRole = nullptr;
			if (PickerWindow.IsValid())
			{
				PickerWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}

		bool IsRoleSelected() const
		{
			return ResultRole != nullptr;
		}

		void OnPickedRole(UClass* ChosenRole)
		{
			ResultRole = ChosenRole;
			StructPickerAnchor->SetIsOpen(false);
		}

		FText OnGetComboTextValue() const
		{
			return ResultRole
				? FText::AsCultureInvariant(ResultRole->GetName())
				: LOCTEXT("None", "None");
		}

		TSharedRef<SWidget> GenerateClassPicker()
		{
			FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

			// Fill in options
			FClassViewerInitializationOptions Options;
			Options.Mode = EClassViewerMode::ClassPicker;
			Options.ClassFilters.Add(MakeShared<FLiveLinkRoleClassFilter>());

			return
				SNew(SBox)
				.WidthOverride(330)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.MaxHeight(500)
					[
						SNew(SBorder)
						.Padding(4)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &FVirtualSubjectFactoryUI::OnPickedRole))
						]
					]
				];
		}

		TSubclassOf<ULiveLinkRole> OpenRoleSelector()
		{
			FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("StructViewer");
			ResultRole = nullptr;

			// Fill in options
			FClassViewerInitializationOptions Options;
			Options.Mode = EClassViewerMode::ClassPicker;
			Options.ClassFilters.Add(MakeShared<FLiveLinkRoleClassFilter>());

			PickerWindow = SNew(SWindow)
				.Title(LOCTEXT("PickerTitle", "Select Role"))
				.ClientSize(FVector2D(350, 100))
				.SupportsMinimize(false)
				.SupportsMaximize(false)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Menu.Background"))
					.Padding(10)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(StructPickerAnchor, SComboButton)
							.ContentPadding(FMargin(2, 2, 2, 1))
							.MenuPlacement(MenuPlacement_BelowAnchor)
							.ButtonContent()
							[
								SNew(STextBlock)
								.Text(this, &FVirtualSubjectFactoryUI::OnGetComboTextValue)
							]
							.OnGetMenuContent(this, &FVirtualSubjectFactoryUI::GenerateClassPicker)
						]
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Right)
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.Text(LOCTEXT("OK", "OK"))
								.IsEnabled(this, &FVirtualSubjectFactoryUI::IsRoleSelected)
								.OnClicked(this, &FVirtualSubjectFactoryUI::OnCreate)
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.Text(LOCTEXT("Cancel", "Cancel"))
								.OnClicked(this, &FVirtualSubjectFactoryUI::OnCancel)
							]
						]
					]
				];

			GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
			PickerWindow.Reset();

			return ResultRole;
		}

	private:
		TSharedPtr<SWindow> PickerWindow;
		TSharedPtr<SComboButton> StructPickerAnchor;
		TSubclassOf<ULiveLinkRole> ResultRole = nullptr;
	};

	TSharedRef<FVirtualSubjectFactoryUI> RoleSelector = MakeShared<FVirtualSubjectFactoryUI>();
	Role = RoleSelector->OpenRoleSelector();

	return Role != nullptr;
}

UObject* ULiveLinkBlueprintVirtualSubjectFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	UBlueprint* VirtualSubject = nullptr;
	if (Role && ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));
		VirtualSubject = FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext);
		if (TSubclassOf<UObject> GeneratedClass = VirtualSubject->GeneratedClass)
		{
			if (ULiveLinkBlueprintVirtualSubject* DefaultSubject = GeneratedClass->GetDefaultObject<ULiveLinkBlueprintVirtualSubject>())
			{
				DefaultSubject->Role = Role;
			}
		}
		FBlueprintEditorUtils::MarkBlueprintAsModified(VirtualSubject);
	}
	return VirtualSubject;
}

bool ULiveLinkBlueprintVirtualSubjectFactory::ShouldShowInNewMenu() const
{
	return true;
}

uint32 ULiveLinkBlueprintVirtualSubjectFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}

TArray<FAssetCategoryPath> ULiveLinkBlueprintVirtualSubjectFactory::GetAssetMenuPathsForCategory(FName InCategory) const
{
	if (ParentClass)
	{
		return GetAssetMenuPaths(ParentClass, InCategory);
	}
	return Super::GetAssetMenuPathsForCategory(InCategory);
}

#undef  LOCTEXT_NAMESPACE
