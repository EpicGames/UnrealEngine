// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"

#define UE_API LIVELINKEDITOR_API


class ULiveLinkPreset;
template <typename T> class TSubclassOf;

class FMenuBuilder;
class FLiveLinkClient;
class ILiveLinkSource;
class IMenu;
class SComboButton;
class SEditableTextBox;
class STextEntryPopup;
class ULiveLinkSourceFactory;
struct FAssetData;

class SLiveLinkClientPanelToolbar : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkClientPanelToolbar)
		: _SourceButtonAlignment(HAlign_Fill)
		, _ShowPresetPicker(true)
		, _ShowSettings(true)
		{}

	/** Horizontal alignment of the add source button. */
	SLATE_ARGUMENT(EHorizontalAlignment, SourceButtonAlignment)
	
	/** Parent window override. */
	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	
	/** (Optional) Custom header displayed on the left side of the toolbar.  */
	SLATE_ARGUMENT(TSharedPtr<SWidget>, CustomHeader)
	
	/** Whether to show the preset picker button. */
	SLATE_ARGUMENT(bool, ShowPresetPicker)
	
	/** Whether to show the settings button. */
	SLATE_ARGUMENT(bool, ShowSettings)
	SLATE_END_ARGS()


	UE_API void Construct(const FArguments& Args, FLiveLinkClient* InClient);

private:

	/** Generates the preset combo button content. */
	TSharedRef<SWidget> OnPresetGeneratePresetsMenu();

	/** Handles saving a preset to disk. */
	void OnSaveAsPreset();

	/** Saves the current preset, or falls back to Save As if none has been loaded. */
	FReply OnSavePreset();

	/** Handles importing a preset from disk. */
	void OnImportPreset(const FAssetData& InPreset);

	/** Handles reverting changes to a preset. */
	FReply OnRevertChanges();

	/** Indicates if a preset has been loaded, used to check if we can revert a preset to its original state. */
	bool HasLoadedLiveLinkPreset() const;

private:

	/** Cached livelink client. */
	FLiveLinkClient* Client;

	/** Parent window override used for creating asset dialogs when running in LiveLinkHub. */
	TSharedPtr<SWindow> ParentWindowOverride;
};

#undef UE_API
