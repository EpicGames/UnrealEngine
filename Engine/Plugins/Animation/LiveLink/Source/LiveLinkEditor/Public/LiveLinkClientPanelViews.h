// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Input/Events.h"
#include "Layout/Geometry.h"
#include "LiveLinkRole.h"
#include "LiveLinkSourceSettings.h"
#include "Misc/Guid.h"
#include "PropertyEditorDelegates.h"
#include "SLiveLinkFilterSearchBox.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"

#define UE_API LIVELINKEDITOR_API

class FLiveLinkClient;
struct FLiveLinkSourceUIEntry;
class FLiveLinkSourcesView;
struct FLiveLinkSubjectUIEntry;
class FLiveLinkSubjectsView;
class FUICommandList;
class IDetailsView;
class ITableRow;
struct FPropertyChangedEvent;
class SLiveLinkSourceListView;
class SLiveLinkDataView;
class STableViewBase;
class SWidget;

template <typename TFilterType>
class SLiveLinkFilterBar;

template <typename TFilterType>
class FFilterBase;

typedef TSharedPtr<FLiveLinkSourceUIEntry> FLiveLinkSourceUIEntryPtr;
typedef TSharedPtr<FLiveLinkSubjectUIEntry> FLiveLinkSubjectUIEntryPtr;

namespace UE::LiveLink
{
	UE_DEPRECATED(5.8, "Use the other CreateSourcesDetailsView method")
	TSharedPtr<IDetailsView> UE_API CreateSourcesDetailsView(const TSharedPtr<FLiveLinkSourcesView>& InSourcesView, const TAttribute<bool>& bInReadOnly);
	TSharedPtr<IDetailsView> UE_API CreateSourcesDetailsView(FOnFinishedChangingProperties InChangedPropertiesDelegate, const TAttribute<bool>& bInReadOnly);
	TSharedPtr<SLiveLinkDataView> UE_API CreateSubjectsDetailsView(FLiveLinkClient* InLiveLinkClient, const TAttribute<bool>& bInReadOnly);
	TSharedPtr<IDetailsView> UE_API CreateDevicesDetailsView(FOnFinishedChangingProperties OnFinishedChangingProperties, const TAttribute<bool>& bInReadOnly);
}

/** Type of the subject entry in the list. */
enum class ELiveLinkEntryType : uint8
{
	Source,
	Subject,
	Device,
	DevicesHeader,
	SourcesHeader
};

// Structure that defines a single entry in the subject UI
struct FLiveLinkSubjectUIEntry
{
	FLiveLinkSubjectUIEntry(const FLiveLinkSubjectKey& InSubjectKey, FLiveLinkClient* InClient, bool bIsSource = false);
	FLiveLinkSubjectUIEntry(FLiveLinkClient* InClient, ELiveLinkEntryType InType);

	// Subject key
	FLiveLinkSubjectKey SubjectKey;
	// LiveLink Client
	FLiveLinkClient* Client;
	// Weak pointer to the device represented by this row.
	TWeakObjectPtr<class ULiveLinkDevice> WeakDevice;
	// What type of item this row represents.
	ELiveLinkEntryType Type = ELiveLinkEntryType::Subject;

	// Children (if this entry represents a source instead of a specific subject
	TArray<TSharedPtr<FLiveLinkSubjectUIEntry>> Children;

	// Whether the item should be hidden since it's been filtered out.
	bool bFilteredOut = false;

	// Whether the subject entry is a subject
	bool IsSubject() const;
	// Whether the subject entry is a source
	bool IsSource() const;
	// Whether this entry is a device.
	bool IsDevice() const;
	// Whether the subject is virtual
	bool IsVirtualSubject() const;
	// Get the subject or source settings
	UObject* GetSettings() const;
	// Whether the subject is enabled
	bool IsSubjectEnabled() const;
	// Whether the subject is valid
	bool IsSubjectValid() const;
	// Whether the entry is a playback source.
	bool IsPlaybackSource() const;
	// Enable or disable a subject
	void SetSubjectEnabled(bool bIsEnabled);
	// Get a textual representation of the ui entry 
	FText GetItemText() const;
	// Get the name of the machine that hosts this source or device.
	FText GetMachineName() const;
	// Get the livelink role of this entry
	TSubclassOf<ULiveLinkRole> GetItemRole() const;
	// Get the translated role for this subject (if translating rebroadcast subjects).
	TSubclassOf<ULiveLinkRole> GetItemTranslatedRole() const;
	// Remove the subject or source from the livelink client
	void RemoveFromClient() const;
	// Pause or unpause a subject if it's already paused
	void PauseSubject();
	// Returns whether a subject is currently paused.
	bool IsPaused() const;
	// Get item string representation for filtering.
	void GetFilterText(TArray<FString>& OutStrings) const;

private:
	// Whether the subject is virtual
	bool bIsVirtualSubject = false;
	// Whether this is a playback source, used to display a different icon when the source is in playback.
	bool bIsPlaybackSource = false;
};

// Structure that defines a single entry in the source UI
struct FLiveLinkSourceUIEntry
{
public:
	FLiveLinkSourceUIEntry(FGuid InEntryGuid, FLiveLinkClient* InClient)
		: EntryGuid(InEntryGuid)
		, Client(InClient)
	{}

	// Get the guid of the source
	FGuid GetGuid() const;
	// Get the type of the source
	FText GetSourceType() const;
	// Get the machine name of the source
	FText GetMachineName() const;
	// Get the source's status
	FText GetStatus() const;
	// Get the source's settings
	ULiveLinkSourceSettings* GetSourceSettings() const;
	// Remove the source from the client
	void RemoveFromClient() const;
	// Get the display name of the source
	FText GetDisplayName() const;
	// Get the tooltip of the source
	FText GetToolTip() const;
	// Get item string representation for filtering.
	void GetFilterText(TArray<FString>& OutStrings) const 
	{ 
		OutStrings.Add(GetDisplayName().ToString());
		OutStrings.Add(GetSourceType().ToString());
	}

	// Whether the item should be hidden since it's been filtered out.
	bool bFilteredOut = false;

private:
	// GUID of the source
	FGuid EntryGuid;
	// Pointer to the livelink client
	FLiveLinkClient* Client = nullptr;

};

namespace UE::LiveLink
{
	/** Base class for live link list/tree views, handles removing list element by pressing delete. */
	template <typename ListType, typename ListElementType>
	class SLiveLinkListView : public ListType
	{
	public:
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
		{
			if (InKeyEvent.GetKey() == EKeys::Delete || InKeyEvent.GetKey() == EKeys::BackSpace)
			{
				if (!bReadOnly.Get())
				{
					TArray<ListElementType> SelectedItem = ListType::GetSelectedItems();
					for (ListElementType Item : SelectedItem)
					{
						Item->RemoveFromClient();
					}
				}

				return FReply::Handled();
			}
			else if (InKeyEvent.GetModifierKeys().IsControlDown() && InKeyEvent.GetKey() == EKeys::A)
			{
				// Use SListView<ListElementType>::GetItems to select all items while avoiding the deprecation warning in STreeView.
				for (const ListElementType& Element: SListView<ListElementType>::GetItems())
				{
					ListType::SetItemSelection(Element, true);
				}

				return FReply::Handled();
			}
			else if (InKeyEvent.GetKey() == EKeys::Escape)
			{
				ListType::ClearSelection();
				return FReply::Handled();
			}

			return FReply::Unhandled();
		}

		// Whether the panel is in read-only mode or not.
		TAttribute<bool> bReadOnly;
	};
}

class SLiveLinkSourceListView : public UE::LiveLink::SLiveLinkListView<SListView<FLiveLinkSourceUIEntryPtr>, FLiveLinkSourceUIEntryPtr>
{
public:
	void Construct(const FArguments& InArgs, TAttribute<bool> bInReadOnly)
	{
		bReadOnly = bInReadOnly;

		UE::LiveLink::SLiveLinkListView<SListView<FLiveLinkSourceUIEntryPtr>, FLiveLinkSourceUIEntryPtr>::Construct(InArgs);
	}
};

class SLiveLinkSubjectsTreeView : public UE::LiveLink::SLiveLinkListView<STreeView<FLiveLinkSubjectUIEntryPtr>, FLiveLinkSubjectUIEntryPtr>
{
public:
	void Construct(const FArguments& InArgs, TAttribute<bool> bInReadOnly)
	{
		bReadOnly = bInReadOnly;
		UE::LiveLink::SLiveLinkListView<STreeView<FLiveLinkSubjectUIEntryPtr>, FLiveLinkSubjectUIEntryPtr>::Construct(InArgs);
	}
};

class FLiveLinkSubjectsView : public TSharedFromThis<FLiveLinkSubjectsView>
{
public:
	DECLARE_DELEGATE_TwoParams(FOnSubjectSelectionChanged, FLiveLinkSubjectUIEntryPtr, ESelectInfo::Type);

	using FFilterType = FLiveLinkSubjectUIEntryPtr;
	using FMediaTreeFilter = TTextFilter<FFilterType>;

	struct FSubjectsViewArgs
	{
		TAttribute<bool> bInReadOnly;
		bool bInShowDevices = false;
	};

	UE_API FLiveLinkSubjectsView(FOnSubjectSelectionChanged InOnSubjectSelectionChanged, const TSharedPtr<FUICommandList>& InCommandList, TAttribute<bool> bInReadOnly);

	UE_API FLiveLinkSubjectsView(FOnSubjectSelectionChanged InOnSubjectSelectionChanged, const TSharedPtr<FUICommandList>& InCommandList, const FSubjectsViewArgs& InArgs);

	// Helper functions for building the subject tree UI
	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(FLiveLinkSubjectUIEntryPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	// Get a subjectsview's children
	UE_API void GetChildrenForInfo(FLiveLinkSubjectUIEntryPtr InInfo, TArray< FLiveLinkSubjectUIEntryPtr >& OutChildren);
	// Handle subject selection changed
	UE_API void OnSubjectSelectionChanged(FLiveLinkSubjectUIEntryPtr SubjectEntry, ESelectInfo::Type SelectInfo);
	// Handler for the subject tree context menu opening
	UE_API TSharedPtr<SWidget> OnOpenVirtualSubjectContextMenu(TSharedPtr<FUICommandList> InCommandList);
	// Return whether a subject can be removed
	UE_API bool CanRemoveSubject() const;
	// Refresh the list of subjects using the livelink client.
	UE_API void RefreshSubjects();
	// Return whether a subject can be paused
	UE_API bool CanPauseSubject() const;
	// Pauses/Unpauses a subject
	UE_API void HandlePauseSubject();
	// Get the subjects list widget
	UE_API TSharedRef<SWidget> GetWidget();

private:
	// Captures the identity of a selected item so it can be matched after the list is rebuilt.
	struct FSavedSelection
	{
		// Full subject key (used for subject and source matching).
		FLiveLinkSubjectKey SubjectKey;
		// Type of the entry.
		ELiveLinkEntryType Type;
		// Weak reference to the device, used to match device entries.
		TWeakObjectPtr<class ULiveLinkDevice> WeakDevice;
	};

	// Saves the current tree view selection into OutSavedSelection.
	void SaveSelection(TArray<FSavedSelection>& OutSavedSelection) const;
	// Restores a previously saved selection by matching entries in the rebuilt tree.
	void RestoreSelection(const TArray<FSavedSelection>& InSavedSelection);
	// Selects Item if it matches any entry in InSavedSelection.
	void RestoreSelectionForItem(const FLiveLinkSubjectUIEntryPtr& Item, const TArray<FSavedSelection>& InSavedSelection);

	void CreateCombinedTreeView(const TSharedPtr<FUICommandList>& InCommandList);
	// Get the label for the pause subject context menu entry.
	FText GetPauseSubjectLabel() const;
	// Get the tooltip for the pause subject context menu entry.
	FText GetPauseSubjectToolTip() const;
	// Returns whether the selected subject is paused.
	bool IsSelectedSubjectPaused() const;
	// Returns whether a device can be removed.
	bool CanRemoveDevice() const;
	// Returns whether a source can be removed.
	bool CanRemoveSource() const;
	// Returns whether there are any sources in the list.
	bool HasAnySources() const;
	// Opens a context menu with options relevant to the selected items.
	TSharedPtr<SWidget> OnCombinedListOpenContextMenu(TSharedPtr<FUICommandList> InCommandList);
	// Populates entries for the Add combo button.
	TSharedRef<SWidget> OnGenerateAddMenuEntries();
	// Initialize default filters for the list view.
	void InitializeFilters();
	// Triggers a refresh of the filtered elements when the filter changes.
	void OnFilterChanged();
	// Sources, Subjects and Devices filters.
	TArray<TSharedRef<FFilterBase<FFilterType>>> CustomFilters;
	// Filter bar widget. Necessary to create the Add filter button, but it's not actually displayed in the LiveLink UI. 
	TSharedPtr<SLiveLinkFilterBar<FFilterType>> FilterBar;

public:
	// Subject tree widget
	TSharedPtr<class SLiveLinkSubjectsTreeView> SubjectsTreeView;
	// Subject tree items
	TArray<FLiveLinkSubjectUIEntryPtr> SubjectData;
	// Filtered subject tree items
	TArray<FLiveLinkSubjectUIEntryPtr> FilteredList;
	// Subject Selection Changed delegate
	FOnSubjectSelectionChanged SubjectSelectionChangedDelegate;
	// Returns whether the panel is in read-only mode.
	TAttribute<bool> bReadOnly;
	// Filter search box.
	TSharedPtr<SLiveLinkFilterSearchBox<FLiveLinkSubjectUIEntryPtr>> FilterSearchBox;

private:
	// Whether to show devices in the menu. (Disabled by default in editor)
	bool bShowDevices = false;
};

class FLiveLinkSourcesView : public TSharedFromThis<FLiveLinkSourcesView>, public FGCObject
{
public:
	DECLARE_DELEGATE_TwoParams(FOnSourceSelectionChanged, FLiveLinkSourceUIEntryPtr, ESelectInfo::Type);

	FLiveLinkSourcesView(FLiveLinkClient* InLiveLinkClient, TSharedPtr<FUICommandList> InCommandList, TAttribute<bool> bInReadOnly, FOnSourceSelectionChanged InOnSourceSelectionChanged);
	~FLiveLinkSourcesView();

	//~ Begin FGCObject interface
	virtual FString GetReferencerName() const override
	{
		return TEXT("SLiveLinkClientPanelToolbar");
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObjects(Factories);
	}
	//~ End FGCObject interface

	// Get list view widget
	UE_API TSharedRef<SWidget> GetWidget();
	// Gather information about all sources and update the list view 
	void RefreshSourceData(bool bRefreshUI);
	// Handler that creates a widget row for a given ui entry
	TSharedRef<ITableRow> MakeSourceListViewWidget(FLiveLinkSourceUIEntryPtr Entry, const TSharedRef<STableViewBase>& OwnerTable) const;
	// Handles constructing a context menu for the sources
	TSharedPtr<SWidget> OnSourceConstructContextMenu(TSharedPtr<FUICommandList> InCommandList);
	// Return whether a source can be removed
	bool CanRemoveSource();
	// Removes a livelink source from the livelink client
	void HandleRemoveSource();
	// Callback when property changes on source settings
	void OnPropertyChanged(const FPropertyChangedEvent& InEvent);
	// Handle selection change, triggering the OnSourceSelectionChangedDelegate delegate.
	void OnSourceListSelectionChanged(FLiveLinkSourceUIEntryPtr Entry, ESelectInfo::Type SelectionType) const;
private:
	// Create the sources list view
	void CreateSourcesListView(const TSharedPtr<FUICommandList>& InCommandList);
	// Populates the source combo menu widget content.
	TSharedRef<SWidget> OnGenerateSourceMenu();

public:
	// Holds the data that will be displayed in the list view
	TArray<FLiveLinkSourceUIEntryPtr> SourceData;
	// Holds the sources list view widget
	TSharedPtr<SLiveLinkSourceListView> SourcesListView;
	// LiveLink Client
	FLiveLinkClient* Client;
	// Source selection changed delegate
	FOnSourceSelectionChanged OnSourceSelectionChangedDelegate;
	// Returns whether the panel is in read-only mode.
	TAttribute<bool> bReadOnly;

private:
	// Filtered list resulting from applying the text filter.
	TArray<FLiveLinkSourceUIEntryPtr> FilteredList;
	// Holds the searchbox widget and the source list.
	TSharedPtr<SWidget> HostWidget;
	// Handle to the tick delegate.
	FTSTicker::FDelegateHandle TickHandle;
	// Holds all the LiveLink souce factories.
	TArray<TObjectPtr<ULiveLinkSourceFactory>> Factories;
	// Filter search box.
	TSharedPtr<SLiveLinkFilterSearchBox<FLiveLinkSourceUIEntryPtr>> FilterSearchBox;
};

#undef UE_API
