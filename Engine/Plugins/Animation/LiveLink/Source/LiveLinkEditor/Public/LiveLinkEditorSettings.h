// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "LiveLinkEditorSettings.generated.h"

/**
 * Editor preferences settings for LiveLink.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class ULiveLinkEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Whether the livelink panel should be in read-only mode. */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink")
	bool bReadOnly = false;
};
