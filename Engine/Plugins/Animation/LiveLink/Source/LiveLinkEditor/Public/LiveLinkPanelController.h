// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/IDelegateInstance.h"
#include "Delegates/DelegateCombinations.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealType.h"

#define UE_API LIVELINKEDITOR_API

class IDetailsView;
class FLiveLinkClient;
struct FLiveLinkSourceUIEntry;
struct FLiveLinkSubjectKey;
struct FLiveLinkSubjectUIEntry;
class SLiveLinkSourceListView;
class FLiveLinkSubjectsView;
class FUICommandList;
class SLiveLinkDataView;
class SWidget;

namespace ESelectInfo { enum Type : int; }
typedef TSharedPtr<FLiveLinkSourceUIEntry> FLiveLinkSourceUIEntryPtr;
typedef TSharedPtr<FLiveLinkSubjectUIEntry> FLiveLinkSubjectUIEntryPtr;


/** Handles callback connections between the sources, subjects and details views. */
class FLiveLinkPanelController : public TSharedFromThis<FLiveLinkPanelController>
{
public:
	UE_API FLiveLinkPanelController(TAttribute<bool> bInReadOnly = false);
	UE_API ~FLiveLinkPanelController();

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSubjectSelectionChanged, const FLiveLinkSubjectKey&);
	/** Subject Selection changed callback. */
	FOnSubjectSelectionChanged& OnSubjectSelectionChanged()
	{
		return SubjectSelectionChangedDelegate;
	}

	/** Get the widget displaying details for the selected item, whether it's a Source, Subject or Device. */
	UE_API TSharedRef<SWidget> GetCombinedDetailsWidget();

private:
	// Bind live link commands 
	void BindCommands();
	// Handles the source collection changing.
	void OnSourcesChangedHandler();
	// Handles the subject collection changing.
	void OnSubjectsChangedHandler();
	// Returns whether there is a source in the list.
	bool HasSource() const;
	// Handles the remove source command.
	void HandleRemoveSource();
	// Handles the remove all sources command.
	void HandleRemoveAllSources();
	// Returns whether a source could be removed.
	bool CanRemoveSource() const;
	// Returns whether a subject can be removed.
	bool CanRemoveSubject() const;
	// Returns whether a device can be removed.
	bool CanRemoveDevice() const;
	// Returns whethera subject can be paused
	bool CanPauseSubject() const;
	// Pauses/Unpauses a subject.
	void HandlePauseSubject();
	// Handles the remove subject command.
	void HandleRemoveSubject();
	// Handles the remove device command.
	void HandleRemoveDevice();
	// Recreates the subject list data behind the tree view.
	void RebuildSubjectList();
	// Hadnles subject selection changing.
	void OnSubjectSelectionChangedHandler(FLiveLinkSubjectUIEntryPtr SubjectEntry, ESelectInfo::Type SelectInfo);
	// Handles triggering the modified device's OnSettingChanged method.
	void OnFinishedChangingDeviceProperties(const FPropertyChangedEvent& InPropertyChangedEvent);
	// Handles triggering LiveLinkClient's OnPropertyChanged method for the modified source.
	void OnFinishedChangingSourceProperties(const FPropertyChangedEvent& InPropertyChangedEvent);
	// Returns the index of the details widget that should be shown, according to the current selection.
	int32 GetCombinedDetailWidgetIndex() const;

public:
	// Sources view
	UE_DEPRECATED(5.8, "Use the subjects view instead.")
	TSharedPtr<class FLiveLinkSourcesView> SourcesView;
	// Subjects view
	TSharedPtr<FLiveLinkSubjectsView> SubjectsView;
	// Reference to connection settings struct details panel
	TSharedPtr<IDetailsView> SourcesDetailsView;
	// Reference to the data value struct details panel
	TSharedPtr<SLiveLinkDataView> SubjectsDetailsView;
	// Reference to device settings details panel
	TSharedPtr<IDetailsView> DevicesDetailsView;
	// LiveLink Client
	FLiveLinkClient* Client;
	// Handle to delegate when client sources list has changed
	FDelegateHandle OnSourcesChangedHandle;
	// Handle to delegate when a client subjects list has changed
	FDelegateHandle OnSubjectsChangedHandle;
	// Guard from reentrant selection
	mutable bool bSelectionChangedGuard = false;
	// Command list
	TSharedPtr<FUICommandList> CommandList;
	// Delegate called when the subject selection changes
	FOnSubjectSelectionChanged SubjectSelectionChangedDelegate;

private:
	// Delegate called when a device is added.
	FDelegateHandle DeviceAddedHandle;
	// Delegate called when a device is removed.
	FDelegateHandle DeviceRemovedHandle;
};

#undef UE_API
