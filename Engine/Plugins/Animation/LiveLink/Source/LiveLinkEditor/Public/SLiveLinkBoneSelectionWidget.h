// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define UE_API LIVELINKEDITOR_API


class IEditableSkeleton;
class SComboButton;

DECLARE_DELEGATE_OneParam(FOnBoneSelectionChanged, FName);
DECLARE_DELEGATE_RetVal(FName, FGetSelectedBone);

class SLiveLinkBoneTreeMenu : public SCompoundWidget
{
public:
	/** Storage object for bone hierarchy */
	struct FBoneNameInfo
	{
		FBoneNameInfo(FName Name) : BoneName(Name) {}

		FName BoneName;
		TArray<TSharedPtr<FBoneNameInfo>> Children;
	};

	SLATE_BEGIN_ARGS(SLiveLinkBoneTreeMenu)
		: _OnBoneSelectionChanged()
		{}

		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(FName, SelectedBone)
		SLATE_EVENT(FOnBoneSelectionChanged, OnBoneSelectionChanged)
	SLATE_END_ARGS();

	/**
	* Construct this widget
	*
	* @param	InArgs	The declaration data for this widget
	*/
	UE_API void Construct(const FArguments& InArgs, TOptional<FLiveLinkSkeletonStaticData> SkeletonStaticData);

	/** Get the filter text widget, e.g. for focus */
	UE_API TSharedPtr<SWidget> GetFilterTextWidget();

private:
	/** SWidget interface */
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Using the current filter, repopulate the tree view */
	UE_API void RebuildBoneList(const FName& SelectedBone);

	/** Make a single tree row widget */
	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(TSharedPtr<FBoneNameInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get the children for the provided bone info */
	UE_API void GetChildrenForInfo(TSharedPtr<FBoneNameInfo> InInfo, TArray< TSharedPtr<FBoneNameInfo> >& OutChildren);

	/** Called when the user changes the search filter */
	UE_API void OnFilterTextChanged(const FText& InFilterText);

	/** Handle the tree view selection changing */
	UE_API void OnSelectionChanged(TSharedPtr<SLiveLinkBoneTreeMenu::FBoneNameInfo> BoneInfo, ESelectInfo::Type SelectInfo);

	/** Select a specific bone, helper for UI handler functions */
	UE_API void SelectBone(TSharedPtr<SLiveLinkBoneTreeMenu::FBoneNameInfo> BoneInfo);
	
	/** Tree info entries for bone picker */
	TArray<TSharedPtr<FBoneNameInfo>> SkeletonTreeInfo;
	/** Mirror of SkeletonTreeInfo but flattened for searching */
	TArray<TSharedPtr<FBoneNameInfo>> SkeletonTreeInfoFlat;

	/** Text to filter bone tree with */
	FText FilterText;

	/** Tree view used in the button menu */
	TSharedPtr<STreeView<TSharedPtr<FBoneNameInfo>>> TreeView;

	/** Filter text widget */
	TSharedPtr<SSearchBox> FilterTextWidget;

	/** Delegate called when a bone is selected. */
	FOnBoneSelectionChanged OnSelectionChangedDelegate;

	/** Static data used to populate the bone list. */
	FLiveLinkSkeletonStaticData SkeletonStaticData;
};

class SLiveLinkBoneSelectionWidget : public SCompoundWidget
{
public: 

	SLATE_BEGIN_ARGS(SLiveLinkBoneSelectionWidget)
		: _OnBoneSelectionChanged()
		, _OnGetSelectedBone()
	{}

		/** set selected bone name */
		SLATE_EVENT(FOnBoneSelectionChanged, OnBoneSelectionChanged);

		/** get selected bone name **/
		SLATE_EVENT(FGetSelectedBone, OnGetSelectedBone);
	SLATE_END_ARGS();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UE_API void Construct(const FArguments& InArgs, const FLiveLinkSubjectKey& InSubjectKey);

	/** Set the subject for which we will list the bones. */
	UE_API void SetSubject(const FLiveLinkSubjectKey& InSubjectKey);

	/** Utility method to create fake static data from a frame translator. */
	UE_API FLiveLinkSkeletonStaticData MakeStaticDataFromTranslator(const FLiveLinkSubjectKey& SubjectKey) const;

private: 

	/** Creates the combo button menu when clicked */
	UE_API TSharedRef<SWidget> CreateSkeletonWidgetMenu();
	/** Called when the user selects a bone name */
	UE_API void OnSelectionChanged(FName BoneName);

	/** Gets the current bone name, used to get the right name for the combo button */
	UE_API FText GetCurrentBoneName() const;

	/** Base combo button */
	TSharedPtr<SComboButton> BonePickerButton;

	/** Subject that for which we are selecting a bone. */
	FLiveLinkSubjectKey SubjectKey;

	// Delegates
	FOnBoneSelectionChanged OnBoneSelectionChanged;
	FGetSelectedBone OnGetSelectedBone;
};

#undef UE_API
