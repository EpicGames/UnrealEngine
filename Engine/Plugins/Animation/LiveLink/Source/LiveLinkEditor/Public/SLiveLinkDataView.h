// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "LiveLinkTypes.h"

#define UE_API LIVELINKEDITOR_API

namespace ETextCommit { enum Type : int; }
struct FPropertyChangedEvent;

class IDetailsView;
class FLiveLinkClient;
class IStructureDetailsView;

class SLiveLinkDataView : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

public:
	SLATE_BEGIN_ARGS(SLiveLinkDataView)
		: _ReadOnly(false)
		{}
	SLATE_ATTRIBUTE(bool, ReadOnly)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args, FLiveLinkClient* InClient);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	UE_API void SetSubjectKey(FLiveLinkSubjectKey InSubjectKey);
	FLiveLinkSubjectKey GetSubjectKey() const { return SubjectKey; }

	void SetRefreshDelay(double DelaySeconds) { UpdateDelay = DelaySeconds; }
	double GetRefreshDelay() const { return UpdateDelay; }

private:
	enum class EDetailType : uint32
	{
		Property,
		StaticData,
		FrameData,
	};

	UE_API void OnPropertyChanged(const FPropertyChangedEvent& InEvent);
	UE_API int32 GetDetailWidgetIndex() const;
	UE_API void OnSelectDetailWidget(EDetailType InDetailType);
	bool IsSelectedDetailWidget(EDetailType InDetailType) const { return InDetailType == DetailType; }
	bool CanEditRefreshDelay() const { return DetailType != EDetailType::Property; }
	void SetRefreshDelayInternal(double InDelaySeconds, ETextCommit::Type) { SetRefreshDelay(InDelaySeconds); }

private:
	FLiveLinkClient* Client;
	FLiveLinkSubjectKey SubjectKey;
	double LastUpdateSeconds;
	double UpdateDelay;
	EDetailType DetailType;

	TSharedPtr<IStructureDetailsView> StructureDetailsView;
	TSharedPtr<IDetailsView> SettingsDetailsView;
};

#undef UE_API
