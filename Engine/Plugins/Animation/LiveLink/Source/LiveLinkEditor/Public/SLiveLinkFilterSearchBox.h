// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Filters/SBasicFilterBar.h"
#include "Misc/TextFilter.h"
#include "Widgets/Input/SSearchBox.h"

/** Concept that ensures a type has a field bFilteredOut. */
template <class Type>
concept CHasFilteredOutField = requires(Type T)
{
	T->bFilteredOut;
};

/** Concept that ensures a type has a Children field. */
template <class Type>
concept CHasChildren = requires(Type T)
{
	T->Children;
};

/**
 * Wrapper around a SSearchBox that provides utility to filter items in a list or tree view.
 */
template <typename ItemType>
class SLiveLinkFilterSearchBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnGatherItems, TArray<ItemType>& /*OutItems*/);
	DECLARE_DELEGATE_OneParam(FOnUpdateFilteredList, const TArray<ItemType>& /*FilteredItems*/);

	SLATE_BEGIN_ARGS(SLiveLinkFilterSearchBox) {}
		/** The list item for this row */
		SLATE_ARGUMENT(TArray<ItemType>*, ItemSource);
		SLATE_EVENT(FOnGatherItems, OnGatherItems);
		SLATE_EVENT(FOnUpdateFilteredList, OnUpdateFilteredList);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SBasicFilterBar<ItemType>> InFilterBar = nullptr)
	{
		static_assert(CHasFilteredOutField<ItemType>);

		SearchTextFilter = MakeShared<FEntryTextFilter>(
			FEntryTextFilter::FItemToStringArray::CreateSP(this, &SLiveLinkFilterSearchBox::GetItemString));

		ItemSource = InArgs._ItemSource;
		OnUpdateFilteredList = InArgs._OnUpdateFilteredList;
		FilterBar = InFilterBar;

		check(ItemSource);

		ChildSlot
		[
			SAssignNew(SearchBox, SSearchBox)
				.HintText(NSLOCTEXT("FilterSearchBox", "SearchHint", "Search"))
				.OnTextChanged(this, &SLiveLinkFilterSearchBox::OnSearchTextChanged)
		];
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		if (bUpdateFilteredItems)
		{
			UpdateFilteredItemSource();
			bUpdateFilteredItems = false;
		}
	}

	/** Trigger a deferred update of the filtered items. */
	void Update()
	{
		bUpdateFilteredItems = true;
	}

private:
	/** Handle search text being changed by the user, triggers a deferred update. */
	void OnSearchTextChanged(const FText& InFilterText)
	{
		SearchTextFilter->SetRawFilterText(InFilterText);
		SearchBox->SetError(SearchTextFilter->GetFilterErrorText());

		bUpdateFilteredItems = true;
	}

	/** Returns whether an item passes any filter bar filter. */
	bool PassesAnyFilterBarFilter(const ItemType& InItem)
	{
		TSharedPtr<TFilterCollection<ItemType>> FilterCollection = FilterBar->GetAllActiveFilters();
		const int32 NumFilters = FilterCollection->Num();
		if (NumFilters == 0)
		{
			return true;
		}

		for (int32 Index = 0; Index < NumFilters; ++Index)
		{
			if (FilterCollection->GetFilterAtIndex(Index)->PassesFilter(InItem))
			{
				return true;
			}
		}

		return false;
	};
	
	/** Updates the bFilteredOut field on the item and its children, then returns if the item or any of its child is visible. */
	bool UpdateVisibleChildren(ItemType& Item)
	{
		const bool bPassesTextSearchbar = SearchTextFilter->GetRawFilterText().IsEmpty() || SearchTextFilter->PassesFilter(Item);
		bool bFilteredOut = !PassesAnyFilterBarFilter(Item) || !bPassesTextSearchbar;

		for (ItemType& Child : Item->Children)
		{
			if (UpdateVisibleChildren(Child))
			{
				bFilteredOut = false;
			}
		}

		Item->bFilteredOut = bFilteredOut;
		return !bFilteredOut;
	}

	/** Updates the filtered item source to match the SearchBox filter. */
	void UpdateFilteredItemSource()
	{
		FilteredItemSource.Reset();

		if (FilterBar)
		{
			if constexpr (CHasChildren<ItemType>)
			{
				for (ItemType& Item : *ItemSource)
				{
					const bool bAnyVisibleChildren = UpdateVisibleChildren(Item);
					if (bAnyVisibleChildren)
					{
						FilteredItemSource.Add(Item);
					}
				}
			}
			else
			{
				Algo::TransformIf(*ItemSource, FilteredItemSource,
					[this](const ItemType& Item)
					{
						return PassesAnyFilterBarFilter(Item) && SearchTextFilter->PassesFilter(Item);
					},
					[](const ItemType& Item) { return Item; });
			}
		}
		else
		{
			FilteredItemSource = *ItemSource;
		}

		OnUpdateFilteredList.ExecuteIfBound(FilteredItemSource);
	}

	/** Get string representation from a subject UI entry. */
	void GetItemString(const ItemType InItem, TArray<FString>& OutStrings)
	{
		InItem->GetFilterText(OutStrings);
	}

private:
	/** Ptr to the unfiltered item source. Used to update the filtered item list when no filtering is applied. */
	TArray<ItemType>* ItemSource = nullptr;
	/** Internal filtered item source.  */
	TArray<ItemType> FilteredItemSource;
	/** Handles deferred updates to the list filter. */
	bool bUpdateFilteredItems = true;
	/** Holds the search box widget. */
	TSharedPtr<SSearchBox> SearchBox;

	using FEntryTextFilter = TTextFilter<ItemType>;
	/** Text filter used for the subject list. */
	TSharedPtr<FEntryTextFilter> SearchTextFilter;

	/** Optional filter bar used to filter items according to a predefined list of filters. */
	TSharedPtr<SBasicFilterBar<ItemType>> FilterBar;
	
	/** Delegate called to the owner of this widget to inform them that the filtered list has been updated. */
	FOnUpdateFilteredList OnUpdateFilteredList;
};

