// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "LiveLinkRole.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"

#define UE_API LIVELINKEDITOR_API

class ULiveLinkPreset;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
template <typename ItemType> class SListView;

struct FAssetData;
struct FLiveLinkSubjectRepresentationPickerEntry;
struct FSlateBrush;

class FMenuBuilder;
class ITableRow;
class SComboButton;
class STableViewBase;

typedef TSharedPtr<FLiveLinkSubjectRepresentationPickerEntry> FLiveLinkSubjectRepresentationPickerEntryPtr;

DECLARE_DELEGATE_OneParam(FOnGetSubjects, TArray<FLiveLinkSubjectKey>& /**OutSubjectsList*/);


/**
 * A widget which allows the user to enter a subject name or discover it from a drop menu.
 */
class SLiveLinkSubjectRepresentationPicker : public SCompoundWidget
{
public:
	struct FLiveLinkSourceSubjectRole
	{
		FLiveLinkSourceSubjectRole() = default;
		explicit FLiveLinkSourceSubjectRole(FLiveLinkSubjectRepresentation InSubjectRepresentation)
			: Subject(InSubjectRepresentation.Subject), Role(InSubjectRepresentation.Role)
		{}
		explicit FLiveLinkSourceSubjectRole(FLiveLinkSubjectKey InSubjectKey)
			: Source(InSubjectKey.Source), Subject(InSubjectKey.SubjectName)
		{}
		FLiveLinkSourceSubjectRole(FGuid InSource, FLiveLinkSubjectName InSubject, TSubclassOf<ULiveLinkRole> InRole)
			: Source(InSource), Subject(InSubject), Role(InRole)
		{}

		FLiveLinkSubjectRepresentation ToSubjectRepresentation() const { return FLiveLinkSubjectRepresentation(Subject, Role); }
		FLiveLinkSubjectKey ToSubjectKey() const { return FLiveLinkSubjectKey(Source, Subject); }

		FGuid Source;
		FLiveLinkSubjectName Subject;
		TSubclassOf<ULiveLinkRole> Role;
	};

	DECLARE_DELEGATE_OneParam(FOnValueChanged, FLiveLinkSourceSubjectRole);

	SLATE_BEGIN_ARGS(SLiveLinkSubjectRepresentationPicker)
		: _ComboButtonStyle(&FCoreStyle::Get().GetWidgetStyle< FComboButtonStyle >("ComboButton"))
		, _ButtonStyle(nullptr)
		, _ForegroundColor(FCoreStyle::Get().GetSlateColor("InvertedForeground"))
		, _ContentPadding(FMargin(2.f, 0.f))
		, _HasMultipleValues(false)
		, _ShowSource(false)
		, _ShowRole(false)
		, _Font()
	{}

		/** The visual style of the combo button */
		SLATE_STYLE_ARGUMENT(FComboButtonStyle, ComboButtonStyle)

		/** The visual style of the button (overrides ComboButtonStyle) */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
	
		/** Foreground color for the picker */
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
	
		/** Content padding for the picker */
		SLATE_ATTRIBUTE(FMargin, ContentPadding)
	
		/** Attribute used to retrieve the current value. */
		SLATE_ATTRIBUTE(FLiveLinkSourceSubjectRole, Value)
	
		/** Delegate for handling when for when the current value changes. */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)
	
		/** Attribute used to retrieve whether the picker has multiple values. */
		SLATE_ATTRIBUTE(bool, HasMultipleValues)

		/** Attribute used to retrieve whether the picker should show the source name. */
		SLATE_ARGUMENT(bool, ShowSource)

		/** Attribute used to retrieve whether the picker should show roles. */
		SLATE_ARGUMENT(bool, ShowRole)

		/** Sets the font used to draw the text on the button */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** (Optional) Delegate that can be specified in order to provide a custom list of subjects. */
		SLATE_EVENT(FOnGetSubjects, OnGetSubjects);

	SLATE_END_ARGS()

	/**
	 * Slate widget construction method
	 */
	UE_API void Construct(const FArguments& InArgs);

	/**
	 * Access the current value of this picker
	 */
	UE_API FLiveLinkSourceSubjectRole GetCurrentValue() const;

private:
	UE_API FText GetSourceNameValueText() const;
	UE_API FText GetSubjectNameValueText() const;
	UE_API const FSlateBrush* GetRoleIcon() const;
	UE_API FText GetRoleText() const;

	UE_API TSharedRef<SWidget> BuildMenu();
	UE_API FText GetPresetSelectedText() const;
	UE_API FSlateColor GetSelectPresetForegroundColor() const;
	UE_API FReply ClearCurrentPreset();
	UE_API bool HasCurrentPreset() const;
	
	UE_API TSharedRef<ITableRow> MakeSubjectRepListViewWidget(FLiveLinkSubjectRepresentationPickerEntryPtr Entry, const TSharedRef<STableViewBase>& OwnerTable) const;
	UE_API void OnSubjectRepListSelectionChanged(FLiveLinkSubjectRepresentationPickerEntryPtr Entry, ESelectInfo::Type SelectionType);

	UE_API TSharedRef<SWidget> BuildPresetSubMenu();
	UE_API void NewPresetSelected(const FAssetData& AssetData);
	UE_API void OnComboTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	UE_API void SetValue(const FLiveLinkSourceSubjectRole& InValue);

	UE_API void BuildSubjectRepDataList();

private:
	TWeakObjectPtr<ULiveLinkPreset> SelectedLiveLinkPreset;
	TWeakPtr<SComboButton> PickerComboButton;
	TWeakPtr<SComboButton> SelectPresetComboButton;
	TWeakPtr<SListView<FLiveLinkSubjectRepresentationPickerEntryPtr>> SubjectListView;
	TArray<FLiveLinkSubjectRepresentationPickerEntryPtr> SubjectRepData;

	FText CachedSourceType;
	TAttribute<FLiveLinkSourceSubjectRole> ValueAttribute;
	FOnValueChanged OnValueChangedDelegate;
	TAttribute<bool> HasMultipleValuesAttribute;
	bool bShowSource;
	bool bShowRole;

	/** Delegate used to retrieve a custom list of subjects. */
	FOnGetSubjects GetSubjectsDelegate;
};

#undef UE_API
