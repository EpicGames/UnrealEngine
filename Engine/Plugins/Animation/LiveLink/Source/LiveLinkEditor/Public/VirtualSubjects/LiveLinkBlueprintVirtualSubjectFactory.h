// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/BlueprintFactory.h"

#include "LiveLinkBlueprintVirtualSubjectFactory.generated.h"

#define UE_API LIVELINKEDITOR_API

class ULiveLinkRole;

UCLASS(MinimalAPI, hidecategories = Object)
class ULiveLinkBlueprintVirtualSubjectFactory : public UBlueprintFactory
{
	GENERATED_BODY()
	
public:
	UE_API ULiveLinkBlueprintVirtualSubjectFactory();

	UPROPERTY(BlueprintReadWrite, Category = "Live Link Blueprint Virtual Subject Factory")
	TSubclassOf<ULiveLinkRole> Role;

	//~ Begin UFactory Interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	UE_API virtual bool ShouldShowInNewMenu() const override;
	UE_API virtual uint32 GetMenuCategories() const override;
	UE_API virtual TArray<FAssetCategoryPath> GetAssetMenuPathsForCategory(FName InCategory) const override;
	//~ Begin UFactory Interface
};

#undef UE_API
