// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkGraphNode : ModuleRules
	{
		public LiveLinkGraphNode(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{

				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"KismetCompiler",
					"LiveLink",
					"LiveLinkAnimationCore",
					"LiveLinkInterface",
					"SlateCore",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"AnimGraph",
					}
				);

				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"BlueprintGraph",
					}
				);
			}
		}
	}
}
