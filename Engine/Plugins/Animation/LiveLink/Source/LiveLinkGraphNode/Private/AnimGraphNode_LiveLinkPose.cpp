// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LiveLinkPose.h"
#include "EdGraph/EdGraphSchema.h"
#include "Animation/AnimAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LiveLinkPose)

#define LOCTEXT_NAMESPACE "LiveLinkAnimNode"

FText UAnimGraphNode_LiveLinkPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Live Link Pose");
}

FText UAnimGraphNode_LiveLinkPose::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Retrieves the current pose associated with the supplied subject");
}

FText UAnimGraphNode_LiveLinkPose::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Live Link");
}

void UAnimGraphNode_LiveLinkPose::ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges)
{
	//Find deprecated SubjectName pin and set new pin with its value
	const FName OldPinName = TEXT("SubjectName"); //Variable now has _DEPRECATED appended so can't use GET_MEMBER_NAME_CHECKED
	const FName NewPinName = GET_MEMBER_NAME_CHECKED(FAnimNode_LiveLinkPose, LiveLinkSubjectName);
	UEdGraphPin** FoundPinPtr = Pins.FindByPredicate([OldPinName](const UEdGraphPin* Other) { return Other->PinName == OldPinName; });
	if (FoundPinPtr != nullptr)
	{
		UEdGraphPin** FoundNewPinPtr = Pins.FindByPredicate([NewPinName](const UEdGraphPin* Other) { return Other->PinName == NewPinName; });
		if (FoundNewPinPtr != nullptr)
		{
			UScriptStruct* StructType = FLiveLinkSubjectName::StaticStruct();
			UEdGraphPin* OldPin = *FoundPinPtr;
			UEdGraphPin* NewPin = *FoundNewPinPtr;

			//Create new structure from old data
			FLiveLinkSubjectName NewName;
			NewName.Name = *OldPin->DefaultValue;

			//Apply new name structure 
			FString StringValue;
			StructType->ExportText(StringValue, &NewName, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
			NewPin->GetSchema()->TrySetDefaultValue(*NewPin, StringValue);

			//Update node data with graph data
			Node.LiveLinkSubjectName = NewName;
		}
	}
}

void UAnimGraphNode_LiveLinkPose::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
	OutAttributes.Add(UE::Anim::FAttributes::Attributes);
}

#undef LOCTEXT_NAMESPACE
