// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EvaluateLiveLinkCustom.h"

#include "K2Node_CallFunction.h"
#include "KismetCompiler.h"
#include "LiveLinkBlueprintLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EvaluateLiveLinkCustom)


#define LOCTEXT_NAMESPACE "K2Node_EvaluateLiveLinkCustom"


namespace UK2Node_EvaluateLiveLinkFrameHelper
{
	static FName LiveLinkWorldTimePinName = "WorldTime";
	static FName LiveLinkSceneTimePinName = "SceneTime";
};

void UK2Node_EvaluateLiveLinkFrameWithSpecificRole::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
}

void UK2Node_EvaluateLiveLinkFrameAtWorldTime::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* LiveLinkWorldTimePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkWorldTimePinName);
	LiveLinkWorldTimePin->PinFriendlyName = LOCTEXT("LiveLinkWorldTimePin", "World Time");
	SetPinToolTip(*LiveLinkWorldTimePin, LOCTEXT("LiveLinkWorldTimePinDescription", "The World Time the subject will be evaluated to"));
}

void UK2Node_EvaluateLiveLinkFrameAtSceneTime::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Timecode pin
	UScriptStruct* TimecodeScriptStruct = FindObject<UScriptStruct>(nullptr, TEXT("/Script/CoreUObject.Timecode"), EFindObjectFlags::ExactClass);
	check(TimecodeScriptStruct);
	UEdGraphPin* LiveLinkSceneTimePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, TimecodeScriptStruct, UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkSceneTimePinName);
	LiveLinkSceneTimePin->PinFriendlyName = LOCTEXT("LiveLinkSceneTimePin", "Scene Time");
	SetPinToolTip(*LiveLinkSceneTimePin, LOCTEXT("LiveLinkSceneTimePinDescription", "The Scene Time the subject will be evaluated to"));
}

FText UK2Node_EvaluateLiveLinkFrameWithSpecificRole::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("EvaluateLiveLinkFrameTitle", "Evaluate Live Link Frame");
}

FText UK2Node_EvaluateLiveLinkFrameAtWorldTime::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("EvaluateLiveLinkFrameAtWorldTimeTitle", "Evaluate Live Link Frame at World Time");
}

FText UK2Node_EvaluateLiveLinkFrameAtSceneTime::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("EvaluateLiveLinkFrameAtSceneTimeTitle", "Evaluate Live Link Frame at Scene Time");
}

FName UK2Node_EvaluateLiveLinkFrameWithSpecificRole::GetEvaluateFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(ULiveLinkBlueprintLibrary, EvaluateLiveLinkFrameWithSpecificRole);
}

FName UK2Node_EvaluateLiveLinkFrameAtWorldTime::GetEvaluateFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(ULiveLinkBlueprintLibrary, EvaluateLiveLinkFrameAtWorldTimeOffset);
}

FName UK2Node_EvaluateLiveLinkFrameAtSceneTime::GetEvaluateFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(ULiveLinkBlueprintLibrary, EvaluateLiveLinkFrameAtSceneTime);
}

void UK2Node_EvaluateLiveLinkFrameWithSpecificRole::AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* EvaluateLiveLinkFrameFunction)
{
}

void UK2Node_EvaluateLiveLinkFrameAtWorldTime::AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* EvaluateLiveLinkFrameFunction)
{
	UEdGraphPin* InPinSwitch = FindPinChecked(UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkWorldTimePinName);
	UEdGraphPin* TimePin = EvaluateLiveLinkFrameFunction->FindPinChecked(TEXT("WorldTime"));
	CompilerContext.CopyPinLinksToIntermediate(*InPinSwitch, *TimePin);
}

void UK2Node_EvaluateLiveLinkFrameAtSceneTime::AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* EvaluateLiveLinkFrameFunction)
{
	UEdGraphPin* InPinSwitch = FindPinChecked(UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkSceneTimePinName);
	UEdGraphPin* TimePin = EvaluateLiveLinkFrameFunction->FindPinChecked(TEXT("SceneTime"));
	CompilerContext.CopyPinLinksToIntermediate(*InPinSwitch, *TimePin);
}

#undef LOCTEXT_NAMESPACE

