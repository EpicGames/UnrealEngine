// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_EvaluateLiveLinkFrame.h"

#include "K2Node_EvaluateLiveLinkCustom.generated.h"

namespace ENodeTitleType { enum Type : int; }

UCLASS()
class UK2Node_EvaluateLiveLinkFrameWithSpecificRole : public UK2Node_EvaluateLiveLinkFrame
{
	GENERATED_BODY()

public:
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	virtual FName GetEvaluateFunctionName() const override;
	virtual void AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* EvaluateLiveLinkFrameFunction) override;
};

UCLASS()
class UK2Node_EvaluateLiveLinkFrameAtWorldTime : public UK2Node_EvaluateLiveLinkFrame
{
	GENERATED_BODY()

public:
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	virtual FName GetEvaluateFunctionName() const override;
	virtual void AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* EvaluateLiveLinkFrameFunction) override;
};

UCLASS()
class UK2Node_EvaluateLiveLinkFrameAtSceneTime : public UK2Node_EvaluateLiveLinkFrame
{
	GENERATED_BODY()

public:
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	virtual FName GetEvaluateFunctionName() const override;
	virtual void AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* EvaluateLiveLinkFrameFunction) override;
};
