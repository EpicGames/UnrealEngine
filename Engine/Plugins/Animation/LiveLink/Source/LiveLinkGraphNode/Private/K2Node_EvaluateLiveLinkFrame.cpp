// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EvaluateLiveLinkFrame.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "K2Node_IfThenElse.h"
#include "KismetCompiler.h"
#include "LiveLinkRole.h"
#include "LiveLinkBlueprintLibrary.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EvaluateLiveLinkFrame)

#define LOCTEXT_NAMESPACE "K2Node_EvaluateLiveLinkFrame"


namespace UK2Node_EvaluateLiveLinkFrameHelper
{
	static FName LiveLinkSubjectPinName = "Subject";
	static FName LiveLinkRolePinName = "Role";
	static FName LiveLinkDataResultPinName = "DataResult";
	static FName FrameNotAvailablePinName = "InvalidFrame";
};

void UK2Node_EvaluateLiveLinkFrame::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Add execution pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// Output pin
	{
		UEdGraphPin* FrameAvailablePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
		FrameAvailablePin->PinFriendlyName = LOCTEXT("FrameAvailablePin", "Valid Frame");
		UEdGraphPin* FrameInvalidePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UK2Node_EvaluateLiveLinkFrameHelper::FrameNotAvailablePinName);
		FrameInvalidePin->PinFriendlyName = LOCTEXT("FrameNotAvailablePinName", "Invalid Frame");
	}
	// Output structs pins
	{
		UEdGraphPin* DataResultPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkDataResultPinName);
		DataResultPin->PinFriendlyName = LOCTEXT("LiveLinkDataResultPinName", "DataResult");
		SetPinToolTip(*DataResultPin, LOCTEXT("DataResultPinDescription", "The data struct, if a frame was present for the given role"));
	}

	// Subject pin
	{
		UEdGraphPin* LiveLinkSubjectRepPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FLiveLinkSubjectName::StaticStruct(), UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkSubjectPinName);
		LiveLinkSubjectRepPin->PinFriendlyName = LOCTEXT("LiveLinkSubjectPinName", "Subject");
		SetPinToolTip(*LiveLinkSubjectRepPin, LOCTEXT("LiveLinkSubjectNamePinDescription", "The Live Link Subject Name to get a frame from"));
	}
	// Role pin
	{
		UEdGraphPin* LiveLinkRoleRepPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, ULiveLinkRole::StaticClass(), UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkRolePinName);
		LiveLinkRoleRepPin->PinFriendlyName = LOCTEXT("LiveLinkRolePinName", "Role");
		LiveLinkRoleRepPin->bNotConnectable = true;
		SetPinToolTip(*LiveLinkRoleRepPin, LOCTEXT("LiveLinkRolePinNameDescription", "The Live Link Role the data will be converted to."));
	}

	Super::AllocateDefaultPins();
}

void UK2Node_EvaluateLiveLinkFrame::SetPinToolTip(UEdGraphPin& InOutMutatablePin, const FText& InPinDescription) const
{
	InOutMutatablePin.PinToolTip = UEdGraphSchema_K2::TypeToText(InOutMutatablePin.PinType).ToString();

	UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	if (K2Schema != nullptr)
	{
		InOutMutatablePin.PinToolTip += TEXT(" ");
		InOutMutatablePin.PinToolTip += K2Schema->GetPinDisplayName(&InOutMutatablePin).ToString();
	}

	InOutMutatablePin.PinToolTip += FString(TEXT("\n")) + InPinDescription.ToString();
}

void UK2Node_EvaluateLiveLinkFrame::RefreshDataOutputPinType()
{
	UScriptStruct* DataType = GetLiveLinkRoleOutputStructType();
	SetReturnTypeForOutputStruct(DataType);
}

void UK2Node_EvaluateLiveLinkFrame::SetReturnTypeForOutputStruct(UScriptStruct* InClass)
{
	UScriptStruct* OldDataStruct = GetReturnTypeForOutputDataStruct();
	if (InClass != OldDataStruct)
	{
		UEdGraphPin* ResultPin = GetResultingDataPin();

		if (ResultPin->SubPins.Num() > 0)
		{
			GetSchema()->RecombinePin(ResultPin);
		}

		// NOTE: purposefully not disconnecting the ResultPin (even though it changed type)... we want the user to see the old
		//       connections, and incompatible connections will produce an error (plus, some super-struct connections may still be valid)
		ResultPin->PinType.PinSubCategoryObject = InClass;
		ResultPin->PinType.PinCategory = (InClass == nullptr) ? UEdGraphSchema_K2::PC_Wildcard : UEdGraphSchema_K2::PC_Struct;
	}
}

UScriptStruct* UK2Node_EvaluateLiveLinkFrame::GetReturnTypeForOutputDataStruct() const
{
	return Cast<UScriptStruct>(GetResultingDataPin()->PinType.PinSubCategoryObject.Get());
}

UScriptStruct* UK2Node_EvaluateLiveLinkFrame::GetLiveLinkRoleOutputStructType() const
{
	UScriptStruct* DataStructType = nullptr;

	TSubclassOf<ULiveLinkRole> Role = GetDefaultRolePinValue();
	if (IsRoleValidForEvaluation(Role))
	{
		DataStructType = Role.GetDefaultObject()->GetBlueprintDataStruct();
	}
	return DataStructType;
}

void UK2Node_EvaluateLiveLinkFrame::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);
}

void UK2Node_EvaluateLiveLinkFrame::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_EvaluateLiveLinkFrame::GetMenuCategory() const
{
	return FText::FromString(TEXT("LiveLink"));
}

bool UK2Node_EvaluateLiveLinkFrame::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if (MyPin == GetResultingDataPin() && MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		bool bDisallowed = true;
		if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			bDisallowed = false;
			if (UScriptStruct* ConnectionType = Cast<UScriptStruct>(OtherPin->PinType.PinSubCategoryObject.Get()))
			{
				bDisallowed = GetLiveLinkRoleOutputStructType() != ConnectionType;
			}
		}
		else if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		{
			bDisallowed = false;
		}

		if (bDisallowed)
		{
			OutReason = TEXT("Must be a struct that inherits from FLiveLinkBaseBlueprintData");
		}
		return bDisallowed;
	}

	if (MyPin == GetLiveLinkRolePin())
	{
		return true;
	}

	return false;
}

void UK2Node_EvaluateLiveLinkFrame::PinDefaultValueChanged(UEdGraphPin* ChangedPin)
{
	UEdGraphPin* LiveLinkRolePin = GetLiveLinkRolePin();
	if (ChangedPin == LiveLinkRolePin && LiveLinkRolePin)
	{
		if (LiveLinkRolePin->DefaultObject != nullptr && LiveLinkRolePin->LinkedTo.Num() == 0)
		{
			UClass* ClassValue = Cast<UClass>(LiveLinkRolePin->DefaultObject);
			bool bIsValid = ClassValue && ClassValue->IsChildOf(ULiveLinkRole::StaticClass());
			if (!bIsValid)
			{
				LiveLinkRolePin->DefaultObject = nullptr;
			}
			RefreshDataOutputPinType();
		}
	}
}

FText UK2Node_EvaluateLiveLinkFrame::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Attempts to Get a LiveLink Frame from a subject using a given Role");
}

UEdGraphPin* UK2Node_EvaluateLiveLinkFrame::GetLiveLinkRolePin() const
{
	UEdGraphPin* Pin = FindPinChecked(UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkRolePinName);
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_EvaluateLiveLinkFrame::GetLiveLinkSubjectPin() const
{
	UEdGraphPin* Pin = FindPinChecked(UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkSubjectPinName);
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_EvaluateLiveLinkFrame::GetFrameNotAvailablePin() const
{
	UEdGraphPin* Pin = FindPinChecked(UK2Node_EvaluateLiveLinkFrameHelper::FrameNotAvailablePinName);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

UEdGraphPin* UK2Node_EvaluateLiveLinkFrame::GetResultingDataPin() const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* Pin = FindPinChecked(UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkDataResultPinName);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

void UK2Node_EvaluateLiveLinkFrame::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	TSubclassOf<ULiveLinkRole> Role = GetDefaultRolePinValue();
	if (Role == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("EvaluateLiveLinkRoleNoRole_Error", "EvaluateLiveLinkFrame must have a Role specified.").ToString(), this);
		// we break exec links so this is the only error we get
		BreakAllNodeLinks();
		return;
	}

	// FUNCTION NODE
	const FName FunctionName = GetEvaluateFunctionName();
	UK2Node_CallFunction* EvaluateLiveLinkFrameFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	EvaluateLiveLinkFrameFunction->FunctionReference.SetExternalMember(FunctionName, ULiveLinkBlueprintLibrary::StaticClass());
	EvaluateLiveLinkFrameFunction->AllocateDefaultPins();
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *(EvaluateLiveLinkFrameFunction->GetExecPin()));

	// Connect the input of our EvaluateLiveLinkFrame to the Input of our Function pin
	{
		UEdGraphPin* OriginalLiveLinkSubjectPin = GetLiveLinkSubjectPin();
		UEdGraphPin* LiveLinkSubjectInPin = EvaluateLiveLinkFrameFunction->FindPinChecked(TEXT("SubjectName"));
		CompilerContext.CopyPinLinksToIntermediate(*OriginalLiveLinkSubjectPin, *LiveLinkSubjectInPin);
	}
	{
		UEdGraphPin* OriginalLiveLinkRolePin = GetLiveLinkRolePin();
		UEdGraphPin* LiveLinkRoleInPin = EvaluateLiveLinkFrameFunction->FindPinChecked(TEXT("Role"));
		CompilerContext.CopyPinLinksToIntermediate(*OriginalLiveLinkRolePin, *LiveLinkRoleInPin);
	}

	AddPins(CompilerContext, EvaluateLiveLinkFrameFunction);

	// Get some pins to work with
	UEdGraphPin* OriginalDataOutPin = FindPinChecked(UK2Node_EvaluateLiveLinkFrameHelper::LiveLinkDataResultPinName);
	UEdGraphPin* FunctionDataOutPin = EvaluateLiveLinkFrameFunction->FindPinChecked(TEXT("OutBlueprintData"));
	UEdGraphPin* FunctionReturnPin = EvaluateLiveLinkFrameFunction->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* FunctionThenPin = EvaluateLiveLinkFrameFunction->GetThenPin();

	// Set the type of each output pins on this expanded mode to match original
	FunctionDataOutPin->PinType = OriginalDataOutPin->PinType;
	FunctionDataOutPin->PinType.PinSubCategoryObject = OriginalDataOutPin->PinType.PinSubCategoryObject;

	//BRANCH NODE
	UK2Node_IfThenElse* BranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	BranchNode->AllocateDefaultPins();
	// Hook up inputs to branch
	FunctionThenPin->MakeLinkTo(BranchNode->GetExecPin());
	FunctionReturnPin->MakeLinkTo(BranchNode->GetConditionPin());

	// Hook up outputs
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *(BranchNode->GetThenPin()));
	CompilerContext.MovePinLinksToIntermediate(*GetFrameNotAvailablePin(), *(BranchNode->GetElsePin()));
	CompilerContext.MovePinLinksToIntermediate(*OriginalDataOutPin, *FunctionDataOutPin);

	BreakAllNodeLinks();
}

FSlateIcon UK2Node_EvaluateLiveLinkFrame::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = GetNodeTitleColor();
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return Icon;
}

void UK2Node_EvaluateLiveLinkFrame::PostReconstructNode()
{
	Super::PostReconstructNode();

	RefreshDataOutputPinType();
}

void UK2Node_EvaluateLiveLinkFrame::EarlyValidation(class FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);

	const UEdGraphPin* LiveLinkSubjectPin = GetLiveLinkSubjectPin();
	if (!LiveLinkSubjectPin)
	{
		MessageLog.Error(*LOCTEXT("MissingPins", "Missing pins in @@").ToString(), this);
		return;
	}

	TSubclassOf<ULiveLinkRole> Role = GetDefaultRolePinValue();
	if (Role.Get() == nullptr)
	{
		MessageLog.Error(*LOCTEXT("NoLiveLinkRole", "No LiveLinkRole in @@").ToString(), this);
		return;
	}

	if (!IsRoleValidForEvaluation(Role))
	{
		MessageLog.Error(*FText::Format(LOCTEXT("InvalidRoleClass", "Cannot EvaluateFrame for Role of type '{0}' in @@"), FText::FromString(Role->GetFName().ToString())).ToString(), this);
		return;
	}

	UScriptStruct* DesiredOutputStruct = GetLiveLinkRoleOutputStructType();
	UScriptStruct* ActualOutputStruct = GetReturnTypeForOutputDataStruct();
	if (DesiredOutputStruct != ActualOutputStruct)
	{
		MessageLog.Error(*LOCTEXT("OutputPinDoNotMatches", "The output data pin does not match in @@").ToString(), this);
		return;
	}

	if (ActualOutputStruct != nullptr)
	{
		UEdGraphPin* ResultPin = GetResultingDataPin();
		if (ResultPin && ResultPin->LinkedTo.Num() > 0)
		{
			for (UEdGraphPin* LinkPin : ResultPin->LinkedTo)
			{
				UScriptStruct* LinkType = Cast<UScriptStruct>(LinkPin->PinType.PinSubCategoryObject.Get());
				if (ActualOutputStruct != LinkType)
				{
					MessageLog.Error(*LOCTEXT("OutputLinkPinDoNotMatches", "A linked pin data structure do not match the Live Link Role data structure in @@. Expected: @@.").ToString(), this, ActualOutputStruct);
					return;
				}
			}
		}
	}
}

void UK2Node_EvaluateLiveLinkFrame::PreloadRequiredAssets()
{
	return Super::PreloadRequiredAssets();
}

void UK2Node_EvaluateLiveLinkFrame::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	UEdGraphPin* LiveLinkRolePin = GetLiveLinkRolePin();
	if (Pin == GetResultingDataPin())
	{
		// this connection would only change the output type if the role pin is undefined
		const bool bIsTypeAuthority = (LiveLinkRolePin->LinkedTo.Num() <= 0 && LiveLinkRolePin->DefaultObject == nullptr);
		if (bIsTypeAuthority)
		{
			RefreshDataOutputPinType();
		}
	}
	else if (Pin == LiveLinkRolePin)
	{
		const bool bConnectionAdded = Pin->LinkedTo.Num() > 0;
		if (bConnectionAdded)
		{
			RefreshDataOutputPinType();
		}
	}
}

TSubclassOf<ULiveLinkRole> UK2Node_EvaluateLiveLinkFrame::GetDefaultRolePinValue() const
{
	UEdGraphPin* LiveLinkRolePin = GetLiveLinkRolePin();
	if (LiveLinkRolePin && LiveLinkRolePin->DefaultObject != nullptr && LiveLinkRolePin->LinkedTo.Num() == 0)
	{
		UClass* ClassValue = Cast<UClass>(LiveLinkRolePin->DefaultObject);
		if (ClassValue && ClassValue->IsChildOf(ULiveLinkRole::StaticClass()))
		{
			return ClassValue;
		}
	}
	return nullptr;
}


bool UK2Node_EvaluateLiveLinkFrame::IsRoleValidForEvaluation(TSubclassOf<ULiveLinkRole> InRoleClass) const
{
	return InRoleClass.Get() && !InRoleClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_HideDropDown);
}

#undef LOCTEXT_NAMESPACE

