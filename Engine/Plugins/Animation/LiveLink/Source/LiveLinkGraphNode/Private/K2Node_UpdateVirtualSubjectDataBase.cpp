// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_UpdateVirtualSubjectDataBase.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "KismetCompiler.h"
#include "LiveLinkRole.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "VirtualSubjects/LiveLinkBlueprintVirtualSubject.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_UpdateVirtualSubjectDataBase)

#define LOCTEXT_NAMESPACE "K2Node_UpdateVirtualSubjectDataBase"

const FName UK2Node_UpdateVirtualSubjectDataBase::LiveLinkStructPinName = "Struct";

const FText UK2Node_UpdateVirtualSubjectDataBase::LiveLinkStructPinDescription = LOCTEXT("LiveLinkStructPinDescription", "The Struct of data to use in the update.");


UScriptStruct* UK2Node_UpdateVirtualSubjectDataBase::GetStructTypeFromBlueprint() const
{
	if (HasValidBlueprint())
	{
		UBlueprint* Blueprint = GetBlueprint();
		if (Blueprint->GeneratedClass.Get()->IsChildOf(ULiveLinkBlueprintVirtualSubject::StaticClass()))
		{
			if (Blueprint->GeneratedClass->bLayoutChanging)
			{
				// Early out if the layout is changing as it's not valid to get the CDO
				return nullptr;
			}

			if (const TSubclassOf<ULiveLinkRole> RoleClass = Blueprint->GeneratedClass->GetDefaultObject<ULiveLinkBlueprintVirtualSubject>()->GetRole())
			{
				if (ULiveLinkRole* Role = RoleClass->GetDefaultObject<ULiveLinkRole>())
				{
					return GetStructTypeFromRole(Role);
				}
			}
		}
	}
	return nullptr;
}

void UK2Node_UpdateVirtualSubjectDataBase::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Add execution pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// Struct pin
	{
		UScriptStruct* DataStructType = GetStructTypeFromBlueprint();
		FName PinCategory = (DataStructType == nullptr) ? UEdGraphSchema_K2::PC_Wildcard : UEdGraphSchema_K2::PC_Struct;
		UEdGraphPin* LiveLinkStructPin = CreatePin(EGPD_Input, PinCategory, DataStructType, LiveLinkStructPinName);
		LiveLinkStructPin->PinFriendlyName = GetStructPinName();
		SetPinToolTip(*LiveLinkStructPin, LiveLinkStructPinDescription);
	}

	Super::AllocateDefaultPins();
}

FText UK2Node_UpdateVirtualSubjectDataBase::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Update Virtual Subject Data");
}

FText UK2Node_UpdateVirtualSubjectDataBase::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Updates the data for a specified Virtual Subject.");
}

void UK2Node_UpdateVirtualSubjectDataBase::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UK2Node_CallFunction* InternalFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	InternalFunction->FunctionReference.SetSelfMember(GetUpdateFunctionName());
	InternalFunction->AllocateDefaultPins();
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *(InternalFunction->GetExecPin()));
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *(InternalFunction->GetThenPin()));

	{
		UEdGraphPin* LiveLinkStructPin = GetLiveLinkStructPin();
		UEdGraphPin* InStructPin = InternalFunction->FindPinChecked(TEXT("InStruct"));
		InStructPin->PinType = LiveLinkStructPin->PinType;
		CompilerContext.MovePinLinksToIntermediate(*LiveLinkStructPin, *InStructPin);
	}

	AddPins(CompilerContext, InternalFunction);

	BreakAllNodeLinks();
}

FSlateIcon UK2Node_UpdateVirtualSubjectDataBase::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = GetNodeTitleColor();
	static const FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return Icon;
}

bool UK2Node_UpdateVirtualSubjectDataBase::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	// Make sure we're valid under the graph schema
	if(CanCreateUnderSpecifiedSchema(TargetGraph->GetSchema()))
	{
		// Only valid in ULiveLinkBlueprintVirtualSubject derived classes
		if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph))
		{
			return Blueprint->ParentClass != nullptr && Blueprint->ParentClass->IsChildOf(ULiveLinkBlueprintVirtualSubject::StaticClass());
		}
	}
	return false;
}

void UK2Node_UpdateVirtualSubjectDataBase::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_UpdateVirtualSubjectDataBase::GetMenuCategory() const
{
	return LOCTEXT("MenuCategory", "LiveLink");
}

bool UK2Node_UpdateVirtualSubjectDataBase::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	return false;
}

void UK2Node_UpdateVirtualSubjectDataBase::EarlyValidation(class FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);

	if (UEdGraphPin* StructPin = GetLiveLinkStructPin())
	{
		if (!StructPin->HasAnyConnections() && !StructPin->SubPins.Num())
		{
			MessageLog.Error(*LOCTEXT("StructNoConnectionError", "Pin @@ requires a connection or needs to be expanded.").ToString(), StructPin);
		}
	}
}

UEdGraphPin* UK2Node_UpdateVirtualSubjectDataBase::GetLiveLinkStructPin() const
{
	UEdGraphPin* Pin = FindPinChecked(LiveLinkStructPinName);
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

void UK2Node_UpdateVirtualSubjectDataBase::SetPinToolTip(UEdGraphPin& InOutMutatablePin, const FText& InPinDescription) const
{
	InOutMutatablePin.PinToolTip = UEdGraphSchema_K2::TypeToText(InOutMutatablePin.PinType).ToString();

	UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	if (K2Schema != nullptr)
	{
		InOutMutatablePin.PinToolTip += TEXT(" ");
		InOutMutatablePin.PinToolTip += K2Schema->GetPinDisplayName(&InOutMutatablePin).ToString();
	}

	InOutMutatablePin.PinToolTip += FString(TEXT("\n")) + InPinDescription.ToString();
}

#undef LOCTEXT_NAMESPACE
