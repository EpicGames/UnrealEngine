// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_UpdateVirtualSubjectDataTyped.h"

#include "K2Node_CallFunction.h"
#include "KismetCompiler.h"
#include "LiveLinkRole.h"
#include "VirtualSubjects/LiveLinkBlueprintVirtualSubject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_UpdateVirtualSubjectDataTyped)

#define LOCTEXT_NAMESPACE "K2Node_UpdateVirtualSubjectDataTyped"


const FName UK2Node_UpdateVirtualSubjectFrameData::LiveLinkTimestampFramePinName = "TimestampFrame";


FText UK2Node_UpdateVirtualSubjectStaticData::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle_Static", "Update Virtual Subject Static Data");
}

UScriptStruct* UK2Node_UpdateVirtualSubjectStaticData::GetStructTypeFromRole(ULiveLinkRole* Role) const
{
	check(Role);
	return Role->GetStaticDataStruct();
}

FName UK2Node_UpdateVirtualSubjectStaticData::GetUpdateFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(ULiveLinkBlueprintVirtualSubject, UpdateVirtualSubjectStaticData_Internal);
}

FText UK2Node_UpdateVirtualSubjectStaticData::GetStructPinName() const
{
	return LOCTEXT("StaticPinName", "Static Data");
}

void UK2Node_UpdateVirtualSubjectFrameData::AllocateDefaultPins()
{

	Super::AllocateDefaultPins();

	// bool pin to enable/disable timestamping the frame data internally. True by default
	UEdGraphPin* StampTimePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, UK2Node_UpdateVirtualSubjectFrameData::LiveLinkTimestampFramePinName);
	StampTimePin->PinFriendlyName = LOCTEXT("LiveLinkTimestampFramePin", "Timestamp Frame");
	StampTimePin->DefaultValue = FString(TEXT("true"));
	SetPinToolTip(*StampTimePin, LOCTEXT("LiveLinkTimestampFramePinDescription", "Whether to timestamp frame data internally"));
}

FText UK2Node_UpdateVirtualSubjectFrameData::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle_Frame", "Update Virtual Subject Frame Data");
}

UScriptStruct* UK2Node_UpdateVirtualSubjectFrameData::GetStructTypeFromRole(ULiveLinkRole* Role) const
{
	check(Role);
	return Role->GetFrameDataStruct();
}

FName UK2Node_UpdateVirtualSubjectFrameData::GetUpdateFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(ULiveLinkBlueprintVirtualSubject, UpdateVirtualSubjectFrameData_Internal);
}

FText UK2Node_UpdateVirtualSubjectFrameData::GetStructPinName() const
{
	return LOCTEXT("FramePinName", "Frame Data");
}

void UK2Node_UpdateVirtualSubjectFrameData::AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* UpdateVirtualSubjectDataFunction) const
{
	UEdGraphPin* InPinSwitch = FindPinChecked(UK2Node_UpdateVirtualSubjectFrameData::LiveLinkTimestampFramePinName);
	UEdGraphPin* TimestampPin = UpdateVirtualSubjectDataFunction->FindPinChecked(TEXT("bInShouldStampCurrentTime"));
	CompilerContext.CopyPinLinksToIntermediate(*InPinSwitch, *TimestampPin);
}

#undef LOCTEXT_NAMESPACE
