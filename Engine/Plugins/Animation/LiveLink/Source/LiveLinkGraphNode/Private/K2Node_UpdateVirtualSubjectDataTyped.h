// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_UpdateVirtualSubjectDataBase.h"
#include "K2Node_UpdateVirtualSubjectDataTyped.generated.h"

#define UE_API LIVELINKGRAPHNODE_API

UCLASS(MinimalAPI)
class UK2Node_UpdateVirtualSubjectStaticData : public UK2Node_UpdateVirtualSubjectDataBase
{
	GENERATED_BODY()

public:

	//~ Begin UEdGraphNode Interface.
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

protected:

	//~ Begin UK2Node_UpdateVirtualSubjectDataBase interface
	UE_API virtual UScriptStruct* GetStructTypeFromRole(ULiveLinkRole* Role) const override;
	UE_API virtual FName GetUpdateFunctionName() const override;
	UE_API virtual FText GetStructPinName() const override;
	virtual void AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* UpdateVirtualSubjectDataFunction) const override {}
	//~ End UK2Node_UpdateVirtualSubjectDataBase interface
};

UCLASS(MinimalAPI)
class UK2Node_UpdateVirtualSubjectFrameData : public UK2Node_UpdateVirtualSubjectDataBase
{
	GENERATED_BODY()

public:

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

protected:

	//~ Begin UK2Node_UpdateVirtualSubjectDataBase interface
	UE_API virtual UScriptStruct* GetStructTypeFromRole(ULiveLinkRole* Role) const override;
	UE_API virtual FName GetUpdateFunctionName() const override;
	UE_API virtual FText GetStructPinName() const override;
	UE_API virtual void AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* UpdateVirtualSubjectDataFunction) const override;
	//~ End UK2Node_UpdateVirtualSubjectDataBase interface

	/** Returns the Timestamp pin  */
	UE_API UEdGraphPin* GetTimestampFramePin() const;

private:

	/** Name of the pin to enable/disable timestamping */
	static UE_API const FName LiveLinkTimestampFramePinName;
};

#undef UE_API
