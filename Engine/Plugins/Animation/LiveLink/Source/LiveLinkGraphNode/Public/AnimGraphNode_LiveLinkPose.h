// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "AnimNode_LiveLinkPose.h"

#include "AnimGraphNode_LiveLinkPose.generated.h"

#define UE_API LIVELINKGRAPHNODE_API

namespace ENodeTitleType { enum Type : int; }



UCLASS(MinimalAPI)
class UAnimGraphNode_LiveLinkPose : public UAnimGraphNode_Base
{
	GENERATED_BODY()

public:

	//~ UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetMenuCategory() const;
	//~ End of UEdGraphNode

	//~ Begin UK2Node interface
	UE_API virtual void ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges) override;
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	//~ End UK2Node interface

public:

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_LiveLinkPose Node;

};

#undef UE_API
