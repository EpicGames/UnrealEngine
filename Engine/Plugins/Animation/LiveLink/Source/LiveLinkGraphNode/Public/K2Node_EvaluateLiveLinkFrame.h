// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"

#include "UObject/Package.h"
#include "K2Node_EvaluateLiveLinkFrame.generated.h"

#define UE_API LIVELINKGRAPHNODE_API

class ULiveLinkRole;

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class UDataTable;
class UEdGraph;
class UK2Node_CallFunction;


UCLASS(MinimalAPI, Abstract)
class UK2Node_EvaluateLiveLinkFrame : public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual void PostReconstructNode() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	UE_API virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual void PreloadRequiredAssets() override;
	UE_API virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	//~ End UK2Node Interface

	/** Get the return types of our struct */
	UE_API UScriptStruct* GetReturnTypeForOutputDataStruct() const;

	/** Get the Live Link Role input pin */
	UE_API UEdGraphPin* GetLiveLinkRolePin() const;

	/** Get the Live Link Subject input pin */
	UE_API UEdGraphPin* GetLiveLinkSubjectPin() const;

	/** Get the exec output pin for when no frame is available for the desired role */
	UE_API UEdGraphPin* GetFrameNotAvailablePin() const;

	/** Get the result output pin */
	UE_API UEdGraphPin* GetResultingDataPin() const;

	/** Get the type that the Live Link Role evaluates to to return */
	UE_API UScriptStruct* GetLiveLinkRoleOutputStructType() const;
	UE_API UScriptStruct* GetLiveLinkRoleOutputFrameStructType() const;

protected:

	/** Get the name of the UFunction we are trying to build from */
	virtual FName GetEvaluateFunctionName() const PURE_VIRTUAL(UK2Node_EvaluateLiveLinkFrame::GetEvaluateFunctionName, return NAME_None; );

	/** Add additionnal pin that the evaluate function would need */
	virtual void AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* EvaluateLiveLinkFrameFunction) PURE_VIRTUAL(UK2Node_EvaluateLiveLinkFrame::AddPins, );

	/**
	 * Takes the specified "MutatablePin" and sets its 'PinToolTip' field (according
	 * to the specified description)
	 *
	 * @param   MutatablePin	The pin you want to set tool-tip text on
	 * @param   PinDescription	A string describing the pin's purpose
	 */
	UE_API void SetPinToolTip(UEdGraphPin& InOutMutatablePin, const FText& InPinDescription) const;

	/** Set the return type of our structs */
	UE_API void SetReturnTypeForOutputStruct(UScriptStruct* InClass);

	/** Queries for the authoritative return type, then modifies the return pin to match */
	UE_API void RefreshDataOutputPinType();

	UE_API TSubclassOf<ULiveLinkRole> GetDefaultRolePinValue() const;

	/** Verify if selected role class is valid for evaluation */
	UE_API bool IsRoleValidForEvaluation(TSubclassOf<ULiveLinkRole> InRoleClass) const;
};

#undef UE_API
