// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"

#include "K2Node_UpdateVirtualSubjectDataBase.generated.h"

#define UE_API LIVELINKGRAPHNODE_API

namespace ENodeTitleType { enum Type : int; }

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class UDataTable;
class UEdGraph;
class UK2Node_CallFunction;
class ULiveLinkRole;

UCLASS(MinimalAPI, Abstract)
class UK2Node_UpdateVirtualSubjectDataBase : public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	UE_API virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UK2Node Interface

	/** Get the Live Link Struct input pin */
	UE_API UEdGraphPin* GetLiveLinkStructPin() const;

protected:
	/**
	 * Takes the specified "MutatablePin" and sets its 'PinToolTip' field (according
	 * to the specified description)
	 *
	 * @param   MutatablePin	The pin you want to set tool-tip text on
	 * @param   PinDescription	A string describing the pin's purpose
	 */
	UE_API void SetPinToolTip(UEdGraphPin& InOutMutatablePin, const FText& InPinDescription) const;

	/** Returns Struct type associated to the subject's role (static or frame) */
	virtual UScriptStruct* GetStructTypeFromRole(ULiveLinkRole* Role) const PURE_VIRTUAL(UK2Node_UpdateVirtualSubjectDataBase::GetStructTypeFromRole, return nullptr; );

	/** Returns the custom thunk function name */
	virtual FName GetUpdateFunctionName() const PURE_VIRTUAL(UK2Node_UpdateVirtualSubjectDataBase::GetUpdateFunctionName, return NAME_None; );

	/** Returns the struct display name */
	virtual FText GetStructPinName() const PURE_VIRTUAL(UK2Node_UpdateVirtualSubjectDataBase::GetStructPinName, return FText::GetEmpty(); );

	/** Add additionnal pins that the update subject function could need */
	virtual void AddPins(FKismetCompilerContext& CompilerContext, UK2Node_CallFunction* UpdateVirtualSubjectDataFunction) const PURE_VIRTUAL(UK2Node_UpdateVirtualSubjectDataBase::AddPins, );

private:

	/** Returns the Struct type associated to the role */
	UE_API UScriptStruct* GetStructTypeFromBlueprint() const;

private:
	
	/** Name of the struct pin */
	static UE_API const FName LiveLinkStructPinName;

	/** Struct pin description */
	static UE_API const FText LiveLinkStructPinDescription;
};

#undef UE_API
