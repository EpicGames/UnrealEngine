// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkEnumHandler.h"

#include "Channels/MovieSceneByteChannel.h"

#include "Channels/MovieSceneChannelData.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"
#include "UObject/EnumProperty.h"


//------------------------------------------------------------------------------
// FMovieSceneLiveLinkEnumHandler implementation.
//------------------------------------------------------------------------------

void FMovieSceneLiveLinkEnumHandler::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		check(ArrayProperty->Inner->IsA<FEnumProperty>());
	}
	else
	{
		check(FoundProperty->IsA<FEnumProperty>());
	}

	PropertyStorage->ByteChannel.SetNum(InElementCount);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}

void FMovieSceneLiveLinkEnumHandler::RecordFrame(const FFrameNumber& InFrameNumber, const UScriptStruct& InStruct, const FLiveLinkBaseFrameData* InFrameData) 
{
	if (InFrameData != nullptr)
	{
		for (int32 i = 0; i < ElementCount; ++i)
		{
			const int64 NewValue = PropertyBinding.GetCurrentValueForEnumAt(i, InStruct, InFrameData);

			FLiveLinkPropertyKey<int64> Key;
			Key.Time = InFrameNumber;
			Key.Value = NewValue;
			Keys[i].Add(Key);
		}
	}
}

void FMovieSceneLiveLinkEnumHandler::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<int64>>& ElementKeys = Keys[i];
		for (const FLiveLinkPropertyKey<int64>& Key : ElementKeys)
		{
			PropertyStorage->ByteChannel[i].GetData().AddKey(Key.Time, Key.Value);
		}
	}

	if (bInReduceKeys)
	{
		// Reduce keys intentionally left blank
	}
}

void FMovieSceneLiveLinkEnumHandler::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	ElementCount = PropertyStorage->ByteChannel.Num();
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			check(ArrayProperty->Inner->IsA<FEnumProperty>());
		}
		else
		{
			check(FoundProperty->IsA<FEnumProperty>());
		}
	}
}

void FMovieSceneLiveLinkEnumHandler::FillFrame(int32 InKeyIndex, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame)
{
	int32 ChannelIndex = 0;
	for (int32 i = 0; i < ElementCount; ++i)
	{
		const int64 Value = GetChannelValue(InKeyIndex, i);
		PropertyBinding.SetCurrentValueForEnumAt(i, InStruct, OutFrame, Value);
	}
}

void FMovieSceneLiveLinkEnumHandler::FillFrameInterpolated(const FFrameTime& InFrameTime, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame)
{
	int32 ChannelIndex = 0;
	for (int32 i = 0; i < ElementCount; ++i)
	{
		const int64 Value = GetChannelValueInterpolated(InFrameTime, i);
		PropertyBinding.SetCurrentValueForEnumAt(i, InStruct, OutFrame, Value);
	}
}

int64 FMovieSceneLiveLinkEnumHandler::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	return PropertyStorage->ByteChannel[InChannelIndex].GetData().GetValues()[InKeyIndex];
}

int64 FMovieSceneLiveLinkEnumHandler::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	uint8 Value;
	PropertyStorage->ByteChannel[InChannelIndex].Evaluate(InFrameTime, Value);
	return (int64)Value;
}

