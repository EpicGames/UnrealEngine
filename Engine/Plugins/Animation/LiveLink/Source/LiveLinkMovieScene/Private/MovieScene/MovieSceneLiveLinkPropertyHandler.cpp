// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkPropertyHandler.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneStringChannel.h"


namespace LiveLinkPropertyHandlerUtils
{
	void FillVectorInterpolated(const FFrameTime& InFrameTime, int32& InOutStartIndex, const TArray<FMovieSceneFloatChannel>& InSourceChannels, FVector& OutVector)
	{
		FVector3f TempVector;

		for (int32 i = 0; i < 3; ++i)
		{
			InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempVector[i]);
		}

		OutVector = (FVector)TempVector;
	}

	void FillVector(int32 InKeyIndex, int32& InOutStartIndex, const TArray<FMovieSceneFloatChannel>& InSourceChannels, FVector& OutVector)
	{
		FVector TempVector;

		for (int32 i = 0; i < 3; ++i)
		{
			TempVector[i] = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex].Value;
			++InOutStartIndex;
		}

		OutVector = TempVector;
	}

	void FillColorInterpolated(const FFrameTime& InFrameTime, int32& InOutStartIndex, const TArray<FMovieSceneByteChannel>& InSourceChannels, FColor& OutColor)
	{
		FColor TempColor;

		InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempColor.R);
		InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempColor.G);
		InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempColor.B);
		InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempColor.A);
		OutColor = TempColor;
	}

	void FillColor(int32 InKeyIndex, int32& InOutStartIndex, const TArray<FMovieSceneByteChannel>& InSourceChannels, FColor& OutColor)
	{
		FColor TempColor;

		TempColor.R = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex];
		++InOutStartIndex;
		TempColor.G = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex];
		++InOutStartIndex;
		TempColor.B = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex];
		++InOutStartIndex;
		TempColor.A = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex];
		++InOutStartIndex;
		OutColor = TempColor;
	}
};

//------------------------------------------------------------------------------
// FMovieSceneLiveLinkPropertyHandler implementation - Float specialization.
//------------------------------------------------------------------------------

template <>
void FMovieSceneLiveLinkPropertyHandler<float>::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		check(ArrayProperty->Inner->IsA<FFloatProperty>());
	}
	else
	{
		check(FoundProperty->IsA<FFloatProperty>());
	}

	PropertyStorage->FloatChannel.SetNum(InElementCount);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}

template <>
void FMovieSceneLiveLinkPropertyHandler<float>::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	ElementCount = PropertyStorage->FloatChannel.Num();
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			check(ArrayProperty->Inner->IsA<FFloatProperty>());
		}
		else
		{
			check(FoundProperty->IsA<FFloatProperty>());
		}
	}
}

template <>
void FMovieSceneLiveLinkPropertyHandler<float>::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<float>>& ElementKeys = Keys[i];

		TArray<FFrameNumber> Times;
		Times.SetNum(ElementKeys.Num());
		TArray<FMovieSceneFloatValue> Values;
		Values.SetNum(ElementKeys.Num());
		int32 j = 0;
		for (const FLiveLinkPropertyKey<float>& Key : ElementKeys)
		{
			Times[j] = Key.Time;
			Values[j] = FMovieSceneFloatValue(Key.Value);
			j++;
		}

		PropertyStorage->FloatChannel[i].Set(Times, Values);
	}

	if (bInReduceKeys)
	{
		for (FMovieSceneFloatChannel& Channel : PropertyStorage->FloatChannel)
		{
			UE::MovieScene::Optimize(&Channel, InOptimizationParams);
		}
	}
	else
	{
		for (FMovieSceneFloatChannel& Channel : PropertyStorage->FloatChannel)
		{
			Channel.AutoSetTangents();
		}
	}
}

template<>
float FMovieSceneLiveLinkPropertyHandler<float>::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	return PropertyStorage->FloatChannel[InChannelIndex].GetValues()[InKeyIndex].Value;
}

template<>
float FMovieSceneLiveLinkPropertyHandler<float>::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	float Value;
	PropertyStorage->FloatChannel[InChannelIndex].Evaluate(InFrameTime, Value);
	return Value;
}

//------------------------------------------------------------------------------
// FMovieSceneLiveLinkPropertyHandler implementation - Int specialization.
//------------------------------------------------------------------------------

template <>
void FMovieSceneLiveLinkPropertyHandler<int32>::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);
	
	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		check(ArrayProperty->Inner->IsA<FIntProperty>());
	}
	else
	{
		check(FoundProperty->IsA<FIntProperty>());
	}

	PropertyStorage->IntegerChannel.SetNum(InElementCount);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}

template <>
void FMovieSceneLiveLinkPropertyHandler<int32>::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	ElementCount = PropertyStorage->IntegerChannel.Num();
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			check(ArrayProperty->Inner->IsA<FIntProperty>());
		}
		else
		{
			check(FoundProperty->IsA<FIntProperty>());
		}
	}
}

template <>
void FMovieSceneLiveLinkPropertyHandler<int32>::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<int32>>& ElementKeys = Keys[i];
		for (const FLiveLinkPropertyKey<int32>& Key : ElementKeys)
		{
			PropertyStorage->IntegerChannel[i].GetData().AddKey(Key.Time, Key.Value);
		}
	}

	if (bInReduceKeys)
	{
		// Reduce keys intentionally left blank
	}
}

template<>
int32 FMovieSceneLiveLinkPropertyHandler<int32>::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	return PropertyStorage->IntegerChannel[InChannelIndex].GetData().GetValues()[InKeyIndex];
}

template<>
int32 FMovieSceneLiveLinkPropertyHandler<int32>::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	int32 Value;
	PropertyStorage->IntegerChannel[InChannelIndex].Evaluate(InFrameTime, Value);
	return Value;
}

//------------------------------------------------------------------------------
// FMovieSceneLiveLinkPropertyHandler implementation - FString specialization.
//------------------------------------------------------------------------------

template <>
void FMovieSceneLiveLinkPropertyHandler<FString>::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		check(ArrayProperty->Inner->IsA<FStrProperty>());
	}
	else
	{
		check(FoundProperty->IsA<FStrProperty>());
	}

	PropertyStorage->StringChannel.SetNum(InElementCount);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}

template <>
void FMovieSceneLiveLinkPropertyHandler<FString>::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	ElementCount = PropertyStorage->StringChannel.Num();
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			check(ArrayProperty->Inner->IsA<FStrProperty>());
		}
		else
		{
			check(FoundProperty->IsA<FStrProperty>());
		}
	}
}

template <>
void FMovieSceneLiveLinkPropertyHandler<FString>::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<FString>>& ElementKeys = Keys[i];
		for (const FLiveLinkPropertyKey<FString>& Key : ElementKeys)
		{
			PropertyStorage->StringChannel[i].GetData().AddKey(Key.Time, Key.Value);
		}
	}

	if (bInReduceKeys)
	{
		// Reduce keys intentionally left blank
	}
}

template<>
FString FMovieSceneLiveLinkPropertyHandler<FString>::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	return PropertyStorage->StringChannel[InChannelIndex].GetData().GetValues()[InKeyIndex];
}

template<>
FString FMovieSceneLiveLinkPropertyHandler<FString>::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	const FString* Value = PropertyStorage->StringChannel[InChannelIndex].Evaluate(InFrameTime);
	return Value ? *Value : FString();
}

//------------------------------------------------------------------------------
// FMovieSceneLiveLinkPropertyHandler implementation - uint8 specialization.
//------------------------------------------------------------------------------

template <>
void FMovieSceneLiveLinkPropertyHandler<uint8>::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		check(ArrayProperty->Inner->IsA<FByteProperty>());
	}
	else
	{
		check(FoundProperty->IsA<FByteProperty>());
	}

	PropertyStorage->ByteChannel.SetNum(InElementCount);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}


template <>
void FMovieSceneLiveLinkPropertyHandler<uint8>::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	ElementCount = PropertyStorage->ByteChannel.Num();
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			check(ArrayProperty->Inner->IsA<FByteProperty>());
		}
		else
		{
			check(FoundProperty->IsA<FByteProperty>());
		}
	}
}

template <>
void FMovieSceneLiveLinkPropertyHandler<uint8>::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<uint8>>& ElementKeys = Keys[i];
		for (const FLiveLinkPropertyKey<uint8>& Key : ElementKeys)
		{
			PropertyStorage->ByteChannel[i].GetData().AddKey(Key.Time, Key.Value);
		}
	}

	if (bInReduceKeys)
	{
		// Reduce keys intentionally left blank
	}
}

template<>
uint8 FMovieSceneLiveLinkPropertyHandler<uint8>::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	return PropertyStorage->ByteChannel[InChannelIndex].GetData().GetValues()[InKeyIndex];
}

template<>
uint8 FMovieSceneLiveLinkPropertyHandler<uint8>::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	uint8 Value;
	PropertyStorage->ByteChannel[InChannelIndex].Evaluate(InFrameTime, Value);
	return Value;
}

//------------------------------------------------------------------------------
// FMovieSceneLiveLinkPropertyHandler implementation - bool specialization.
//------------------------------------------------------------------------------

template <>
void FMovieSceneLiveLinkPropertyHandler<bool>::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		check(ArrayProperty->Inner->IsA<FBoolProperty>());
	}
	else
	{
		check(FoundProperty->IsA<FBoolProperty>());
	}

	PropertyStorage->BoolChannel.SetNum(InElementCount);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}

template <>
void FMovieSceneLiveLinkPropertyHandler<bool>::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	ElementCount = PropertyStorage->BoolChannel.Num();
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			check(ArrayProperty->Inner->IsA<FBoolProperty>());
		}
		else
		{
			check(FoundProperty->IsA<FBoolProperty>());
		}
	}
}

template <>
void FMovieSceneLiveLinkPropertyHandler<bool>::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<bool>>& ElementKeys = Keys[i];
		for (const FLiveLinkPropertyKey<bool>& Key : ElementKeys)
		{
			PropertyStorage->BoolChannel[i].GetData().AddKey(Key.Time, Key.Value);
		}
	}

	if (bInReduceKeys)
	{
		// Reduce keys intentionally left blank
	}
}

template<>
bool FMovieSceneLiveLinkPropertyHandler<bool>::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	return PropertyStorage->BoolChannel[InChannelIndex].GetData().GetValues()[InKeyIndex];
}

template<>
bool FMovieSceneLiveLinkPropertyHandler<bool>::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	bool Value;
	PropertyStorage->BoolChannel[InChannelIndex].Evaluate(InFrameTime, Value);
	return Value;
}

//------------------------------------------------------------------------------
// FMovieSceneLiveLinkPropertyHandler implementation - FVector specialization.
//------------------------------------------------------------------------------

template <>
void FMovieSceneLiveLinkPropertyHandler<FVector>::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		FStructProperty* ArrayInnerProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
		check(ArrayInnerProperty->Struct->GetFName() == NAME_Vector);
	}
	else
	{
		FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(FoundProperty);
		check(StructProperty->Struct->GetFName() == NAME_Vector);
	}

	PropertyStorage->FloatChannel.SetNum(InElementCount * 3);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}


template <>
void FMovieSceneLiveLinkPropertyHandler<FVector>::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	check((PropertyStorage->FloatChannel.Num() % 3) == 0);

	ElementCount = PropertyStorage->FloatChannel.Num() / 3;
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			FStructProperty* ArrayStructProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
			check(ArrayStructProperty->Struct->GetFName() == NAME_Vector);

		}
		else
		{
			FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(FoundProperty);
			check(StructProperty->Struct->GetFName() == NAME_Vector);
		}
	}
}

template <>
void FMovieSceneLiveLinkPropertyHandler<FVector>::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<FVector>>& ElementKeys = Keys[i];

		TArray<FFrameNumber> Times;
		Times.SetNum(ElementKeys.Num());
		TArray<FMovieSceneFloatValue> ValuesX, ValuesY, ValuesZ;
		ValuesX.SetNum(ElementKeys.Num());
		ValuesY.SetNum(ElementKeys.Num());
		ValuesZ.SetNum(ElementKeys.Num());
		int32 j = 0;
		for (const FLiveLinkPropertyKey<FVector>& Key : ElementKeys)
		{
			Times[j] = Key.Time;
			ValuesX[j] = FMovieSceneFloatValue(Key.Value.X);
			ValuesY[j] = FMovieSceneFloatValue(Key.Value.Y);
			ValuesZ[j] = FMovieSceneFloatValue(Key.Value.Z);
			j++;
		}

		PropertyStorage->FloatChannel[(i * 3) + 0].Set(Times, ValuesX);
		PropertyStorage->FloatChannel[(i * 3) + 1].Set(Times, ValuesY);
		PropertyStorage->FloatChannel[(i * 3) + 2].Set(Times, ValuesZ);
	}

	if (bInReduceKeys)
	{
		for (FMovieSceneFloatChannel& Channel : PropertyStorage->FloatChannel)
		{
			UE::MovieScene::Optimize(&Channel, InOptimizationParams);
		}
	}
	else
	{
		for (FMovieSceneFloatChannel& Channel : PropertyStorage->FloatChannel)
		{
			Channel.AutoSetTangents();
		}
	}
}

template<>
FVector FMovieSceneLiveLinkPropertyHandler<FVector>::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	FVector Value;
	int32 OffsetChannelIndex = InChannelIndex * 3;
	LiveLinkPropertyHandlerUtils::FillVector(InKeyIndex, OffsetChannelIndex, PropertyStorage->FloatChannel, Value);
	return Value;
}

template<>
FVector FMovieSceneLiveLinkPropertyHandler<FVector>::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	FVector Value;
	int32 OffsetChannelIndex = InChannelIndex * 3;
	LiveLinkPropertyHandlerUtils::FillVectorInterpolated(InFrameTime, OffsetChannelIndex, PropertyStorage->FloatChannel, Value);
	return Value;
}

//------------------------------------------------------------------------------
// FMovieSceneLiveLinkPropertyHandler implementation - FColor specialization.
//------------------------------------------------------------------------------

template <>
void FMovieSceneLiveLinkPropertyHandler<FColor>::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
	{
		FStructProperty* ArrayInnerProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
		check(ArrayInnerProperty->Struct->GetFName() == NAME_Color);
	}
	else
	{
		FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(FoundProperty);
		check(StructProperty->Struct->GetFName() == NAME_Color);
	}

	PropertyStorage->ByteChannel.SetNum(InElementCount * 4);
	ElementCount = InElementCount;
	Keys.SetNum(ElementCount);
}

template <>
void FMovieSceneLiveLinkPropertyHandler<FColor>::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	check((PropertyStorage->ByteChannel.Num() % 4) == 0);

	ElementCount = PropertyStorage->ByteChannel.Num() / 3;
	check(ElementCount > 0);
	
	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			FStructProperty* ArrayStructProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
			check(ArrayStructProperty->Struct->GetFName() == NAME_Color);

		}
		else
		{
			FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(FoundProperty);
			check(StructProperty->Struct->GetFName() == NAME_Color);
		}
	}
}

template <>
void FMovieSceneLiveLinkPropertyHandler<FColor>::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		const TArray<FLiveLinkPropertyKey<FColor>>& ElementKeys = Keys[i];
		for (const FLiveLinkPropertyKey<FColor>& Key : ElementKeys)
		{
			PropertyStorage->ByteChannel[(i * 4) + 0].GetData().AddKey(Key.Time, Key.Value.R);
			PropertyStorage->ByteChannel[(i * 4) + 1].GetData().AddKey(Key.Time, Key.Value.G);
			PropertyStorage->ByteChannel[(i * 4) + 2].GetData().AddKey(Key.Time, Key.Value.B);
			PropertyStorage->ByteChannel[(i * 4) + 3].GetData().AddKey(Key.Time, Key.Value.A);
		}
	}

	if (bInReduceKeys)
	{
		for (FMovieSceneByteChannel& Channel : PropertyStorage->ByteChannel)
		{
			UE::MovieScene::Optimize(&Channel, InOptimizationParams);
		}
	}
}

template<>
FColor FMovieSceneLiveLinkPropertyHandler<FColor>::GetChannelValue(int32 InKeyIndex, int32 InChannelIndex)
{
	FColor Value;
	int32 OffsetChannelIndex = InChannelIndex * 4;
	LiveLinkPropertyHandlerUtils::FillColor(InKeyIndex, OffsetChannelIndex, PropertyStorage->ByteChannel, Value);
	return Value;
}

template<>
FColor FMovieSceneLiveLinkPropertyHandler<FColor>::GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex)
{
	FColor Value;
	int32 OffsetChannelIndex = InChannelIndex * 4;
	LiveLinkPropertyHandlerUtils::FillColorInterpolated(InFrameTime, OffsetChannelIndex, PropertyStorage->ByteChannel, Value);
	return Value;
}
