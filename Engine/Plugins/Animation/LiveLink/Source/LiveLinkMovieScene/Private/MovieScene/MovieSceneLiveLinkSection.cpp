// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkSection.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "LiveLinkCustomVersion.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieScene/MovieSceneLiveLinkSectionTemplate.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "MovieScene/MovieSceneLiveLinkSubSection.h"
#include "Channels/MovieSceneStringChannel.h"
#include "MovieScene/MovieSceneLiveLinkSubSectionAnimation.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"
#include "MovieScene/MovieSceneLiveLinkSubSectionBasicRole.h"
#include "MovieScene/MovieSceneLiveLinkSubSectionProperties.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h" //When loading old data
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkSection)

namespace LiveLinkSectionUtils
{
	void SetupDefaultValues(UMovieSceneLiveLinkSubSection* SubSection)
	{
		//Go through all possible channels of supported types and setup default value based on the first one
		for (FLiveLinkPropertyData& Data : SubSection->SubSectionData.Properties)
		{
			for (FMovieSceneFloatChannel& Channel : Data.FloatChannel)
			{
				if (Channel.GetNumKeys() > 0 && !Channel.GetDefault().IsSet())
				{
					Channel.SetDefault(Channel.GetValues()[0].Value);
				}
			}

			for (FMovieSceneStringChannel& Channel : Data.StringChannel)
			{
				if (Channel.GetNumKeys() > 0 && !Channel.GetDefault().IsSet())
				{
					TMovieSceneChannelData<const FString> StringData = Channel.GetData();
					Channel.SetDefault(StringData.GetValues()[0]);
				}
			}

			for (FMovieSceneIntegerChannel& Channel : Data.IntegerChannel)
			{
				if (Channel.GetNumKeys() > 0 && !Channel.GetDefault().IsSet())
				{
					Channel.SetDefault(Channel.GetValues()[0]);
				}
			}

			for (FMovieSceneBoolChannel& Channel : Data.BoolChannel)
			{
				if (Channel.GetNumKeys() > 0 && !Channel.GetDefault().IsSet())
				{
					Channel.SetDefault(Channel.GetValues()[0]);
				}
			}

			for (FMovieSceneByteChannel& Channel : Data.ByteChannel)
			{
				if (Channel.GetNumKeys() > 0 && !Channel.GetDefault().IsSet())
				{
					Channel.SetDefault(Channel.GetValues()[0]);
				}
			}
		}
	}
}



PRAGMA_DISABLE_DEPRECATION_WARNINGS
UMovieSceneLiveLinkSection::UMovieSceneLiveLinkSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BlendType = EMovieSceneBlendType::Absolute;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UMovieSceneLiveLinkSection::SetMask(const TArray<bool>& InChannelMask)
{
	ChannelMask = InChannelMask;

	UpdateChannelProxy();
}

void UMovieSceneLiveLinkSection::RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData)
{
	ExpandToFrame(InFrameNumber);

	for (UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		SubSection->RecordFrame(InFrameNumber, InFrameData);
	}
}

void UMovieSceneLiveLinkSection::FinalizeSection(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		SubSection->FinalizeSection(bInReduceKeys, InOptimizationParams);
		LiveLinkSectionUtils::SetupDefaultValues(SubSection);
	}
}

int32 UMovieSceneLiveLinkSection::CreateChannelProxy()
{
	int ChannelIndex = 0;
	FMovieSceneChannelProxyData Channels;

	for (UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		ChannelIndex += SubSection->CreateChannelProxy(ChannelIndex, ChannelMask, Channels);
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));

	return ChannelIndex;
}

void UMovieSceneLiveLinkSection::UpdateChannelProxy()
{
	//This is called when loading or when updating mask. We must give back each subsections its static data since it's only serialized in the section.
	//Channels must be linked to proxy again too
	int ChannelIndex = 0;
	FMovieSceneChannelProxyData Channels;
	for (UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		SubSection->SetStaticData(StaticData);
		ChannelIndex += SubSection->CreateChannelProxy(ChannelIndex, ChannelMask, Channels);
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

void UMovieSceneLiveLinkSection::Initialize(const FLiveLinkSubjectPreset& InSubjectPreset, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData)
{
	SubjectPreset = InSubjectPreset;
	StaticData = InStaticData;

	const TArray<TSubclassOf<UMovieSceneLiveLinkSubSection>> FoundSubSections = UMovieSceneLiveLinkSubSection::GetLiveLinkSubSectionForRole(SubjectPreset.Role);
	for (const TSubclassOf<UMovieSceneLiveLinkSubSection>& SubSection : FoundSubSections)
	{
		if (SubSection.GetDefaultObject()->IsRoleSupported(SubjectPreset.Role))
		{
			UMovieSceneLiveLinkSubSection* NewSubSection = NewObject<UMovieSceneLiveLinkSubSection>(this, SubSection.Get(), NAME_None, RF_Transactional);
			NewSubSection->Initialize(SubjectPreset.Role, StaticData);
			SubSections.Add(NewSubSection);
		}
	}
	
	//Initialize the mask with the number of channels with all of them enabled by default
	ChannelMask.Init(true, GetChannelCount());
}

int32 UMovieSceneLiveLinkSection::GetChannelCount() const
{
	int32 ChannelCount = 0;
	for (const UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		ChannelCount += SubSection->GetChannelCount();
	}

	return ChannelCount;
}

void UMovieSceneLiveLinkSection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FLiveLinkCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		StaticData = MakeShared<FLiveLinkStaticDataStruct>();

		if (Ar.CustomVer(FLiveLinkCustomVersion::GUID) < FLiveLinkCustomVersion::NewLiveLinkRoleSystem)
		{
			ConvertPreRoleData();
		}
		else
		{
			bool bValidStaticData = false;
			Ar << bValidStaticData;
			if (bValidStaticData)
			{
				Ar << *StaticData;
			}
		}
	}
	else if(Ar.IsSaving())
	{
		bool bValidStaticData = StaticData.IsValid();
		Ar << bValidStaticData;
		if (bValidStaticData)
		{
			Ar << *StaticData;
		}
	}
}

void UMovieSceneLiveLinkSection::PostEditImport()
{
	Super::PostEditImport();

	UpdateChannelProxy();
}

void UMovieSceneLiveLinkSection::PostLoad()
{
	Super::PostLoad();

	for (UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		SubSection->ConditionalPostLoad();
	}

	UpdateChannelProxy();

	//Fixup default values
	//Note: Once out of point update, add verification of custom object
	//To avoid doing it all the time. For now, this fixups previously recorded
	//LiveLink tracks. New ones will have default values.
	for (UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		LiveLinkSectionUtils::SetupDefaultValues(SubSection);
	}
}

#if WITH_EDITOR
bool UMovieSceneLiveLinkSection::Modify(bool bAlwaysMarkDirty /*= true*/)
{
	bool bWasModified = Super::Modify(bAlwaysMarkDirty);

	for (UMovieSceneLiveLinkSubSection* SubSection : SubSections)
	{
		bWasModified |= SubSection->Modify(bAlwaysMarkDirty);
	}

	return bWasModified;
}

void UMovieSceneLiveLinkSection::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);	

	//Make sure the channel proxy is always in sync with subsections data. They are recreated whenever we undo / redo.
	UpdateChannelProxy();
}

void UMovieSceneLiveLinkSection::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	Super::ExportCustomProperties(Out, Indent);

	if (StaticData)
	{
		FString Text;
		StaticData->GetStruct()->ExportText(Text, StaticData->GetBaseData(), nullptr, nullptr, PPF_None, nullptr);
		Out.Logf(TEXT("%sCustomProperties StaticDataStruct Path=%s StaticData=%s\r\n"), FCString::Spc(Indent), *StaticData->GetStruct()->GetPathName(), *Text);
	}
}

void UMovieSceneLiveLinkSection::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	if (FParse::Command(&SourceText, TEXT("StaticDataStruct")))
	{
		FString StructPath;
		if (!FParse::Value(SourceText, TEXT("Path="), StructPath))
		{
			return;
		}

		FString StructValue;
		if (FParse::Value(SourceText, TEXT("StaticData="), StructValue, false))
		{
			if (UScriptStruct* Struct = FindObject<UScriptStruct>(nullptr, *StructPath))
			{
				TSharedPtr<FLiveLinkStaticDataStruct> StaticStruct = MakeShared<FLiveLinkStaticDataStruct>(Struct);

				Struct->ImportText(*StructValue, StaticStruct->GetBaseData(), this, PPF_None, nullptr, Struct->GetName());
				StaticData = MoveTemp(StaticStruct);
			}
		}
	}
	else
	{
		Super::ImportCustomProperties(SourceText, Warn);
	}
}
#endif

TArray<TSubclassOf<UMovieSceneLiveLinkSection>> UMovieSceneLiveLinkSection::GetMovieSectionForRole(const TSubclassOf<ULiveLinkRole>& InRoleToSupport)
{
	TArray<TSubclassOf<UMovieSceneLiveLinkSection>> Results;
	for (TObjectIterator<UClass> Itt; Itt; ++Itt)
	{
		if (Itt->IsChildOf(UMovieSceneLiveLinkSection::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			Results.Add(*Itt);
		}
	}
	return MoveTemp(Results);
}

FMovieSceneEvalTemplatePtr UMovieSceneLiveLinkSection::CreateSectionTemplate(const UMovieScenePropertyTrack& InTrack) const
{
	return FMovieSceneLiveLinkSectionTemplate(*this, InTrack);
}

void UMovieSceneLiveLinkSection::ConvertPreRoleData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SubjectPreset.Key.Source.Invalidate();
	SubjectPreset.Key.SubjectName = SubjectName_DEPRECATED;

	//Verify if there are any bones for this subject. No bones means only curves were present so create a basic role in this case.
	SubjectPreset.Role = ULiveLinkAnimationRole::StaticClass();
	StaticData->InitializeWith(FLiveLinkSkeletonStaticData::StaticStruct(), nullptr);
	
	//Take old skeleton data for new static data structure

	FLiveLinkSkeletonStaticData* SkeletonData = StaticData->Cast<FLiveLinkSkeletonStaticData>();
	SkeletonData->BoneNames = MoveTemp(RefSkeleton_DEPRECATED.BoneNames);
	SkeletonData->BoneParents = MoveTemp(RefSkeleton_DEPRECATED.BoneParents);

	//Create subsection that will manage this data
	UMovieSceneLiveLinkSubSectionAnimation* AnimationSubSection = NewObject<UMovieSceneLiveLinkSubSectionAnimation>(this, UMovieSceneLiveLinkSubSectionAnimation::StaticClass(), NAME_None, RF_Transactional);
	AnimationSubSection->Initialize(SubjectPreset.Role, StaticData);

	//Convert transforms to float data channels
	const int32 TransformCount = SkeletonData->BoneNames.Num() * 9;
	if (TransformCount > 0)
	{
		ensure(AnimationSubSection->SubSectionData.Properties[0].FloatChannel.Num() == TransformCount);
		AnimationSubSection->SubSectionData.Properties[0].FloatChannel = TArray<FMovieSceneFloatChannel>(PropertyFloatChannels_DEPRECATED.GetData(), TransformCount);
		PropertyFloatChannels_DEPRECATED.RemoveAtSwap(0, TransformCount);
		SubSections.Add(AnimationSubSection);
	}

	// If there were curves, add a basic section handler
	if (CurveNames_DEPRECATED.Num() > 0)
	{
		FLiveLinkBaseStaticData* BaseStaticData = StaticData->Cast<FLiveLinkBaseStaticData>();
		BaseStaticData->PropertyNames = MoveTemp(CurveNames_DEPRECATED);
		
		//Basic role sub section to handle properties
		UMovieSceneLiveLinkSubSectionBasicRole* BasicRoleSubSection = NewObject<UMovieSceneLiveLinkSubSectionBasicRole>(this, UMovieSceneLiveLinkSubSectionBasicRole::StaticClass(), NAME_None, RF_Transactional);
		BasicRoleSubSection->Initialize(SubjectPreset.Role, StaticData);
		ensure(BasicRoleSubSection->SubSectionData.Properties[0].FloatChannel.Num() == PropertyFloatChannels_DEPRECATED.Num());
		BasicRoleSubSection->SubSectionData.Properties[0].FloatChannel = MoveTemp(PropertyFloatChannels_DEPRECATED);
		SubSections.Add(BasicRoleSubSection);
	}

	//Generic properties sub section because all sections have it
	UMovieSceneLiveLinkSubSectionProperties* PropertiesSubSection = NewObject<UMovieSceneLiveLinkSubSectionProperties>(this, UMovieSceneLiveLinkSubSectionProperties::StaticClass(), NAME_None, RF_Transactional);
	PropertiesSubSection->Initialize(SubjectPreset.Role, StaticData);
	SubSections.Add(PropertiesSubSection);

	//Reinitialize channel mask to account for new possible data
	const int32 ChannelCount = GetChannelCount();
	ChannelMask.SetNum(ChannelCount);
	ChannelMask.Init(true, ChannelCount);

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
