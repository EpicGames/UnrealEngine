// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkSectionTemplate.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "LiveLinkCustomVersion.h"
#include "Channels/MovieSceneByteChannel.h"
#include "LiveLinkMovieScenePrivate.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneLiveLinkSource.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "MovieScene/MovieSceneLiveLinkSection.h"
#include "Channels/MovieSceneStringChannel.h"
#include "MovieScene/MovieSceneLiveLinkSubSection.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "LiveLinkRole.h"
#include "MovieScene/IMovieSceneLiveLinkPropertyHandler.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkSectionTemplate)

static TAutoConsoleVariable<int32> CVarSequencerAlwaysSendInterpolatedLiveLink(
	TEXT("Sequencer.AlwaysSendInterpolatedLiveLink"),
	0,
	TEXT("If nonzero we always interpolate when sending out live link data, if 0 we may send out frames at a higher rate than engine tick, if the data is dense."),
	ECVF_Default);


//Converts time's in our movie scene frame rate to times in the time code frame rate, based upon where our frame time is and where the timecode frame time is.
static FQualifiedFrameTime ConvertFrameTimeToTimeCodeTime(const FFrameTime& FrameTime, const FFrameRate& FrameRate, const FFrameTime& FrameTimeEqualToTimecodeFrameTime, const FQualifiedFrameTime& TimecodeTime)
{
	FFrameTime DiffFrameTime = FrameTime - FrameTimeEqualToTimecodeFrameTime;
	DiffFrameTime = FFrameRate::TransformTime(DiffFrameTime, FrameRate, TimecodeTime.Rate);
	return FQualifiedFrameTime(TimecodeTime.Time + DiffFrameTime, TimecodeTime.Rate);
}

static FLiveLinkWorldTime ConvertFrameTimeToLiveLinkWorldTime(const FFrameTime& FrameTime, const FFrameRate& FrameRate, const FFrameTime& FrameTimeEqualToWorldFrameTime, const FLiveLinkWorldTime& LiveLinkWorldTime)
{
	FFrameTime DiffFrameTime = FrameTime - FrameTimeEqualToWorldFrameTime;
	double DiffSeconds = FrameRate.AsSeconds(DiffFrameTime);
	return FLiveLinkWorldTime(DiffSeconds + LiveLinkWorldTime.GetOffsettedTime(), 0.0);
}

struct FMovieSceneLiveLinkSectionTemplatePersistentData : IPersistentEvaluationData
{
	TSharedPtr<FMovieSceneLiveLinkSource> LiveLinkSource;
};

namespace LiveLinkSectionTemplateUtils
{
	//Initial LiveLink Track Recorder was not writing out default values in each recorded channel
	//When there is no Keys (either none recorded or everything erased), channel must have a default value to be used
	//Or we could end up with garbage causing NaN behavior down the road.
	template<class ChannelType>
	bool AreChannelsUsable(const TArray<ChannelType>& Channels)
	{
		//If no channels, consider this valid. It will never be used to build the frame data
		if (Channels.Num() <= 0)
		{
			return true;
		}

		for (const ChannelType& Channel : Channels)
		{
			if (Channel.GetTimes().Num() > 0 || Channel.GetDefault().IsSet())
			{
				return true;
			}
		}
	
		return false;
	}
}


FMovieSceneLiveLinkSectionTemplate::FMovieSceneLiveLinkSectionTemplate()
{
	//If we want to use direct frames, all channels must have the same amount of keys.
	bMustDoInterpolation = AreChannelKeyCountEqual() == false;
	bIsSectionUsable = CacheIsSectionUsable();
}

FMovieSceneLiveLinkSectionTemplate::FMovieSceneLiveLinkSectionTemplate(const UMovieSceneLiveLinkSection& Section, const UMovieScenePropertyTrack& Track)
	: FMovieScenePropertySectionTemplate(Track.GetPropertyName(), Track.GetPropertyPath().ToString())
	, SubjectPreset(Section.SubjectPreset)
	, ChannelMask(Section.ChannelMask)
	, StaticData(Section.StaticData)
{
	for (const UMovieSceneLiveLinkSubSection* SubSection : Section.SubSections)
	{
		SubSectionsData.Add(SubSection->SubSectionData);
	}

	//If we want to use direct frames, all channels must have the same amount of keys.
	bMustDoInterpolation = AreChannelKeyCountEqual() == false;
	
	//Cache whether or not this section is usable. No keys AND no default values would cause this.
	bIsSectionUsable = CacheIsSectionUsable();
	if (SubjectPreset.Key.SubjectName.Name != NAME_None && !bIsSectionUsable)
	{
		UE_LOGF(LogLiveLinkMovieScene, Verbose, "Subject '%ls' LiveLinkSection isn't usable. No samples were recorded.", *SubjectPreset.Key.SubjectName.ToString());
	}

	InitializePropertyHandlers();
}

FMovieSceneLiveLinkSectionTemplate::FMovieSceneLiveLinkSectionTemplate(const FMovieSceneLiveLinkSectionTemplate& InOther)
	: Super(InOther)
	, SubjectPreset(InOther.SubjectPreset)
	, ChannelMask(InOther.ChannelMask)
	, SubSectionsData(InOther.SubSectionsData)
	, bMustDoInterpolation(InOther.bMustDoInterpolation)
	, bIsSectionUsable(InOther.bIsSectionUsable)
	, StaticData(InOther.StaticData)
{
	InitializePropertyHandlers();
}

bool FMovieSceneLiveLinkSectionTemplate::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FLiveLinkCustomVersion::GUID);

	UScriptStruct& Struct = GetScriptStructImpl();

	// Serialize normal tagged data
	if (!Ar.IsCountingMemory())
	{
		Struct.SerializeTaggedProperties(Ar, (uint8*)this, &Struct, nullptr);
	}

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FLiveLinkCustomVersion::GUID) >= FLiveLinkCustomVersion::NewLiveLinkRoleSystem)
		{
			StaticData = MakeShared<FLiveLinkStaticDataStruct>();

			bool bValidStaticData = false;
			Ar << bValidStaticData;
			if (bValidStaticData)
			{
				Ar << *StaticData;
			}

			InitializePropertyHandlers();
		}
	}
	else if (Ar.IsSaving())
	{
		bool bValidStaticData = StaticData.IsValid();
		Ar << bValidStaticData;
		if (bValidStaticData)
		{
			Ar << *StaticData;
		}
	}

	//We have handled serialization, return true to let serializer know
	return true;
}

bool FMovieSceneLiveLinkSectionTemplate::GetLiveLinkFrameArray(const FFrameTime& FrameTime, const FFrameTime& LowerBound, const FFrameTime& UpperBound, TArray<FLiveLinkFrameDataStruct>&  LiveLinkFrameDataArray, const FFrameRate& FrameRate) const
{
	//See if we have a valid time code time. 
	//If so we may can possible send raw data if not asked to only send interpolated.
	TOptional<FQualifiedFrameTime> TimeCodeFrameTime = FApp::GetCurrentFrameTime();

	//Send interpolated if told to or no valid timecode synced.
	const bool bAlwaysSendInterpolated = CVarSequencerAlwaysSendInterpolatedLiveLink->GetInt() == 0 ? false : true;

	bool bSendInterpolated = bAlwaysSendInterpolated || !TimeCodeFrameTime.IsSet() || LowerBound == UpperBound || bMustDoInterpolation;
	FLiveLinkWorldTime WorldTime = FLiveLinkWorldTime(); //this calls FPlatform::Seconds()

	if (!bSendInterpolated)
	{
		FFrameTime FrameRangeEnd = LowerBound > UpperBound ? LowerBound : UpperBound;
		FFrameTime FrameRangeStart = LowerBound > UpperBound ? UpperBound : LowerBound;
		{
			TArrayView<const FFrameNumber> Times;
			GetFirstTimeArray(Times);

			int32 EndIndex = INDEX_NONE, StartIndex = INDEX_NONE;
			EndIndex = Algo::LowerBound(Times, FrameRangeEnd.FrameNumber);

			FFrameNumber Frame;
			if (EndIndex != INDEX_NONE)
			{
				if (EndIndex >= Times.Num())
				{
					EndIndex = Times.Num() - 1;
				}

				StartIndex = Algo::UpperBound(Times, FrameRangeStart.FrameNumber);
				if (StartIndex == INDEX_NONE)
				{
					StartIndex = EndIndex;
				}

			}
			else
			{
				StartIndex = Algo::UpperBound(Times, FrameRangeStart.FrameNumber);
				if (StartIndex >= Times.Num())
				{
					StartIndex = Times.Num() - 1;
				}
				if (StartIndex != INDEX_NONE)
				{
					EndIndex = StartIndex;
				}
			}
			bSendInterpolated = true; //if we don't send at least one key send interpolated
			if (EndIndex != INDEX_NONE)
			{
				UE_LOGF(LogLiveLinkMovieScene, Verbose, "Send Key LiveLink Start/End Index '%d'  '%d'", StartIndex,EndIndex);
				for (int32 Index = StartIndex; Index <= EndIndex; ++Index)
				{
					Frame = Times[Index];
					if (Frame > FrameRangeStart && Frame <= FrameRangeEnd) // doing (begin,end] want to make sure we get the last frame always, future better than past.
					{
						UE_LOGF(LogLiveLinkMovieScene, Verbose, "Send Key LiveLink Key Index '%d'", Index);
						bSendInterpolated = false;
						const FLiveLinkWorldTime LiveLinkWorldTime = ConvertFrameTimeToLiveLinkWorldTime(Times[Index], FrameRate, FrameTime, WorldTime);

						TOptional<FQualifiedFrameTime> TimecodeTime;
						if (TimeCodeFrameTime.IsSet())
						{
							TimecodeTime = ConvertFrameTimeToTimeCodeTime(Times[Index], FrameRate, FrameTime, TimeCodeFrameTime.GetValue());
						}

						FLiveLinkFrameDataStruct NewFrameStruct(SubjectPreset.Role.GetDefaultObject()->GetFrameDataStruct());
						FillFrame(Index, LiveLinkWorldTime, TimecodeTime, NewFrameStruct);

						LiveLinkFrameDataArray.Add(MoveTemp(NewFrameStruct));
					}
				}
			}
		}
	}
	if (bSendInterpolated)
	{
		//send both engine time and if we have a synchronized timecode provider the qualified time also
		FLiveLinkFrameDataStruct NewFrameStruct(SubjectPreset.Role.GetDefaultObject()->GetFrameDataStruct());
		FillFrameInterpolated(FrameTime, WorldTime, TimeCodeFrameTime, NewFrameStruct);

		LiveLinkFrameDataArray.Add(MoveTemp(NewFrameStruct));
	}
	return true;
}

void FMovieSceneLiveLinkSectionTemplate::FillFrame(int32 InKeyIndex, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, FLiveLinkFrameDataStruct& OutFrame) const
{
	FLiveLinkBaseFrameData* ContainerData = OutFrame.GetBaseData();

	if (InTimecodeTime.IsSet())
	{
		ContainerData->MetaData.SceneTime = InTimecodeTime.GetValue();
	}

	ContainerData->WorldTime = InWorldTime;

	const UScriptStruct* Container = SubjectPreset.Role.GetDefaultObject()->GetFrameDataStruct();
	for (TSharedPtr<IMovieSceneLiveLinkPropertyHandler> PropertyHandler : PropertyHandlers)
	{
		PropertyHandler->FillFrame(InKeyIndex, InWorldTime, InTimecodeTime, *Container, OutFrame.GetBaseData());
	}
}

void FMovieSceneLiveLinkSectionTemplate::FillFrameInterpolated(const FFrameTime& InFrameTime, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, FLiveLinkFrameDataStruct& OutFrame) const
{
	FLiveLinkBaseFrameData* ContainerData = OutFrame.GetBaseData();

	if (InTimecodeTime.IsSet())
	{
		ContainerData->MetaData.SceneTime = InTimecodeTime.GetValue();
	}

	ContainerData->WorldTime = InWorldTime;

	const UScriptStruct* Container = SubjectPreset.Role.GetDefaultObject()->GetFrameDataStruct();
	for (TSharedPtr<IMovieSceneLiveLinkPropertyHandler> PropertyHandler : PropertyHandlers)
	{
		PropertyHandler->FillFrameInterpolated(InFrameTime, InWorldTime, InTimecodeTime, *Container, OutFrame.GetBaseData());
	}
}

void FMovieSceneLiveLinkSectionTemplate::EvaluateSwept(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const TRange<FFrameNumber>& SweptRange, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	FMovieSceneLiveLinkSectionTemplatePersistentData* Data = PersistentData.FindSectionData<FMovieSceneLiveLinkSectionTemplatePersistentData>();
	if (bIsSectionUsable && Data && Data->LiveLinkSource.IsValid() && Data->LiveLinkSource->IsSourceStillValid() && SubjectPreset.Role)
	{
		TArray<FLiveLinkFrameDataStruct>  LiveLinkFrameDataArray;
		GetLiveLinkFrameArray(Context.GetTime(), SweptRange.GetLowerBoundValue(), SweptRange.GetUpperBoundValue(), LiveLinkFrameDataArray, Context.GetFrameRate());

		Data->LiveLinkSource->PublishLiveLinkFrameData(LiveLinkFrameDataArray);
	}
}

void FMovieSceneLiveLinkSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	FMovieSceneLiveLinkSectionTemplatePersistentData* Data = PersistentData.FindSectionData<FMovieSceneLiveLinkSectionTemplatePersistentData>();
	if (bIsSectionUsable && Data && Data->LiveLinkSource.IsValid() && Data->LiveLinkSource->IsSourceStillValid() && SubjectPreset.Role)
	{
		TArray<FLiveLinkFrameDataStruct>  LiveLinkFrameDataArray;
		FFrameTime FrameTime = Context.GetTime();
		GetLiveLinkFrameArray(FrameTime, FrameTime, FrameTime, LiveLinkFrameDataArray, Context.GetFrameRate());

		Data->LiveLinkSource->PublishLiveLinkFrameData(LiveLinkFrameDataArray);
	}
}

void FMovieSceneLiveLinkSectionTemplate::Setup(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	if (StaticData)
	{
		FMovieSceneLiveLinkSectionTemplatePersistentData& Data = PersistentData.GetOrAddSectionData<FMovieSceneLiveLinkSectionTemplatePersistentData>();
		Data.LiveLinkSource = FMovieSceneLiveLinkSource::CreateLiveLinkSource(SubjectPreset);
		Data.LiveLinkSource->PublishLiveLinkStaticData(*StaticData);
	}
}

void FMovieSceneLiveLinkSectionTemplate::TearDown(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	FMovieSceneLiveLinkSectionTemplatePersistentData* Data = PersistentData.FindSectionData<FMovieSceneLiveLinkSectionTemplatePersistentData>();

	if (Data && Data->LiveLinkSource.IsValid())
	{
		if (Data->LiveLinkSource->IsSourceStillValid())
		{
			FMovieSceneLiveLinkSource::RemoveLiveLinkSource(Data->LiveLinkSource);
		}
		Data->LiveLinkSource.Reset();
	}
}
void FMovieSceneLiveLinkSectionTemplate::InitializePropertyHandlers()
{
	if (SubjectPreset.Role)
	{
		const UScriptStruct* Container = SubjectPreset.Role.GetDefaultObject()->GetFrameDataStruct();
		for (FLiveLinkSubSectionData& SubSection : SubSectionsData)
		{
			for (FLiveLinkPropertyData& PropertyDataElement : SubSection.Properties)
			{
				TSharedPtr<IMovieSceneLiveLinkPropertyHandler> NewHandler = LiveLinkPropertiesUtils::CreatePropertyHandler(*Container, &PropertyDataElement);
				if (NewHandler.IsValid())
				{
					NewHandler->InitializeFromExistingChannels(*Container);
					PropertyHandlers.Add(NewHandler);
				}
			}
		}
	}
}

bool FMovieSceneLiveLinkSectionTemplate::AreChannelKeyCountEqual() const
{
	int32 KeyCount = -1;
	for (const FLiveLinkSubSectionData& SubSectionData : SubSectionsData)
	{
		for (const FLiveLinkPropertyData& SubSectionProperties : SubSectionData.Properties)
		{
			for (const FMovieSceneFloatChannel& Channel : SubSectionProperties.FloatChannel)
			{
				if (KeyCount == -1)
				{
					KeyCount = Channel.GetTimes().Num();
				}
				else if(Channel.GetTimes().Num() != KeyCount)
				{
					return false;
				}
			}

			for (const FMovieSceneBoolChannel& Channel : SubSectionProperties.BoolChannel)
			{
				if (KeyCount == -1)
				{
					KeyCount = Channel.GetTimes().Num();
				}
				else if (Channel.GetTimes().Num() != KeyCount)
				{
					return false;
				}
			}

			for (const FMovieSceneIntegerChannel& Channel : SubSectionProperties.IntegerChannel)
			{
				if (KeyCount == -1)
				{
					KeyCount = Channel.GetTimes().Num();
				}
				else if (Channel.GetTimes().Num() != KeyCount)
				{
					return false;
				}
			}

			for (const FMovieSceneByteChannel& Channel : SubSectionProperties.ByteChannel)
			{
				if (KeyCount == -1)
				{
					KeyCount = Channel.GetTimes().Num();
				}
				else if (Channel.GetTimes().Num() != KeyCount)
				{
					return false;
				}
			}

			for (const FMovieSceneStringChannel& Channel : SubSectionProperties.StringChannel)
			{
				if (KeyCount == -1)
				{
					KeyCount = Channel.GetTimes().Num();
				}
				else if (Channel.GetTimes().Num() != KeyCount)
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool FMovieSceneLiveLinkSectionTemplate::CacheIsSectionUsable() const
{
	for (const FLiveLinkSubSectionData& SubSectionData : SubSectionsData)
	{
		for (const FLiveLinkPropertyData& SubSectionProperties : SubSectionData.Properties)
		{
			if (!LiveLinkSectionTemplateUtils::AreChannelsUsable(SubSectionProperties.FloatChannel))
			{
				return false;
			}

			if (!LiveLinkSectionTemplateUtils::AreChannelsUsable(SubSectionProperties.StringChannel))
			{
				return false;
			}

			if (!LiveLinkSectionTemplateUtils::AreChannelsUsable(SubSectionProperties.IntegerChannel))
			{
				return false;
			}

			if (!LiveLinkSectionTemplateUtils::AreChannelsUsable(SubSectionProperties.BoolChannel))
			{
				return false;
			}

			if (!LiveLinkSectionTemplateUtils::AreChannelsUsable(SubSectionProperties.ByteChannel))
			{
				return false;
			}
		}
	}

	return true;
}

void FMovieSceneLiveLinkSectionTemplate::GetFirstTimeArray(TArrayView<const FFrameNumber>& OutKeyTimes) const
{
	for (const FLiveLinkSubSectionData& SubSectionData : SubSectionsData)
	{
		for (const FLiveLinkPropertyData& SubSectionProperties : SubSectionData.Properties)
		{
			for (const FMovieSceneFloatChannel& Channel : SubSectionProperties.FloatChannel)
			{
				OutKeyTimes = Channel.GetTimes();
			}

			for (const FMovieSceneBoolChannel& Channel : SubSectionProperties.BoolChannel)
			{
				OutKeyTimes = Channel.GetTimes();
			}

			for (const FMovieSceneIntegerChannel& Channel : SubSectionProperties.IntegerChannel)
			{
				OutKeyTimes = Channel.GetTimes();
			}

			for (const FMovieSceneByteChannel& Channel : SubSectionProperties.ByteChannel)
			{
				OutKeyTimes = Channel.GetTimes();
			}

			for (const FMovieSceneStringChannel& Channel : SubSectionProperties.StringChannel)
			{
				OutKeyTimes = Channel.GetTimes();
			}
		}
	}
}


