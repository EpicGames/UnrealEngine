// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieScenePropertyTemplate.h"

#include "LiveLinkPresetTypes.h"

#include "MovieSceneLiveLinkSectionTemplate.generated.h"

class IMovieSceneLiveLinkPropertyHandler;
struct FLiveLinkPropertyData;
struct FLiveLinkSubSectionData;
struct FQualifiedFrameTime;

class UMovieSceneLiveLinkSection;
class UMovieScenePropertyTrack;
struct FMovieSceneLiveLinkSubSectionTemplateData;



/** A movie scene evaluation template for post move settings live link sections. */
USTRUCT()
struct FMovieSceneLiveLinkSectionTemplate : public FMovieScenePropertySectionTemplate
{
	GENERATED_BODY()

public:
	FMovieSceneLiveLinkSectionTemplate();
	FMovieSceneLiveLinkSectionTemplate(const UMovieSceneLiveLinkSection& Section, const UMovieScenePropertyTrack& Track);

	/** Template is copied after being created. We need to re-link property storage to property handler*/
	FMovieSceneLiveLinkSectionTemplate(const FMovieSceneLiveLinkSectionTemplate& InOther);

	virtual ~FMovieSceneLiveLinkSectionTemplate() = default;

public:
	/** Implementing Serialize to adjust old data and save static data */
	bool Serialize(FArchive& Ar);
	
protected:
	bool GetLiveLinkFrameArray(const FFrameTime &FrameTime, const FFrameTime& LowerBound, const FFrameTime& UpperBound, TArray<FLiveLinkFrameDataStruct>& LiveLinkFrameDataArray, const FFrameRate& FrameRate) const;
	void FillFrame(int32 InKeyIndex, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, FLiveLinkFrameDataStruct& OutFrame) const;
	void FillFrameInterpolated(const FFrameTime& InFrameTime, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, FLiveLinkFrameDataStruct& OutFrame) const;

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	virtual void EvaluateSwept(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const TRange<FFrameNumber>& SweptRange, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	virtual void SetupOverrides() override
	{
		EnableOverrides(RequiresSetupFlag | RequiresTearDownFlag );
	}
	virtual void Setup(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const override;
	virtual void TearDown(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const override;


	void FillPropertyArray(int32 InKeyIndex, const FLiveLinkPropertyData& InSourceData, FArrayProperty* InSourceArray, void* InDestinationAddress);
	void FillPropertyArrayInterpolated(const FFrameTime& InFrameTime, const FLiveLinkPropertyData& InSourceData, FArrayProperty* InSourceArray, void* InDestinationAddress);

	void InitializePropertyHandlers();

	bool AreChannelKeyCountEqual() const;
	bool CacheIsSectionUsable() const;
	void GetFirstTimeArray(TArrayView<const FFrameNumber>& OutKeyTimes) const;

public:
	
	UPROPERTY()
	FLiveLinkSubjectPreset SubjectPreset;

	UPROPERTY()
	TArray<bool> ChannelMask;

	UPROPERTY()
	TArray<FLiveLinkSubSectionData> SubSectionsData;

	bool bMustDoInterpolation;

	bool bIsSectionUsable;

	TArray<TSharedPtr<IMovieSceneLiveLinkPropertyHandler>> PropertyHandlers;

	TSharedPtr<FLiveLinkStaticDataStruct> StaticData;
};

template<> struct TStructOpsTypeTraits<FMovieSceneLiveLinkSectionTemplate> : public TStructOpsTypeTraitsBase2<FMovieSceneLiveLinkSectionTemplate>
{
	enum { WithSerializer = true };
};

