// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLiveLinkSource.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkSource.h"
#include "LiveLinkTypes.h"

FMovieSceneLiveLinkSource::FMovieSceneLiveLinkSource()
	: Client(nullptr)
{
}

TSharedPtr<FMovieSceneLiveLinkSource> FMovieSceneLiveLinkSource::CreateLiveLinkSource(const FLiveLinkSubjectPreset& InSubjectPreset)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		TSharedPtr<FMovieSceneLiveLinkSource> Source = MakeShared<FMovieSceneLiveLinkSource>();
		Source->SubjectPreset = InSubjectPreset;
		LiveLinkClient->AddSource(Source);
		return Source;
	}
	return TSharedPtr<FMovieSceneLiveLinkSource>();
}

void FMovieSceneLiveLinkSource::RemoveLiveLinkSource(TSharedPtr<FMovieSceneLiveLinkSource> InSource)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

		//If we're still enabled, put back the previous one with our name if it was enabled
		if (InSource->PreviousSubjectEnabled.Source.IsValid() && LiveLinkClient->IsSubjectEnabled(InSource->SubjectPreset.Key, false))
		{
			LiveLinkClient->SetSubjectEnabled(InSource->PreviousSubjectEnabled, true);
		}

		LiveLinkClient->RemoveSource(InSource);
	}
}

void FMovieSceneLiveLinkSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;

	//Update the preset with the source id we just received to link it with it and mark us as enabled by default.
	SubjectPreset.Key.Source = SourceGuid;
	SubjectPreset.bEnabled = true;

	//When our subject is registered, set it as enabled. Keep the current one to re-enable it once we're teared down
	const bool bIncludeDisabledSubjects = false;
	const bool bIncludeVirtualSubjects = false;
	TArray<FLiveLinkSubjectKey> EnabledSubjects = Client->GetSubjects(bIncludeDisabledSubjects, bIncludeVirtualSubjects);
	const FLiveLinkSubjectKey* FoundSubjectPtr = EnabledSubjects.FindByPredicate([this](const FLiveLinkSubjectKey& InOther) { return SubjectPreset.Key.SubjectName.Name == InOther.SubjectName.Name; });
	if (FoundSubjectPtr && FoundSubjectPtr->Source != SourceGuid)
	{
		PreviousSubjectEnabled = *FoundSubjectPtr;
	}
	else
	{
		PreviousSubjectEnabled.Source.Invalidate();
	}

	Client->CreateSubject(SubjectPreset);
}

void FMovieSceneLiveLinkSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	ILiveLinkSource::InitializeSettings(Settings);

	Settings->BufferSettings.MaxNumberOfFrameToBuffered = 1;
}

bool FMovieSceneLiveLinkSource::IsSourceStillValid() const
{
	return Client != nullptr;
}

bool FMovieSceneLiveLinkSource::RequestSourceShutdown()
{
	Client = nullptr;
	return true;
}

FText FMovieSceneLiveLinkSource::GetSourceMachineName() const
{
	return FText().FromString(FPlatformProcess::ComputerName());
}

FText FMovieSceneLiveLinkSource::GetSourceStatus() const
{
	return NSLOCTEXT( "MovieSceneLiveLinkSource", "MovieSceneLiveLinkSourceStatus", "Active" );
}

FText FMovieSceneLiveLinkSource::GetSourceType() const
{
	return FText::Format(NSLOCTEXT("FMovieSceneLiveLinkSource", "MovieSceneLiveLinkSourceType", "{0} ({1})"),
		FText::FromString(UE::LiveLink::SourceTypes::SequencerSourcePrefix),
		FText::FromName(SubjectPreset.Key.SubjectName));
}

void FMovieSceneLiveLinkSource::PublishLiveLinkStaticData(FLiveLinkStaticDataStruct& StaticData)
{
	check(Client != nullptr);

	FLiveLinkStaticDataStruct TempStaticData;
	TempStaticData.InitializeWith(StaticData);
	Client->PushSubjectStaticData_AnyThread(SubjectPreset.Key, SubjectPreset.Role, MoveTemp(TempStaticData));
}

void FMovieSceneLiveLinkSource::PublishLiveLinkFrameData(TArray<FLiveLinkFrameDataStruct>& LiveLinkFrameDataArray)
{
	check(Client != nullptr);
	for (FLiveLinkFrameDataStruct& LiveLinkFrame : LiveLinkFrameDataArray)
	{
		// Share the data locally with the LiveLink client
		Client->PushSubjectFrameData_AnyThread(SubjectPreset.Key, MoveTemp(LiveLinkFrame));
	}
}

