// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"
#include "LiveLinkPresetTypes.h"



/** Publishes LiveLink From Sequencer*/
class FMovieSceneLiveLinkSource : public ILiveLinkSource, public TSharedFromThis<FMovieSceneLiveLinkSource>
{
public:
	FMovieSceneLiveLinkSource();
	virtual ~FMovieSceneLiveLinkSource() = default;
	static TSharedPtr<FMovieSceneLiveLinkSource> CreateLiveLinkSource(const FLiveLinkSubjectPreset& SubjectPreset);
	static void RemoveLiveLinkSource(TSharedPtr<FMovieSceneLiveLinkSource> Source);
	void PublishLiveLinkStaticData(FLiveLinkStaticDataStruct& StaticData);
	void PublishLiveLinkFrameData(TArray<FLiveLinkFrameDataStruct>& LiveLinkFrameDataArray);


	// ILiveLinkSource interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;
	virtual bool IsSourceStillValid() const override;
	virtual bool RequestSourceShutdown() override;
	virtual FText GetSourceMachineName() const override;
	virtual FText GetSourceStatus() const override;
	virtual FText GetSourceType() const override;
	// End ILiveLinkSource

public:	
	/** The local client to push data updates to */
	ILiveLinkClient* Client;

	/** Our identifier in LiveLink */
	FGuid SourceGuid;

	/** Our subject preset in LiveLink */
	FLiveLinkSubjectPreset SubjectPreset;

	/** Previously enabled subject with the same name than us when we were added to livelink. Used to put it back up when we tear down */
	FLiveLinkSubjectKey PreviousSubjectEnabled;
};
