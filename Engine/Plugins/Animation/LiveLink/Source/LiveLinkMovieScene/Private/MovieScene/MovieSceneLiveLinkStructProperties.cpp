// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkStructProperties.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "HAL/IConsoleManager.h"
#include "MovieScene/MovieSceneLiveLinkEnumHandler.h"
#include "MovieScene/MovieSceneLiveLinkPropertyHandler.h"
#include "MovieScene/MovieSceneLiveLinkTransformHandler.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UObject/EnumProperty.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkStructProperties)

static TAutoConsoleVariable<bool> CVarLiveLinkCompressSubSectionData(
	TEXT("LiveLink.CompressSubSectionData"), 1,
	TEXT("When enabled, compresses the LiveLink subsection data before serializing it to disk."),
	ECVF_Default);

FLiveLinkPropertyData::FLiveLinkPropertyData() = default;
FLiveLinkPropertyData::~FLiveLinkPropertyData() = default;

namespace LiveLinkPropertiesUtils
{
	TSharedPtr<IMovieSceneLiveLinkPropertyHandler> CreateHandlerFromProperty(FProperty* InProperty, const FLiveLinkStructPropertyBindings& InBinding, FLiveLinkPropertyData* InPropertyData)
	{
		if (InProperty->IsA(FFloatProperty::StaticClass()))
		{
			return MakeShared<FMovieSceneLiveLinkPropertyHandler<float>>(InBinding, InPropertyData);
		}
		else if (InProperty->IsA(FIntProperty::StaticClass()))
		{
			return MakeShared<FMovieSceneLiveLinkPropertyHandler<int32>>(InBinding, InPropertyData);
		}
		else if (InProperty->IsA(FBoolProperty::StaticClass()))
		{
			return MakeShared<FMovieSceneLiveLinkPropertyHandler<bool>>(InBinding, InPropertyData);
		}
		else if (InProperty->IsA(FStrProperty::StaticClass()))
		{
			return MakeShared<FMovieSceneLiveLinkPropertyHandler<FString>>(InBinding, InPropertyData);
		}
		else if (InProperty->IsA(FByteProperty::StaticClass()))
		{
			return MakeShared<FMovieSceneLiveLinkPropertyHandler<uint8>>(InBinding, InPropertyData);
		}
		else if (InProperty->IsA(FEnumProperty::StaticClass()))
		{
			return MakeShared<FMovieSceneLiveLinkEnumHandler>(InBinding, InPropertyData);
		}
		else if (InProperty->IsA(FStructProperty::StaticClass()))
		{
			if (FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (StructProperty->Struct->GetFName() == NAME_Transform)
				{
					return MakeShared<FMovieSceneLiveLinkTransformHandler>(InBinding, InPropertyData);
				}
				else if (StructProperty->Struct->GetFName() == NAME_Vector)
				{
					return MakeShared<FMovieSceneLiveLinkPropertyHandler<FVector>>(InBinding, InPropertyData);
				}
				else if (StructProperty->Struct->GetFName() == NAME_Color)
				{
					return MakeShared<FMovieSceneLiveLinkPropertyHandler<FColor>>(InBinding, InPropertyData);
				}
			}
		}

		return TSharedPtr<IMovieSceneLiveLinkPropertyHandler>();
	}

	TSharedPtr<IMovieSceneLiveLinkPropertyHandler> CreatePropertyHandler(const UScriptStruct& InStruct, FLiveLinkPropertyData* InPropertyData)
	{
		FLiveLinkStructPropertyBindings PropertyBinding(InPropertyData->PropertyName, InPropertyData->PropertyName.ToString());
		FProperty* PropertyPtr = PropertyBinding.GetProperty(InStruct);
		if (PropertyPtr == nullptr)
		{
			return TSharedPtr<IMovieSceneLiveLinkPropertyHandler>();
		}

		if (PropertyPtr->IsA(FArrayProperty::StaticClass()))
		{
			return CreateHandlerFromProperty(CastFieldChecked<FArrayProperty>(PropertyPtr)->Inner, PropertyBinding, InPropertyData);
		}
		else
		{
			return CreateHandlerFromProperty(PropertyPtr, PropertyBinding, InPropertyData);
		}
	}

}

bool FLiveLinkSubSectionData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsPersistent() && !Ar.IsObjectReferenceCollector())
	{
		TArray<uint8> DataBuffer;
		int32 UncompressedBufferSize = 0;
		const UScriptStruct* Struct = FLiveLinkSubSectionData::StaticStruct();

		if (Ar.IsLoading())
		{
			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LiveLinkCompressSubSectionData)
			{
				// Fallback to tagged property serialization.
				return false;
			}

			// Read uncompressed buffer size
			Ar << UncompressedBufferSize;
			DataBuffer.SetNumUninitialized(UncompressedBufferSize);

			if (UncompressedBufferSize > 0)
			{
				Ar.SerializeCompressedNew(DataBuffer.GetData(), DataBuffer.Num(), NAME_Oodle, NAME_Oodle, COMPRESS_BiasMemory);
			}

			FMemoryReader Reader(DataBuffer);
			FObjectAndNameAsStringProxyArchive StructAr(Reader, false);

			Struct->SerializeTaggedProperties(StructAr, (uint8*)this, Struct, nullptr);
			return true;
		}
		else if (CVarLiveLinkCompressSubSectionData.GetValueOnAnyThread())
		{
			FMemoryWriter MemoryWriter(DataBuffer);

			FObjectAndNameAsStringProxyArchive StructAr(MemoryWriter, false);

			Struct->SerializeTaggedProperties(StructAr, (uint8*)this, Struct, nullptr);

			// Store uncompressed buffer size
			UncompressedBufferSize = DataBuffer.Num();

			Ar << UncompressedBufferSize;

			if (UncompressedBufferSize > 0)
			{
				Ar.SerializeCompressedNew(DataBuffer.GetData(), DataBuffer.Num(), NAME_Oodle, NAME_Oodle, COMPRESS_BiasMemory);
			}

			return true;
		}
	}

	// Use tagged property serialization
	return false;
}