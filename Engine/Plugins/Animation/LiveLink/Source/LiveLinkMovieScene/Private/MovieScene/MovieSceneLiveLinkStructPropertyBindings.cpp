// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkStructPropertyBindings.h"

#include "LiveLinkMovieScenePrivate.h"
#include "UObject/EnumProperty.h"

//------------------------------------------------------------------------------
// FLiveLinkStructPropertyBindings implementation.
//------------------------------------------------------------------------------

TMap<FLiveLinkStructPropertyBindings::FPropertyNameKey, FLiveLinkStructPropertyBindings::FPropertyWrapper> FLiveLinkStructPropertyBindings::PropertyCache;

FLiveLinkStructPropertyBindings::FLiveLinkStructPropertyBindings(FName InPropertyName, const FString& InPropertyPath)
	: PropertyPath(InPropertyPath)
	, PropertyName(InPropertyName)
{
}

void FLiveLinkStructPropertyBindings::CacheBinding(const UScriptStruct& InStruct)
{
	FPropertyWrapper Property = FindProperty(InStruct, PropertyPath);
	PropertyCache.Add(FPropertyNameKey(InStruct.GetFName(), PropertyName), Property);
}

FProperty* FLiveLinkStructPropertyBindings::GetProperty(const UScriptStruct& InStruct) const
{
	FPropertyWrapper FoundProperty = PropertyCache.FindRef(FPropertyNameKey(InStruct.GetFName(), PropertyName));
	if (FProperty* Property = FoundProperty.GetProperty())
	{
		return Property;
	}

	return FindProperty(InStruct, PropertyPath).GetProperty();
}

int64 FLiveLinkStructPropertyBindings::GetCurrentValueForEnumAt(int32 InIndex, const UScriptStruct& InStruct, const void* InSourceAddress)
{
	FPropertyWrapper FoundProperty = FindOrAdd(InStruct);

	if (FProperty* Property = FoundProperty.GetProperty())
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(ArrayProperty->Inner))
			{
				const void* BaseAddr = FoundProperty.GetPropertyAddress<void>(InSourceAddress, 0);
				FScriptArrayHelper ArrayHelper(ArrayProperty, BaseAddr);
				ArrayHelper.ExpandForIndex(InIndex);
				const void* ValueAddr = ArrayHelper.GetRawPtr(InIndex);

				FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
				return UnderlyingProperty->GetSignedIntPropertyValue(ValueAddr);
			}
			else
			{
				UE_LOGF(LogLiveLinkMovieScene, Error, "Mismatch in property binding evaluation. %ls is not of type: %ls", *Property->GetName(), *FEnumProperty::StaticClass()->GetName());
			}
			}
		else
		{
			if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
			{
				FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
				const void* ValueAddr = FoundProperty.GetPropertyAddress<void>(InSourceAddress, InIndex);
				return UnderlyingProperty->GetSignedIntPropertyValue(ValueAddr);
		}
		else
		{
			UE_LOGF(LogLiveLinkMovieScene, Error, "Mismatch in property binding evaluation. %ls is not of type: %ls", *Property->GetName(), *FEnumProperty::StaticClass()->GetName());
		}

		
		}
	}

	return 0;
}

void FLiveLinkStructPropertyBindings::SetCurrentValueForEnumAt(int32 InIndex, const UScriptStruct& InStruct, void* InSourceAddress, int64 InValue)
{
	FPropertyWrapper FoundProperty = FindOrAdd(InStruct);

	if (FProperty* Property = FoundProperty.GetProperty())
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(ArrayProperty->Inner))
			{
				const void* BaseAddr = FoundProperty.GetPropertyAddress<void>(InSourceAddress, 0);
				FScriptArrayHelper ArrayHelper(ArrayProperty, BaseAddr);
				ArrayHelper.ExpandForIndex(InIndex);
				void* ValueAddr = ArrayHelper.GetRawPtr(InIndex);

				FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
				UnderlyingProperty->SetIntPropertyValue(ValueAddr, InValue);
			}
			else
			{
				UE_LOGF(LogLiveLinkMovieScene, Error, "Mismatch in property binding evaluation. %ls is not of type: %ls", *Property->GetName(), *FEnumProperty::StaticClass()->GetName());
			}
		}
		else
		{
			if (FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(Property))
			{
				FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
				void* ValueAddr = FoundProperty.GetPropertyAddress<void>(InSourceAddress, InIndex);	
				UnderlyingProperty->SetIntPropertyValue(ValueAddr, InValue);
			}
			else
			{
				UE_LOGF(LogLiveLinkMovieScene, Error, "Mismatch in property binding evaluation. %ls is not of type: %ls", *Property->GetName(), *FEnumProperty::StaticClass()->GetName());
			}
		}
	}
}

template<> bool FLiveLinkStructPropertyBindings::GetCurrentValue<bool>(const UScriptStruct& InStruct, const void* InSourceAddress)
{
	return GetCurrentValueAt<bool>(0, InStruct, InSourceAddress);
}

template<> bool FLiveLinkStructPropertyBindings::GetCurrentValueAt<bool>(int32 InIndex, const UScriptStruct& InStruct, const void* InSourceAddress)
{
	FPropertyWrapper FoundProperty = FindOrAdd(InStruct);
	if (FProperty* Property = FoundProperty.GetProperty())
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(ArrayProperty->Inner))
			{
				const void* BaseAddr = FoundProperty.GetPropertyAddress<void>(InSourceAddress, 0);
				FScriptArrayHelper ArrayHelper(ArrayProperty, BaseAddr);
				ArrayHelper.ExpandForIndex(InIndex);
				const uint8* ValuePtr = ArrayHelper.GetRawPtr(InIndex);
				return BoolProperty->GetPropertyValue(ValuePtr);
			}
			else
			{
				UE_LOGF(LogLiveLinkMovieScene, Error, "Mismatch in property binding evaluation. %ls is not of type: %ls", *Property->GetName(), *FBoolProperty::StaticClass()->GetName());
			}
		}
		else
		{
			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				const uint8* ValuePtr = FoundProperty.GetPropertyAddress<uint8>(InSourceAddress, InIndex);
				return BoolProperty->GetPropertyValue(ValuePtr);
		}
		else
		{
			UE_LOGF(LogLiveLinkMovieScene, Error, "Mismatch in property binding evaluation. %ls is not of type: %ls", *Property->GetName(), *FBoolProperty::StaticClass()->GetName());
		}
	}
	}

	return false;
}

template<> void FLiveLinkStructPropertyBindings::SetCurrentValue<bool>(const UScriptStruct& InStruct, void* InSourceAddress, TCallTraits<bool>::ParamType InValue)
{
	SetCurrentValueAt<bool>(0, InStruct, InSourceAddress, InValue);
}

template<> void FLiveLinkStructPropertyBindings::SetCurrentValueAt<bool>(int32 InIndex, const UScriptStruct& InStruct, void* InSourceAddress, TCallTraits<bool>::ParamType InValue)
{
	FPropertyWrapper FoundProperty = FindOrAdd(InStruct);
	if (FProperty* Property = FoundProperty.GetProperty())
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(ArrayProperty->Inner))
			{
				const void* BaseAddr = FoundProperty.GetPropertyAddress<void>(InSourceAddress, 0);
				FScriptArrayHelper ArrayHelper(ArrayProperty, BaseAddr);
				ArrayHelper.ExpandForIndex(InIndex);
				void* ValuePtr = ArrayHelper.GetRawPtr(InIndex);
				if (ValuePtr)
				{
					BoolProperty->SetPropertyValue(ValuePtr, InValue);
				}
			}
		}
		else
		{
			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				uint8* ValuePtr = FoundProperty.GetPropertyAddress<uint8>(InSourceAddress, InIndex);
				BoolProperty->SetPropertyValue(ValuePtr, InValue);
			}
		}
	}
}

FLiveLinkStructPropertyBindings::FPropertyWrapper FLiveLinkStructPropertyBindings::FindPropertyRecursive(const UScriptStruct* InStruct, TArray<FString>& InPropertyNames, uint32 Index, void* ContainerAddress, int32 PreviousDelta)
{
	FPropertyWrapper FoundProperty;
	FoundProperty.Property = FindFProperty<FProperty>(InStruct, *InPropertyNames[Index]);
	FoundProperty.DeltaAddress = PreviousDelta;

	if (FStructProperty* StructProp = CastField<FStructProperty>(FoundProperty.Property.Get()))
	{
		if (InPropertyNames.IsValidIndex(Index + 1))
		{
			//For each structure depth, keep the address delta from the root to be able to reuse it for each frame data
			void* StructContainer = StructProp->ContainerPtrToValuePtr<void>(ContainerAddress);
			const int32 NewDelta = ((int64)StructContainer - (int64)ContainerAddress) + PreviousDelta;
			return FindPropertyRecursive(StructProp->Struct, InPropertyNames, Index + 1, StructContainer, NewDelta);
		}
		else
		{
			check(StructProp->GetName() == InPropertyNames[Index]);
		}
	}

	return FoundProperty;
}

FLiveLinkStructPropertyBindings::FPropertyWrapper FLiveLinkStructPropertyBindings::FindProperty(const UScriptStruct& InStruct, const FString& InPropertyPath)
{
	//Split the property path to recursively find the actual property
	TArray<FString> PropertyNames;
	InPropertyPath.ParseIntoArray(PropertyNames, TEXT("."), true);

	if (PropertyNames.Num() > 0)
	{
		return FindPropertyRecursive(&InStruct, PropertyNames, 0, (void*)&InStruct, 0);
	}
	else
	{
		return FPropertyWrapper();
	}
}
