// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkSubSection.h"


#include "UObject/UObjectIterator.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkSubSection)


UMovieSceneLiveLinkSubSection::UMovieSceneLiveLinkSubSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


void UMovieSceneLiveLinkSubSection::Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData)
{
	SubjectRole = InSubjectRole;
	SetStaticData(InStaticData);
}

int32 UMovieSceneLiveLinkSubSection::GetChannelCount() const
{
	int32 ChannelCount = 0;
	for (const FLiveLinkPropertyData& Data : SubSectionData.Properties)
	{
		ChannelCount += Data.GetChannelCount();
	}

	return ChannelCount;
}

TArray<TSubclassOf<UMovieSceneLiveLinkSubSection>> UMovieSceneLiveLinkSubSection::GetLiveLinkSubSectionForRole(const TSubclassOf<ULiveLinkRole>& InRoleToSupport)
{
	TArray<TSubclassOf<UMovieSceneLiveLinkSubSection>> Results;
	for (TObjectIterator<UClass> Itt; Itt; ++Itt)
	{
		if (Itt->IsChildOf(UMovieSceneLiveLinkSubSection::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			Results.Add(*Itt);
		}
	}
	return MoveTemp(Results);
}

void UMovieSceneLiveLinkSubSection::SetStaticData(const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData)
{
	StaticData = InStaticData;
}

void UMovieSceneLiveLinkSubSection::PostLoad()
{
	Super::PostLoad();
}

