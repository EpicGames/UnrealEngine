// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkSubSectionAnimation.h"

#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"
#include "MovieScene/MovieSceneLiveLinkSubSection.h"
#include "MovieScene/MovieSceneLiveLinkTransformHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkSubSectionAnimation)

#define LOCTEXT_NAMESPACE "MovieSceneLiveLinkSubSectionAnimation"


namespace MovieSceneLiveLinkSubSectionAnimationUtil
{
	static const TArray<FString> StringArray =
	{
		"Translation-X",
		"Translation-Y",
		"Translation-Z",
		"Rotation-X",
		"Rotation-Y",
		"Rotation-Z",
		"Scale-X",
		"Scale-Y",
		"Scale-Z"
	};
}


UMovieSceneLiveLinkSubSectionAnimation::UMovieSceneLiveLinkSubSectionAnimation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMovieSceneLiveLinkSubSectionAnimation::Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData)
{
	Super::Initialize(InSubjectRole, InStaticData);

	CreatePropertiesChannel();
}

int32 UMovieSceneLiveLinkSubSectionAnimation::CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData)
{
	int32 StartIndex = InChannelIndex;
	InChannelIndex = 0;

	FLiveLinkSkeletonStaticData* SkeletonData = StaticData->Cast<FLiveLinkSkeletonStaticData>();
	check(SkeletonData);

	for (int32 i = 0; i < SkeletonData->BoneNames.Num(); ++i)
	{
		const FName BoneName = SkeletonData->BoneNames[i];
		for (FString String : MovieSceneLiveLinkSubSectionAnimationUtil::StringArray)
		{
			const FText DisplayName = FText::Format(LOCTEXT("LinkLinkFormat", "{0} : {1}"), FText::FromName(BoneName), FText::FromString(String));
#if WITH_EDITOR
			MovieSceneLiveLinkSectionUtils::CreateChannelEditor(DisplayName, SubSectionData.Properties[0].FloatChannel[InChannelIndex], StartIndex + InChannelIndex, TMovieSceneExternalValue<float>(), OutChannelMask, OutChannelData);
#else
			OutChannelData.Add(SubSectionData.Properties[0].FloatChannel[InChannelIndex]);
#endif //#WITH_EDITOR

			++InChannelIndex;
		}
	}

	return InChannelIndex;
}

void UMovieSceneLiveLinkSubSectionAnimation::CreatePropertiesChannel()
{
	FLiveLinkSkeletonStaticData* SkeletonData = StaticData->Cast<FLiveLinkSkeletonStaticData>();
	check(SkeletonData);

	const FName TransformsPropertyName = GET_MEMBER_NAME_CHECKED(FLiveLinkAnimationFrameData, Transforms);
	const int32 TransformCount = SkeletonData->BoneNames.Num();
	if (TransformCount <= 0)
	{
		return;
	}

	SubSectionData.Properties.SetNum(1);
	SubSectionData.Properties[0].PropertyName = TransformsPropertyName;

	TransformHandler = MakeShared<FMovieSceneLiveLinkTransformHandler>(FLiveLinkStructPropertyBindings(TransformsPropertyName, TransformsPropertyName.ToString()), &SubSectionData.Properties[0]);
	TransformHandler->CreateChannels(*FLiveLinkAnimationFrameData::StaticStruct(), TransformCount);
}

void UMovieSceneLiveLinkSubSectionAnimation::RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData)
{
	const FLiveLinkAnimationFrameData* AnimationFrameData = InFrameData.Cast<FLiveLinkAnimationFrameData>();
	check(AnimationFrameData);

	if (TransformHandler.IsValid())
	{
		TransformHandler->RecordFrame(InFrameNumber, *FLiveLinkAnimationFrameData::StaticStruct(), InFrameData.GetBaseData());
	}
}

void UMovieSceneLiveLinkSubSectionAnimation::FinalizeSection(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	if (TransformHandler.IsValid())
	{
		TransformHandler->Finalize(bInReduceKeys, InOptimizationParams);
	}
}

bool UMovieSceneLiveLinkSubSectionAnimation::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const
{
	return RoleToSupport->IsChildOf(ULiveLinkAnimationRole::StaticClass());
}

#undef LOCTEXT_NAMESPACE // MovieSceneLiveLinkAnimationSection

