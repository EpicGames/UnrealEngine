// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkSubSectionBasicRole.h"

#include "Channels/MovieSceneFloatChannel.h"
#include "MovieScene/IMovieSceneLiveLinkPropertyHandler.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"
#include "MovieScene/MovieSceneLiveLinkSubSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkSubSectionBasicRole)

#define LOCTEXT_NAMESPACE "MovieSceneLiveLinkSubSectionBasicRole"


UMovieSceneLiveLinkSubSectionBasicRole::UMovieSceneLiveLinkSubSectionBasicRole(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMovieSceneLiveLinkSubSectionBasicRole::Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData)
{
	Super::Initialize(InSubjectRole, InStaticData);

	CreatePropertiesChannel();
}

int32 UMovieSceneLiveLinkSubSectionBasicRole::CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData)
{
	int32 StartIndex = InChannelIndex;
	InChannelIndex = 0;

	FLiveLinkBaseStaticData* BasicData = StaticData->Cast<FLiveLinkBaseStaticData>();
	check(BasicData);
	for (int32 i = 0; i < BasicData->PropertyNames.Num(); ++i)
	{
		const FName PropertyName = BasicData->PropertyNames[i];
		const FText DisplayName = FText::Format(LOCTEXT("LinkLinkFormat", "{0}"), FText::FromName(PropertyName));
#if WITH_EDITOR
		MovieSceneLiveLinkSectionUtils::CreateChannelEditor(DisplayName, SubSectionData.Properties[0].FloatChannel[InChannelIndex], StartIndex + InChannelIndex, TMovieSceneExternalValue<float>(), OutChannelMask, OutChannelData);
#else
		OutChannelData.Add(SubSectionData.Properties[0].FloatChannel[InChannelIndex]);
#endif //#WITH_EDITOR

		++InChannelIndex;
	}

	return InChannelIndex;
}

void UMovieSceneLiveLinkSubSectionBasicRole::CreatePropertiesChannel()
{
	FLiveLinkBaseStaticData* BasicData = StaticData->Cast<FLiveLinkBaseStaticData>();
	check(BasicData);

	const int32 PropertyCount = BasicData->PropertyNames.Num();
	if (PropertyCount <= 0)
	{
		return;
	}

	const FName PropertyName = GET_MEMBER_NAME_CHECKED(FLiveLinkBaseFrameData, PropertyValues);
	SubSectionData.Properties.SetNum(1);
	SubSectionData.Properties[0].PropertyName = PropertyName;
	PropertyHandler = LiveLinkPropertiesUtils::CreatePropertyHandler(*FLiveLinkBaseFrameData::StaticStruct(), &SubSectionData.Properties[0]);
	PropertyHandler->CreateChannels(*FLiveLinkBaseFrameData::StaticStruct(), PropertyCount);
}

void UMovieSceneLiveLinkSubSectionBasicRole::RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData)
{
	const FLiveLinkBaseFrameData* BaseFrameData = InFrameData.GetBaseData();
	check(BaseFrameData);

	if (PropertyHandler.IsValid())
	{
		PropertyHandler->RecordFrame(InFrameNumber, *FLiveLinkBaseFrameData::StaticStruct(), InFrameData.GetBaseData());
	}
}

void UMovieSceneLiveLinkSubSectionBasicRole::FinalizeSection(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	if (StaticData->GetBaseData()->PropertyNames.Num() > 0)
	{
		PropertyHandler->Finalize(bInReduceKeys, InOptimizationParams);
	}
}

bool UMovieSceneLiveLinkSubSectionBasicRole::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const
{
	return true;
}

#undef LOCTEXT_NAMESPACE // MovieSceneLiveLinkSubSectionBasicRole

