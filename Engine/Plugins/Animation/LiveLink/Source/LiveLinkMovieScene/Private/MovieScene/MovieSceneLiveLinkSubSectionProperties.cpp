// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkSubSectionProperties.h"

#include "Channels/MovieSceneBoolChannel.h"

#include "Channels/MovieSceneByteChannel.h"
#include "LiveLinkMovieScenePrivate.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "LiveLinkRole.h"
#include "MovieScene/IMovieSceneLiveLinkPropertyHandler.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"
#include "MovieScene/MovieSceneLiveLinkStructPropertyBindings.h"
#include "MovieScene/MovieSceneLiveLinkSubSection.h"
#include "UObject/EnumProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkSubSectionProperties)

#define LOCTEXT_NAMESPACE "MovieSceneLiveLinkSubSectionProperties"


namespace MovieSceneLiveLinkPropertiesUtil
{
	static const TArray<FString> TransformStrings =
	{
		"Translation.X",
		"Translation.Y",
		"Translation.Z",
		"Rotation.X",
		"Rotation.Y",
		"Rotation.Z",
		"Scale.X",
		"Scale.Y",
		"Scale.Z"
	};

	static const TArray<FString> VectorStrings =
	{
		"Vector.X",
		"Vector.Y",
		"Vector.Z",
	};

	static const TArray<FString> ColorStrings =
	{
		"Color.R",
		"Color.G",
		"Color.B",
		"Color.A",
	};
};



UMovieSceneLiveLinkSubSectionProperties::UMovieSceneLiveLinkSubSectionProperties(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMovieSceneLiveLinkSubSectionProperties::Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData)
{
	Super::Initialize(InSubjectRole, InStaticData);

	UScriptStruct* ScriptStruct = SubjectRole.GetDefaultObject()->GetFrameDataStruct();
	CreatePropertiesChannel(ScriptStruct);
}

int32 UMovieSceneLiveLinkSubSectionProperties::CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData)
{
	int32 StartIndex = InChannelIndex;
	InChannelIndex = 0;

	for (FLiveLinkPropertyData& Data : SubSectionData.Properties)
	{
		UScriptStruct* ScriptStruct = SubjectRole.GetDefaultObject()->GetFrameDataStruct();

		FLiveLinkStructPropertyBindings PropertyBinding(Data.PropertyName, Data.PropertyName.ToString());
		if(FProperty* PropertyPtr = PropertyBinding.GetProperty(*ScriptStruct))
		{
			if (PropertyPtr->ArrayDim > 1)
			{
				for (int32 CArrayIndex = 0; CArrayIndex < PropertyPtr->ArrayDim; ++CArrayIndex)
				{
					const FText PropertyName = FText::Format(LOCTEXT("LiveLinkRecordedPropertyName", "{0}[{1}]"), FText::FromName(Data.PropertyName), CArrayIndex);
					InChannelIndex += CreateChannelProxyInternal(PropertyPtr, Data, CArrayIndex, StartIndex + InChannelIndex, OutChannelMask, OutChannelData, PropertyName);
				}
			}
			else
			{
				const FText PropertyName = FText::FromName(Data.PropertyName);
				InChannelIndex += CreateChannelProxyInternal(PropertyPtr, Data, 0, StartIndex + InChannelIndex, OutChannelMask, OutChannelData, PropertyName);
			}
		}
	}
	
	return InChannelIndex;
}

void UMovieSceneLiveLinkSubSectionProperties::CreatePropertyList(UScriptStruct* InScriptStruct, bool bCheckInterpFlag, const FString& InOwner)
{
	for (TFieldIterator<FProperty> It(InScriptStruct, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::ExcludeInterfaces); It; ++It)
	{
		if (bCheckInterpFlag)
		{
		const bool bIsInterpField = It->HasAllPropertyFlags(CPF_Interp);
		if (!bIsInterpField)
		{
			continue;
		}
		}

		if (*It)
		{
			FString FullPath = InOwner;
			if (!FullPath.IsEmpty())
			{
				FullPath += ".";
			}

			FullPath += *It->GetFName().ToString();

		if (!IsPropertyTypeSupported(*It))
		{
				//Property is not directly support, dig deeper if it's a struct
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(*It))
				{
					const bool bDeeperCheckInterpFlag = false;
					CreatePropertyList(StructProperty->Struct, bDeeperCheckInterpFlag, FullPath);
				}
			continue;
		}

		FLiveLinkPropertyData& NewProperty = SubSectionData.Properties.AddDefaulted_GetRef();
			NewProperty.PropertyName = *FullPath;
		}
	}
}

void UMovieSceneLiveLinkSubSectionProperties::CreatePropertiesChannel(UScriptStruct* InScriptStruct)
{
	const FString EmptyOwner;
	const bool bCheckForInterpFlag = true;
	CreatePropertyList(InScriptStruct, bCheckForInterpFlag, EmptyOwner);

	//Create the handler in a second pass so its reference to the property data is valid
	const UScriptStruct* Container = SubjectRole.GetDefaultObject()->GetFrameDataStruct();
	for (FLiveLinkPropertyData& Data : SubSectionData.Properties)
	{
		//Query the property for its dimension to support c-style arrays
		FLiveLinkStructPropertyBindings PropertyBinding(Data.PropertyName, Data.PropertyName.ToString());
		FProperty* PropertyPtr = PropertyBinding.GetProperty(*Container);

		//If it was added to the sub section data, property must be valid
		check(PropertyPtr);
		
		TSharedPtr<IMovieSceneLiveLinkPropertyHandler> PropertyHandler = LiveLinkPropertiesUtils::CreatePropertyHandler(*Container, &Data);
		PropertyHandler->CreateChannels(*Container, PropertyPtr->ArrayDim);
		PropertyHandlers.Add(PropertyHandler);
	}
}

void UMovieSceneLiveLinkSubSectionProperties::RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData)
{
	for (TSharedPtr<IMovieSceneLiveLinkPropertyHandler>& PropertyHandler : PropertyHandlers)
	{
		PropertyHandler->RecordFrame(InFrameNumber, *SubjectRole.GetDefaultObject()->GetFrameDataStruct(), InFrameData.GetBaseData());
	}
}

void UMovieSceneLiveLinkSubSectionProperties::FinalizeSection(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	for (TSharedPtr<IMovieSceneLiveLinkPropertyHandler>& PropertyHandler : PropertyHandlers)
	{
		PropertyHandler->Finalize(bInReduceKeys, InOptimizationParams);
	}
}

bool UMovieSceneLiveLinkSubSectionProperties::IsPropertyTypeSupported(const FProperty* InProperty) const
{
	if (InProperty == nullptr)
	{
		return false;
	}

	//Arrays are not supported because we can't know the number of elements to create for it. If arrays are desired, a sub section will have to manage it 
	//See Animation subsection with transforms
	if (CastField<FFloatProperty>(InProperty)
		|| CastField<FIntProperty>(InProperty)
		|| CastField<FStrProperty>(InProperty)
		|| CastField<FByteProperty>(InProperty)
		|| CastField<FBoolProperty>(InProperty)
		|| CastField<FEnumProperty>(InProperty))
	{
		return true;
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		if (StructProperty->Struct->GetFName() == NAME_Transform
			|| StructProperty->Struct->GetFName() == NAME_Vector
			|| StructProperty->Struct->GetFName() == NAME_Color)
		{
			return true;
		}
	}

	return false;
}

int32 UMovieSceneLiveLinkSubSectionProperties::CreateChannelProxyInternal(FProperty* InPropertyPtr, FLiveLinkPropertyData& OutPropertyData, int32 InPropertyIndex, int32 GlobalIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData, const FText& InPropertyName)
{
	int32 CreatedChannelCount = 0;

	check(InPropertyPtr);

	if (CastField<FFloatProperty>(InPropertyPtr))
	{
#if WITH_EDITOR
		MovieSceneLiveLinkSectionUtils::CreateChannelEditor(InPropertyName, OutPropertyData.FloatChannel[InPropertyIndex], GlobalIndex, TMovieSceneExternalValue<float>(), OutChannelMask, OutChannelData);
#else
		OutChannelData.Add(OutPropertyData.FloatChannel[InPropertyIndex]);
#endif //#WITH_EDITOR
		CreatedChannelCount = 1;
	}
	else if (CastField<FIntProperty>(InPropertyPtr))
	{
#if WITH_EDITOR
		MovieSceneLiveLinkSectionUtils::CreateChannelEditor(InPropertyName, OutPropertyData.IntegerChannel[InPropertyIndex], GlobalIndex, TMovieSceneExternalValue<int32>(), OutChannelMask, OutChannelData);
#else
		OutChannelData.Add(OutPropertyData.IntegerChannel[InPropertyIndex]);
#endif //#WITH_EDITOR
		CreatedChannelCount = 1;
	}
	else if (CastField<FBoolProperty>(InPropertyPtr))
	{
#if WITH_EDITOR
		MovieSceneLiveLinkSectionUtils::CreateChannelEditor(InPropertyName, OutPropertyData.BoolChannel[InPropertyIndex], GlobalIndex, TMovieSceneExternalValue<bool>(), OutChannelMask, OutChannelData);
#else
		OutChannelData.Add(OutPropertyData.BoolChannel[InPropertyIndex]);
#endif //#WITH_EDITOR
		CreatedChannelCount = 1;
	}
	else if (CastField<FStrProperty>(InPropertyPtr))
	{
#if WITH_EDITOR
		MovieSceneLiveLinkSectionUtils::CreateChannelEditor(InPropertyName, OutPropertyData.StringChannel[InPropertyIndex], GlobalIndex, TMovieSceneExternalValue<FString>(), OutChannelMask, OutChannelData);
#else
		OutChannelData.Add(OutPropertyData.StringChannel[InPropertyIndex]);
#endif //#WITH_EDITOR
		CreatedChannelCount = 1;
	}
	else if (CastField<FByteProperty>(InPropertyPtr))
	{
#if WITH_EDITOR
		MovieSceneLiveLinkSectionUtils::CreateChannelEditor(InPropertyName, OutPropertyData.ByteChannel[InPropertyIndex], GlobalIndex, TMovieSceneExternalValue<uint8>(), OutChannelMask, OutChannelData);
#else
		OutChannelData.Add(OutPropertyData.ByteChannel[InPropertyIndex]);
#endif //#WITH_EDITOR
		CreatedChannelCount = 1;
	}
	else if (CastField<FEnumProperty>(InPropertyPtr))
	{
#if WITH_EDITOR
		MovieSceneLiveLinkSectionUtils::CreateChannelEditor(InPropertyName, OutPropertyData.ByteChannel[InPropertyIndex], GlobalIndex, TMovieSceneExternalValue<uint8>(), OutChannelMask, OutChannelData);
#else
		OutChannelData.Add(OutPropertyData.ByteChannel[InPropertyIndex]);
#endif //#WITH_EDITOR
		CreatedChannelCount = 1;
	}
	else if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyPtr))
	{
		if (StructProperty->Struct->GetFName() == NAME_Transform)
		{
			const int32 PerPropertyChannelCount = MovieSceneLiveLinkPropertiesUtil::TransformStrings.Num();
			const int32 ChannelOffset = PerPropertyChannelCount * InPropertyIndex;
			for (int32 i = 0; i < PerPropertyChannelCount; ++i)
			{
				const FText DisplayName = FText::Format(LOCTEXT("LinkLinkFormat", "{0} : {1}"), InPropertyName, FText::FromString(MovieSceneLiveLinkPropertiesUtil::TransformStrings[i]));
#if WITH_EDITOR
				MovieSceneLiveLinkSectionUtils::CreateChannelEditor(DisplayName, OutPropertyData.FloatChannel[i + ChannelOffset], GlobalIndex + CreatedChannelCount, TMovieSceneExternalValue<float>(), OutChannelMask, OutChannelData);
#else
				OutChannelData.Add(OutPropertyData.FloatChannel[i + ChannelOffset]);
#endif //#WITH_EDITOR
				++CreatedChannelCount;
			}
		}
		else if (StructProperty->Struct->GetFName() == NAME_Vector)
		{
			const int32 PerPropertyChannelCount = MovieSceneLiveLinkPropertiesUtil::VectorStrings.Num();
			const int32 ChannelOffset = PerPropertyChannelCount * InPropertyIndex;
			for (int32 i = 0; i < PerPropertyChannelCount; ++i)
			{
				const FText DisplayName = FText::Format(LOCTEXT("LinkLinkFormat", "{0} : {1}"), InPropertyName, FText::FromString(MovieSceneLiveLinkPropertiesUtil::VectorStrings[i]));
#if WITH_EDITOR
				MovieSceneLiveLinkSectionUtils::CreateChannelEditor(DisplayName, OutPropertyData.FloatChannel[i + ChannelOffset], GlobalIndex + CreatedChannelCount, TMovieSceneExternalValue<float>(), OutChannelMask, OutChannelData);
#else
				OutChannelData.Add(OutPropertyData.FloatChannel[i + ChannelOffset]);
#endif //#WITH_EDITOR
				++CreatedChannelCount;
			}
		}
		else if (StructProperty->Struct->GetFName() == NAME_Color)
		{
			const int32 PerPropertyChannelCount = MovieSceneLiveLinkPropertiesUtil::ColorStrings.Num();
			const int32 ChannelOffset = PerPropertyChannelCount * InPropertyIndex;
			for (int32 i = 0; i < PerPropertyChannelCount; ++i)
			{
				const FText DisplayName = FText::Format(LOCTEXT("LinkLinkFormat", "{0} : {1}"), InPropertyName, FText::FromString(MovieSceneLiveLinkPropertiesUtil::ColorStrings[i]));
#if WITH_EDITOR
				MovieSceneLiveLinkSectionUtils::CreateChannelEditor(DisplayName, OutPropertyData.ByteChannel[i + ChannelOffset], GlobalIndex + CreatedChannelCount, TMovieSceneExternalValue<uint8>(), OutChannelMask, OutChannelData);
#else
				OutChannelData.Add(OutPropertyData.ByteChannel[i + ChannelOffset]);
#endif //#WITH_EDITOR
				++CreatedChannelCount;
			}
		}
		else
		{
			//This should not happen. Supported properties should be filtered in UMovieSceneLiveLinkSubSectionProperties::IsPropertyTypeSupported with this code updated accordingly.
			UE_LOGF(LogLiveLinkMovieScene, Warning, "Trying to create a proxy channel for subject role '%ls' for an unsupported structure type '%ls'", *SubjectRole.GetDefaultObject()->GetDisplayName().ToString(), *StructProperty->Struct->GetFName().ToString());
		}
	}
	else
	{
		//This should not happen. Supported properties should be filtered in UMovieSceneLiveLinkSubSectionProperties::IsPropertyTypeSupported with this code updated accordingly.
		UE_LOGF(LogLiveLinkMovieScene, Warning, "Trying to create a proxy channel for subject role '%ls' for an unsupported property type '%ls'", *SubjectRole.GetDefaultObject()->GetDisplayName().ToString(), *InPropertyPtr->GetFName().ToString());
	}

	return CreatedChannelCount;
}

bool UMovieSceneLiveLinkSubSectionProperties::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const
{
	return true;
}

#undef LOCTEXT_NAMESPACE // MovieSceneLiveLinkSubSectionProperties

