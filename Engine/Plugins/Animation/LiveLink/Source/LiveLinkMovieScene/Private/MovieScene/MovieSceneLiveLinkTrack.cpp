// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkTrack.h"

#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "MovieScene/MovieSceneLiveLinkSection.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h" // IWYU pragma: keep
#include "MovieSceneLiveLinkSectionTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkTrack)

UMovieSceneLiveLinkTrack::UMovieSceneLiveLinkTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(48, 227, 255, 65);
#endif
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneLiveLinkTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneLiveLinkSection::StaticClass();
}

UMovieSceneSection* UMovieSceneLiveLinkTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSection>(this, UMovieSceneLiveLinkSection::StaticClass(), NAME_None, RF_Transactional);
}

FMovieSceneEvalTemplatePtr UMovieSceneLiveLinkTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneLiveLinkSectionTemplate(*CastChecked<const UMovieSceneLiveLinkSection>(&InSection), *this);
}

void UMovieSceneLiveLinkTrack::PostCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const
{
	Track.SetEvaluationMethod(EEvaluationMethod::Swept);
}

#if WITH_EDITORONLY_DATA

void UMovieSceneLiveLinkTrack::SetDisplayName(const FText& NewDisplayName)
{
	Super::SetDisplayName(NewDisplayName);
	FString StringName = NewDisplayName.ToString();
	FName Name(*StringName);
	SetPropertyNameAndPath(Name, StringName);
	for (UMovieSceneSection* Section : Sections)
	{
		UMovieSceneLiveLinkSection* LiveLinkSection = Cast<UMovieSceneLiveLinkSection>(Section);
		if (LiveLinkSection)
		{
			LiveLinkSection->SetSubjectName(Name);
		}
	}
}

#endif

