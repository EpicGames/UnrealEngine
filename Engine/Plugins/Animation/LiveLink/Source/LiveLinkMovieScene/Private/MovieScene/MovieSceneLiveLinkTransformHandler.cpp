// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLiveLinkTransformHandler.h"

#include "Channels/MovieSceneChannelData.h"
#include "Misc/ScopedSlowTask.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"


namespace LiveLinkTransformHandlerUtils
{
	void FillTransformInterpolated(const FFrameTime& InFrameTime, int32& InOutStartIndex, const TArray<FMovieSceneFloatChannel>& InSourceChannels, FTransform& OutTransform)
	{
		FVector3f TempVector;

		for (int32 i = 0; i < 3; ++i)
		{
			InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempVector[i]);
		}
		OutTransform.SetLocation((FVector)TempVector);

		for (int32 i = 0; i < 3; ++i)
		{
			InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempVector[i]);
		}
		FRotator Rotator(TempVector.Y, TempVector.Z, TempVector.X); //pitch, yaw, roll
		FQuat Quat = Rotator.Quaternion();
		OutTransform.SetRotation(Quat);

		for (int32 i = 0; i < 3; ++i)
		{
			InSourceChannels[InOutStartIndex++].Evaluate(InFrameTime, TempVector[i]);
		}
		OutTransform.SetScale3D((FVector)TempVector);
	}

	void FillTransform(int32 InKeyIndex, int32& InOutStartIndex, const TArray<FMovieSceneFloatChannel>& InSourceChannels, FTransform& OutTransform)
	{
		FVector TempVector;

		for (int32 i = 0; i < 3; ++i)
		{
			TempVector[i] = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex].Value;
			++InOutStartIndex;
		}
		OutTransform.SetLocation(TempVector);

		for (int32 i = 0; i < 3; ++i)
		{
			TempVector[i] = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex].Value;
			++InOutStartIndex;
		}
		FRotator Rotator(TempVector.Y, TempVector.Z, TempVector.X); //pitch, yaw, roll
		FQuat Quat = Rotator.Quaternion();
		OutTransform.SetRotation(Quat);

		for (int32 i = 0; i < 3; ++i)
		{
			TempVector[i] = InSourceChannels[InOutStartIndex].GetValues()[InKeyIndex].Value;
			++InOutStartIndex;
		}
		OutTransform.SetScale3D(TempVector);
	}
}


//------------------------------------------------------------------------------
// FMovieSceneLiveLinkTransformHandler implementation.
//------------------------------------------------------------------------------


void FMovieSceneLiveLinkTransformHandler::InitializeFromExistingChannels(const UScriptStruct& InStruct)
{
	check((PropertyStorage->FloatChannel.Num() % 9) == 0);

	ElementCount = PropertyStorage->FloatChannel.Num() / 9;
	check(ElementCount > 0);

	FProperty* FoundProperty = PropertyBinding.GetProperty(InStruct);
	if (FoundProperty)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(FoundProperty))
		{
			FStructProperty* ArrayStructProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
			check(ArrayStructProperty->Struct->GetFName() == NAME_Transform);
		}
		else
		{
			FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(FoundProperty);
			check(StructProperty->Struct->GetFName() == NAME_Transform);
		}
	}
}

void FMovieSceneLiveLinkTransformHandler::FillFrame(int32 InKeyIndex, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame)
{
	int32 ChannelIndex = 0;
	for (int32 i = 0; i < ElementCount; ++i)
	{
		FTransform BuiltTransform;
		LiveLinkTransformHandlerUtils::FillTransform(InKeyIndex, ChannelIndex, PropertyStorage->FloatChannel, BuiltTransform);
		PropertyBinding.SetCurrentValueAt<FTransform>(i, InStruct, OutFrame, BuiltTransform);
	}
}

void FMovieSceneLiveLinkTransformHandler::FillFrameInterpolated(const FFrameTime& InFrameTime, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame)
{
	int32 ChannelIndex = 0;
	for (int32 i = 0; i < ElementCount; ++i)
	{
		FTransform BuiltTransform;
		LiveLinkTransformHandlerUtils::FillTransformInterpolated(InFrameTime, ChannelIndex, PropertyStorage->FloatChannel, BuiltTransform);
		PropertyBinding.SetCurrentValueAt<FTransform>(i, InStruct, OutFrame, BuiltTransform);
	}
}

void FMovieSceneLiveLinkTransformHandler::CreateChannels(const UScriptStruct& InStruct, int32 InElementCount)
{
	FProperty* Property = PropertyBinding.GetProperty(InStruct);
	check(InElementCount > 0);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		FStructProperty* ArrayInnerProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
		check(ArrayInnerProperty->Struct->GetFName() == NAME_Transform);
	}
	else
	{
		FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);
		check(StructProperty->Struct->GetFName() == NAME_Transform);
	}

	ElementCount = InElementCount;
	PropertyStorage->FloatChannel.SetNum(ElementCount * 9);
	BufferedTransformsPerElement.SetNum(ElementCount);
}

void FMovieSceneLiveLinkTransformHandler::RecordFrame(const FFrameNumber& InFrameNumber, const UScriptStruct& InStruct, const FLiveLinkBaseFrameData* InFrameData)
{
	if (InFrameData != nullptr)
	{
		for (int32 i = 0; i < ElementCount; ++i)
		{
			FTransform NewValue = PropertyBinding.GetCurrentValueAt<FTransform>(i, InStruct, InFrameData);
			BufferedTransformsPerElement[i].Add(NewValue, InFrameNumber);
		}
	}
}

void FMovieSceneLiveLinkTransformHandler::Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	FScopedSlowTask SlowTask((float)ElementCount + 1.0f, NSLOCTEXT("TakeRecorderLiveLink", "ProcessingTransforms", "Processing Transforms"));

	int32 FloatChannelIndex = 0;
	for (FLiveLinkTransformKeys& BufferedTransforms : BufferedTransformsPerElement)
	{
		check(BufferedTransforms.Times.Num() == BufferedTransforms.LocationX.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.LocationY.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.LocationZ.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.RotationX.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.RotationY.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.RotationZ.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.ScaleX.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.ScaleY.Num());
		check(BufferedTransforms.Times.Num() == BufferedTransforms.ScaleZ.Num());

		SlowTask.EnterProgressFrame();

		BufferedTransforms.AppendToFloatChannelsAndReset(FloatChannelIndex, PropertyStorage->FloatChannel);
		FloatChannelIndex += 9;
	}

	SlowTask.EnterProgressFrame();

	if (bInReduceKeys)
	{
		for (FMovieSceneFloatChannel& Channel : PropertyStorage->FloatChannel)
		{
			Channel.Optimize(InOptimizationParams);
		}
	}
	else
	{
		for (FMovieSceneFloatChannel& Channel : PropertyStorage->FloatChannel)
		{
			Channel.AutoSetTangents();
		}
	}
}

