// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Misc/FrameNumber.h"

class UScriptStruct;
struct FFrameTime;
struct FLiveLinkBaseFrameData;
struct FLiveLinkWorldTime;
struct FQualifiedFrameTime;
template <typename OptionalType> struct TOptional;


struct FKeyDataOptimizationParams;

/** Helper struct for recording properties */
template<typename PropertyType>
struct FLiveLinkPropertyKey
{
	PropertyType Value;
	FFrameNumber Time;
};

/** Interface for live link property handlers (recording and evaluating) */
class IMovieSceneLiveLinkPropertyHandler
{
public:

	virtual ~IMovieSceneLiveLinkPropertyHandler() = default;

	/**Methods used when creating the tracks and filling the channels */
	virtual void CreateChannels(const UScriptStruct& InStruct, int32 InElementCount) = 0;
	virtual void RecordFrame(const FFrameNumber& InFrameNumber, const UScriptStruct& InStruct, const FLiveLinkBaseFrameData* InFrameData) = 0;
	virtual void  Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams) = 0;
	
	/** Methods used when playing back a track */
	virtual void InitializeFromExistingChannels(const UScriptStruct& InStruct) = 0;
	virtual void FillFrame(int32 InKeyIndex, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame) = 0;
	virtual void FillFrameInterpolated(const FFrameTime& InFrameTime, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame) = 0;
};
