// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"


/** Data structure to handle buffering transform keys. Inspired from 3d Transform Track Recorder */
struct FLiveLinkTransformKeys
{
	TArray<FFrameNumber> Times;
	TArray<FMovieSceneFloatValue> LocationX, LocationY, LocationZ;
	TArray<FMovieSceneFloatValue> RotationX, RotationY, RotationZ;
	TArray<FMovieSceneFloatValue> ScaleX, ScaleY, ScaleZ;

	//returns rotation as -180 to 180.
	FVector GetNormalizedRotation(float X, float Y, float Z)
	{
		const FQuat Quat(FRotator(Y, Z, X));
		const FVector Rot = Quat.Euler();
		return Rot;
	}

	void FixEulerFlips(int32 StartIndex, TArray<FMovieSceneFloatChannel>& FloatChannels)
	{
		TArrayView<const FMovieSceneFloatValue> XRotChannel = FloatChannels[StartIndex].GetValues();
		TArrayView<const FMovieSceneFloatValue> YRotChannel = FloatChannels[StartIndex].GetValues();
		TArrayView<const FMovieSceneFloatValue> ZRotChannel = FloatChannels[StartIndex].GetValues();
		if (XRotChannel.Num() > 0)
		{
			int32 LastOne = XRotChannel.Num() - 1;
			FVector Rotation = GetNormalizedRotation(RotationX[0].Value, RotationY[0].Value, RotationZ[0].Value);
			RotationX[0].Value = Rotation.X;
			RotationY[0].Value = Rotation.Y;
			RotationZ[0].Value = Rotation.Z;
			//Due winding from last one saved with the new one normalized.
			FMath::WindRelativeAnglesDegrees(XRotChannel[LastOne].Value, RotationX[0].Value);
			FMath::WindRelativeAnglesDegrees(YRotChannel[LastOne].Value, RotationY[0].Value);
			FMath::WindRelativeAnglesDegrees(ZRotChannel[LastOne].Value, RotationZ[0].Value);
		}
		else
		{
			FVector Rotation = GetNormalizedRotation(RotationX[0].Value, RotationY[0].Value, RotationZ[0].Value);
			RotationX[0].Value = Rotation.X;
			RotationY[0].Value = Rotation.Y;
			RotationZ[0].Value = Rotation.Z;
		}
		int32 TotalCount = Times.Num();
		for (int32 RotIndex = 0; RotIndex < TotalCount - 1; RotIndex++)
		{
			FVector Rotation = GetNormalizedRotation(RotationX[RotIndex + 1].Value, RotationY[RotIndex + 1].Value, RotationZ[RotIndex + 1].Value);
			RotationX[RotIndex + 1].Value = Rotation.X;
			RotationY[RotIndex + 1].Value = Rotation.Y;
			RotationZ[RotIndex + 1].Value = Rotation.Z;
			FMath::WindRelativeAnglesDegrees(RotationX[RotIndex].Value, RotationX[RotIndex + 1].Value);
			FMath::WindRelativeAnglesDegrees(RotationY[RotIndex].Value, RotationY[RotIndex + 1].Value);
			FMath::WindRelativeAnglesDegrees(RotationZ[RotIndex].Value, RotationZ[RotIndex + 1].Value);
		}
	}

	void Add(const FTransform& InTransform, FFrameNumber InKeyTime)
	{
		Times.Add(InKeyTime);
		Add(InTransform);
	}

	void Add(const FTransform& InTransform)
	{
		FMovieSceneFloatValue NewValue(InTransform.GetTranslation().X);
		NewValue.InterpMode = RCIM_Cubic;
		LocationX.Add(NewValue);
		NewValue = FMovieSceneFloatValue(InTransform.GetTranslation().Y);
		NewValue.InterpMode = RCIM_Cubic;
		LocationY.Add(NewValue);
		NewValue = FMovieSceneFloatValue(InTransform.GetTranslation().Z);
		NewValue.InterpMode = RCIM_Cubic;
		LocationZ.Add(NewValue);

		FRotator WoundRotation = InTransform.Rotator();
		NewValue = FMovieSceneFloatValue(WoundRotation.Roll);
		NewValue.InterpMode = RCIM_Cubic;
		RotationX.Add(NewValue);

		NewValue = FMovieSceneFloatValue(WoundRotation.Pitch);
		NewValue.InterpMode = RCIM_Cubic;
		RotationY.Add(NewValue);

		NewValue = FMovieSceneFloatValue(WoundRotation.Yaw);
		NewValue.InterpMode = RCIM_Cubic;
		RotationZ.Add(NewValue);

		NewValue = FMovieSceneFloatValue(InTransform.GetScale3D().X);
		NewValue.InterpMode = RCIM_Cubic;
		ScaleX.Add(NewValue);

		NewValue = FMovieSceneFloatValue(InTransform.GetScale3D().Y);
		NewValue.InterpMode = RCIM_Cubic;
		ScaleY.Add(NewValue);

		NewValue = FMovieSceneFloatValue(InTransform.GetScale3D().Z);
		NewValue.InterpMode = RCIM_Cubic;
		ScaleZ.Add(NewValue);
	}

	//This function is the one that's called when recording live link incrementally. We move the values over from our saved 
	//Location, Rotation and Scale buffers into the specified float channels and then reset our buffers, re-using it's memory 
	//for the next iteration. We also fix any euler flips during this process, avoiding iterating over the data once again during Finalize.
	void AppendToFloatChannelsAndReset(int32 StartIndex, TArray<FMovieSceneFloatChannel>& FloatChannels)
	{
		if (Times.Num() > 0)
		{
			FloatChannels[StartIndex++].AddKeys(Times, LocationX);
			LocationX.Reset();
			FloatChannels[StartIndex++].AddKeys(Times, LocationY);
			LocationY.Reset();
			FloatChannels[StartIndex++].AddKeys(Times, LocationZ);
			LocationZ.Reset();

			FixEulerFlips(StartIndex, FloatChannels);

			FloatChannels[StartIndex++].AddKeys(Times, RotationX);
			RotationX.Reset();
			FloatChannels[StartIndex++].AddKeys(Times, RotationY);
			RotationY.Reset();
			FloatChannels[StartIndex++].AddKeys(Times, RotationZ);
			RotationZ.Reset();

			FloatChannels[StartIndex++].AddKeys(Times, ScaleX);
			ScaleX.Reset();
			FloatChannels[StartIndex++].AddKeys(Times, ScaleY);
			ScaleY.Reset();
			FloatChannels[StartIndex++].AddKeys(Times, ScaleZ);
			ScaleZ.Reset();

			Times.Reset();
		}
	}
};
