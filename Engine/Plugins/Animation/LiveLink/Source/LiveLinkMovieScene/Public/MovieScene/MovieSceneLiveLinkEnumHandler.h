// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MovieScene/IMovieSceneLiveLinkPropertyHandler.h"
#include "MovieScene/MovieSceneLiveLinkStructPropertyBindings.h"

struct FLiveLinkPropertyData;


struct FKeyDataOptimizationParams;


class FMovieSceneLiveLinkEnumHandler : public IMovieSceneLiveLinkPropertyHandler
{
public:
	FMovieSceneLiveLinkEnumHandler(const FLiveLinkStructPropertyBindings& InBinding, FLiveLinkPropertyData* PropertyStorage)
		: PropertyStorage(PropertyStorage)
		, PropertyBinding(InBinding)
		, ElementCount(0)
	{
	}

	virtual ~FMovieSceneLiveLinkEnumHandler() = default;

public:

	virtual void CreateChannels(const UScriptStruct& InStruct, int32 InElementCount) override;
	virtual void RecordFrame(const FFrameNumber& InFrameNumber, const UScriptStruct& InStruct, const FLiveLinkBaseFrameData* InFrameData) override;
	virtual void Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams) override;


	virtual void InitializeFromExistingChannels(const UScriptStruct& InStruct) override;
	virtual void FillFrame(int32 InKeyIndex, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame) override;
	virtual void FillFrameInterpolated(const FFrameTime& InFrameTime, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame) override;

protected:
	int64 GetChannelValue(int32 InKeyIndex, int32 InChannelIndex);
	int64 GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex);

protected:

	/** Channel storage for this property */
	FLiveLinkPropertyData* PropertyStorage;

	/** Binding for this property */
	FLiveLinkStructPropertyBindings PropertyBinding;

	/** Number of elements to record each frame */
	int32 ElementCount;

	/** Enum property value are interacted as int64. We store keys like this but channel used will be a byte one */
	TArray<TArray<FLiveLinkPropertyKey<int64>>> Keys;
};
