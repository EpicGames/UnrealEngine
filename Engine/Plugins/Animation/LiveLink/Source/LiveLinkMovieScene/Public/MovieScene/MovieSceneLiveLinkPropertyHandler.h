// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MovieScene/IMovieSceneLiveLinkPropertyHandler.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"
#include "MovieScene/MovieSceneLiveLinkStructPropertyBindings.h"


struct FKeyDataOptimizationParams;


template <typename PropertyType>
class FMovieSceneLiveLinkPropertyHandler : public IMovieSceneLiveLinkPropertyHandler
{
public:

	FMovieSceneLiveLinkPropertyHandler(const FLiveLinkStructPropertyBindings& InBinding , FLiveLinkPropertyData* InOutPropertyStorage)
		: PropertyStorage(InOutPropertyStorage)
		, PropertyBinding(InBinding)
		, ElementCount(0)
	{
		PropertyStorage->PropertyName = InBinding.GetPropertyName();
	}

	virtual ~FMovieSceneLiveLinkPropertyHandler() = default;

public:

	virtual void CreateChannels(const UScriptStruct& InStruct, int32 InElementCount) override;
	virtual void RecordFrame(const FFrameNumber& InFrameNumber, const UScriptStruct& InStruct, const FLiveLinkBaseFrameData* InFrameData) override
	{
		if (InFrameData != nullptr)
		{
			for (int32 i = 0; i < ElementCount; ++i)
			{
				PropertyType NewValue = PropertyBinding.GetCurrentValueAt<PropertyType>(i, InStruct, InFrameData);

				FLiveLinkPropertyKey<PropertyType> Key;
				Key.Time = InFrameNumber;
				Key.Value = NewValue;
				Keys[i].Add(Key);
			}
		}
	}

	virtual void Finalize(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams) override;

	virtual void InitializeFromExistingChannels(const UScriptStruct& InStruct) override;
	virtual void FillFrame(int32 InKeyIndex, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame) override
	{
		for (int32 i = 0; i < ElementCount; ++i)
		{
			const PropertyType Value = GetChannelValue(InKeyIndex, i);
			PropertyBinding.SetCurrentValueAt<PropertyType>(i, InStruct, OutFrame, Value);
		}
	}

	virtual void FillFrameInterpolated(const FFrameTime& InFrameTime, const FLiveLinkWorldTime& InWorldTime, const TOptional<FQualifiedFrameTime>& InTimecodeTime, const UScriptStruct& InStruct, FLiveLinkBaseFrameData* OutFrame) override
	{
		for (int32 i = 0; i < ElementCount; ++i)
		{
			const PropertyType Value = GetChannelValueInterpolated(InFrameTime, i);
			PropertyBinding.SetCurrentValueAt<PropertyType>(i, InStruct, OutFrame, Value);
		}
	}

protected:
	PropertyType GetChannelValue(int32 InKeyIndex, int32 InChannelIndex);
	PropertyType GetChannelValueInterpolated(const FFrameTime& InFrameTime, int32 InChannelIndex);

protected:

	/** Channel storage for this property */
	FLiveLinkPropertyData* PropertyStorage;

	/** Binding for this property */
	FLiveLinkStructPropertyBindings PropertyBinding;

	/** Number of elements to record each frame */
	int32 ElementCount;
	
	/** The keys that are being recorded */
	TArray<TArray<FLiveLinkPropertyKey<PropertyType>>> Keys;
};
