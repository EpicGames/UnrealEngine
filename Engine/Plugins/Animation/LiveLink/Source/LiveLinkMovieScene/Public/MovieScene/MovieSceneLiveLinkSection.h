// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"

#include "LiveLinkPresetTypes.h"

#include "MovieSceneLiveLinkSection.generated.h"

#define UE_API LIVELINKMOVIESCENE_API

struct FMovieSceneEvalTemplatePtr;
struct FMovieSceneFloatChannel;

class UMovieScenePropertyTrack;
struct FKeyDataOptimizationParams;
struct FLiveLinkSubjectPreset;
class UMovieSceneLiveLinkSubSection;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/**
* A movie scene section for all live link recorded data
*/
UCLASS(MinimalAPI)
class UMovieSceneLiveLinkSection : public UMovieSceneSection
{
	GENERATED_BODY()

public:

	UE_API UMovieSceneLiveLinkSection(const FObjectInitializer& ObjectInitializer);

	UE_API void Initialize(const FLiveLinkSubjectPreset& SubjectPreset, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData);
	void SetSubjectName(const FName& InSubjectName) { SubjectPreset.Key.SubjectName = InSubjectName; }

public:
	
	/**
	 * Called when first created. Creates the channels required to represent this section
	 */
	UE_API int32 CreateChannelProxy();

	/**
	 * Called during loading. 
	 */
	UE_API void UpdateChannelProxy();

	UE_API void SetMask(const TArray<bool>& InChannelMask);

	UE_API void RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData);
	UE_API void FinalizeSection(bool bReduceKeys, const FKeyDataOptimizationParams& OptimizationParams);

public:
	static UE_API TArray<TSubclassOf<UMovieSceneLiveLinkSection>> GetMovieSectionForRole(const TSubclassOf<ULiveLinkRole>& RoleToSupport);

	UE_API virtual FMovieSceneEvalTemplatePtr CreateSectionTemplate(const UMovieScenePropertyTrack& InTrack) const;

public:
	//~ Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;

#if WITH_EDITOR
	UE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	UE_API virtual void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;
#endif
	//~ End UObject interface

protected:

	UE_API virtual int32 GetChannelCount() const;

private:
	UE_API void ConvertPreRoleData();

public:

	UPROPERTY()
	FLiveLinkSubjectPreset SubjectPreset;

	// Channels that we may not send to live link or they are sent but not priority (MattH to do).
	UPROPERTY()
	TArray<bool> ChannelMask;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneLiveLinkSubSection>> SubSections;

	TSharedPtr<FLiveLinkStaticDataStruct> StaticData;

	UPROPERTY()
	FName SubjectName_DEPRECATED;
	UPROPERTY()
	FLiveLinkFrameData TemplateToPush_DEPRECATED;
	UPROPERTY()
	FLiveLinkRefSkeleton RefSkeleton_DEPRECATED;
	UPROPERTY()
	TArray<FName> CurveNames_DEPRECATED;
	UPROPERTY()
	TArray <FMovieSceneFloatChannel> PropertyFloatChannels_DEPRECATED;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
