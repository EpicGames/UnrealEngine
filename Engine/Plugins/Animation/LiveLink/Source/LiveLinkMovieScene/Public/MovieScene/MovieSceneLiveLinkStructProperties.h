// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneLiveLinkStructProperties.generated.h"

struct FLiveLinkPropertyData;
struct FMovieSceneBoolChannel;
struct FMovieSceneByteChannel;
struct FMovieSceneFloatChannel;
struct FMovieSceneIntegerChannel;
struct FMovieSceneStringChannel;

class IMovieSceneLiveLinkPropertyHandler;

namespace LiveLinkPropertiesUtils
{
	LIVELINKMOVIESCENE_API TSharedPtr< IMovieSceneLiveLinkPropertyHandler> CreatePropertyHandler(const UScriptStruct& InStruct, FLiveLinkPropertyData* InPropertyData);
}

USTRUCT()
struct FLiveLinkPropertyData
{
	GENERATED_BODY()

public:
	LIVELINKMOVIESCENE_API FLiveLinkPropertyData();
	LIVELINKMOVIESCENE_API ~FLiveLinkPropertyData();

	UPROPERTY()
	FName PropertyName;

	UPROPERTY()
	TArray<FMovieSceneFloatChannel> FloatChannel;

	UPROPERTY()
	TArray<FMovieSceneStringChannel> StringChannel;

	UPROPERTY()
	TArray<FMovieSceneIntegerChannel> IntegerChannel;

	UPROPERTY()
	TArray<FMovieSceneBoolChannel> BoolChannel;

	UPROPERTY()
	TArray<FMovieSceneByteChannel> ByteChannel;

	int32 GetChannelCount() const
	{
		return FloatChannel.Num() + StringChannel.Num() + IntegerChannel.Num() + BoolChannel.Num() + ByteChannel.Num();
	}
};

USTRUCT()
struct FLiveLinkSubSectionData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FLiveLinkPropertyData> Properties;

	/** Custom serializer that compresses property data before serializing it when doing persistent serialization. */
	bool Serialize(FArchive& Archive);
};


template<>
struct TStructOpsTypeTraits<FLiveLinkSubSectionData> : public TStructOpsTypeTraitsBase2<FLiveLinkSubSectionData>
{
	enum
	{
		WithSerializer = true
	};
};
