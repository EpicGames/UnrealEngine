// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

#define UE_API LIVELINKMOVIESCENE_API


/**
 * Manages bindings to keyed properties of LiveLink UStructs.
 */
class FLiveLinkStructPropertyBindings
{
public:
	UE_API FLiveLinkStructPropertyBindings(FName InPropertyName, const FString& InPropertyPath);

	/**
	 * Rebuilds the property mappings for a specific UStruct, and adds them to the cache
	 *
	 * @param InContainer The type to cache property for
	 */
	UE_API void CacheBinding(const UScriptStruct& InStruct);

	/**
	 * Gets the FProperty that is bound to the container
	 *
	 * @param InContainer	The Struct that owns the property
	 * @return				The property on the Struct if it exists
	 */
	UE_API FProperty* GetProperty(const UScriptStruct& InStruct) const;

	/**
	 * Gets the current value of a property on a UStruct
	 *
	 * @param InStruct	The struct to get the property from
	 * @param InSourceAddress	The source address of the struct instance
	 * @return ValueType	The current value
	 */
	template <typename ValueType>
	UE_DEPRECATED(4.24, "This function is deprecated. Use a GetCurrentValueAt with InIndex set to 0 for the same result.")
	ValueType GetCurrentValue(const UScriptStruct& InStruct, const void* InSourceAddress)
	{
		return GetCurrentValueAt<ValueType>(0, InStruct, InSourceAddress);
	}

	/**
	 * Gets the current value of a property at desired Index. Must be on ArrayProperty
	 *
	 * @param InIndex	The index of the desired value in the ArrayProperty
	 * @param InStruct	The struct to get the property from
	 * @param InSourceAddress	The source address of the struct instance
	 * @return ValueType	The current value
	 */
	template <typename ValueType>
	ValueType GetCurrentValueAt(int32 InIndex, const UScriptStruct& InStruct, const void* InSourceAddress)
	{
		FPropertyWrapper FoundProperty = FindOrAdd(InStruct);

		if (FProperty* Property = FoundProperty.GetProperty())
		{
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				const ValueType* BaseAddr = FoundProperty.GetPropertyAddress<ValueType>(InSourceAddress, 0);
				FScriptArrayHelper ArrayHelper(ArrayProperty, BaseAddr);
				ArrayHelper.ExpandForIndex(InIndex);
				const ValueType* ValuePtr = reinterpret_cast<const ValueType*>(ArrayHelper.GetRawPtr(InIndex));
				return ValuePtr ? *ValuePtr : ValueType();
			}
			else
			{
				checkSlow(InIndex >= 0 && InIndex < Property->ArrayDim);
				const ValueType* BaseAddr = FoundProperty.GetPropertyAddress<ValueType>(InSourceAddress, InIndex);
				return BaseAddr ? *BaseAddr : ValueType();
			}
		}
		return ValueType();
	}

	/**
	 * Gets the current value of an enum property on a struct
	 *
	 * @param InStruct	The struct to get the property from
	 * @param InSourceAddress	The address of the instanced struct
	 * @return ValueType	The current value
	 */
	UE_DEPRECATED(4.24, "This function is deprecated. Use a GetCurrentValueForEnumAt with InIndex set to 0 for the same result.")
	int64 GetCurrentValueForEnum(const UScriptStruct& InStruct, const void* InSourceAddress)
	{
		return GetCurrentValueForEnumAt(0, InStruct, InSourceAddress);
	}

	/**
	 * Gets the current value of an enum property at desired index
	 *
	 * @param InIndex	The index of the desired value in the ArrayProperty
	 * @param InStruct	The struct to get the property from
	 * @param InSourceAddress	The address of the instanced struct
	 * @return ValueType	The current value
	 */
	UE_API int64 GetCurrentValueForEnumAt(int32 InIndex, const UScriptStruct& InStruct, const void* InSourceAddress);


	/**
	 * Sets the current value of a property on an instance of a UStruct
	 *
	 * @param InStruct	The struct to set the property on
	 * @param InSourceAddress	The address of the instanced struct
	 * @param InValue   The value to set
	 */
	template <typename ValueType>
	UE_DEPRECATED(4.24, "This function is deprecated. Use a SetCurrentValueAt with InIndex set to 0 for the same result.")
	void SetCurrentValue(const UScriptStruct& InStruct, void* InSourceAddress, typename TCallTraits<ValueType>::ParamType InValue)
	{
		SetCurrentValueAt<ValueType>(0, InStruct, InSourceAddress, InValue);
	}

	/**
	 * Sets the current value of a property on an instance of a UStruct
	 *
	 * @param InIndex	The index in the array property to set the value on
	 * @param InStruct	The struct to set the property on
	 * @param InSourceAddress	The address of the instanced struct
	 * @param InValue   The value to set
	 */
	template <typename ValueType>
	void SetCurrentValueAt(int32 InIndex, const UScriptStruct& InStruct, void* InSourceAddress, typename TCallTraits<ValueType>::ParamType InValue)
	{
		FPropertyWrapper FoundProperty = FindOrAdd(InStruct);

		if (FProperty* Property = FoundProperty.GetProperty())
		{
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				ValueType* BaseAddr = FoundProperty.GetPropertyAddress<ValueType>(InSourceAddress, 0);
				FScriptArrayHelper ArrayHelper(ArrayProperty, BaseAddr);
				ArrayHelper.ExpandForIndex(InIndex);
				ValueType* ValuePtr = reinterpret_cast<ValueType*>(ArrayHelper.GetRawPtr(InIndex));
				if (ValuePtr)
				{
					*ValuePtr = InValue;
				}
			}
			else
			{
				checkSlow(InIndex >= 0 && InIndex < Property->ArrayDim);
				ValueType* BaseAddr = FoundProperty.GetPropertyAddress<ValueType>(InSourceAddress, InIndex);
				if (BaseAddr)
				{
					*BaseAddr = InValue;
				}
			}
		}
	}

	/**
	 * Sets the current value of an Enum property on an instance of a UStruct
	 *
	 * @param InStruct	The struct to set the property on
	 * @param InSourceAddress	The address of the instanced struct
	 * @param InValue   The value to set
	 */
	UE_DEPRECATED(4.24, "This function is deprecated. Use a SetCurrentValueForEnumAt with InIndex set to 0 for the same result.")
	void SetCurrentValueForEnum(const UScriptStruct& InStruct, void* InSourceAddress, int64 InValue)
	{
		SetCurrentValueForEnumAt(0, InStruct, InSourceAddress, InValue);
	}

	/**
	 * Sets the current value of an Enum property on an instance of a UStruct to a certain Index
	 *
	 * @param InIndex, Index where to write the value
	 * @param InStruct	The struct to set the property on
	 * @param InSourceAddress	The address of the instanced struct
	 * @param InValue   The value to set
	 */
	UE_API void SetCurrentValueForEnumAt(int32 InIndex, const UScriptStruct& InStruct, void* InSourceAddress, int64 InValue);

	/** @return the property path that this binding was initialized from */
	const FString& GetPropertyPath() const
	{
		return PropertyPath;
	}

	/** @return the property name that this binding was initialized from */
	const FName& GetPropertyName() const
	{
		return PropertyName;
	}

private:
	struct FPropertyNameKey
	{
		FPropertyNameKey(FName InStructName, FName InPropertyName)
			: StructName(InStructName), PropertyName(InPropertyName)
		{}

		bool operator==(const FPropertyNameKey& Other) const
		{
			return (StructName == Other.StructName) && (PropertyName == Other.PropertyName);
		}

		friend uint32 GetTypeHash(const FPropertyNameKey& Key)
		{
			return HashCombine(GetTypeHash(Key.StructName), GetTypeHash(Key.PropertyName));
		}

		FName StructName;
		FName PropertyName;
	};

	struct FPropertyWrapper
	{
		TWeakFieldPtr<FProperty> Property;
		int64 DeltaAddress;

		FProperty* GetProperty() const
		{
			return Property.Get();
		}

		template<typename ValueType>
		ValueType* GetPropertyAddress(void* BaseContainerAddress, int32 Index) const
		{
			FProperty* PropertyPtr = GetProperty();
			const PTRINT NewAddress = (PTRINT)BaseContainerAddress + DeltaAddress;
			return PropertyPtr ? PropertyPtr->ContainerPtrToValuePtr<ValueType>((void*)NewAddress, Index) : nullptr;
		}

		template<typename ValueType>
		const ValueType* GetPropertyAddress(const void* BaseContainerAddress, int32 Index) const
		{
			FProperty* PropertyPtr = GetProperty();
			const PTRINT NewAddress = (PTRINT)BaseContainerAddress + DeltaAddress;
			return PropertyPtr ? PropertyPtr->ContainerPtrToValuePtr<const ValueType>((void*)NewAddress, Index) : nullptr;
		}
		
		FPropertyWrapper()
			: Property(nullptr)
			, DeltaAddress(0)
		{}
	};

	static UE_API FPropertyWrapper FindPropertyRecursive(const UScriptStruct* InStruct, TArray<FString>& InPropertyNames, uint32 Index, void* ContainerAddress, int32 PreviousDelta);
	static UE_API FPropertyWrapper FindProperty(const UScriptStruct& InStruct, const FString& InPropertyName);

	/** Find or add the FPropertyWrapper for the specified struct */
	FPropertyWrapper FindOrAdd(const UScriptStruct& InStruct)
	{
		FPropertyNameKey Key(InStruct.GetFName(), PropertyName);

		const FPropertyWrapper* PropertyWrapper = PropertyCache.Find(Key);
		if (PropertyWrapper)
		{
			return *PropertyWrapper;
		}

		CacheBinding(InStruct);
		return PropertyCache.FindRef(Key);
	}

private:
	/** Mapping of UStructs property */
	static UE_API TMap<FPropertyNameKey, FPropertyWrapper> PropertyCache;

	/** Path to the property we are bound to */
	FString PropertyPath;

	/** Actual name of the property we are bound to */
	FName PropertyName;
};

/** Explicit specializations for bools */
template<>
UE_DEPRECATED(4.24, "This function is deprecated. Use a GetCurrentValueAt with InIndex set to 0 for the same result.")
LIVELINKMOVIESCENE_API bool FLiveLinkStructPropertyBindings::GetCurrentValue<bool>(const UScriptStruct& InStruct, const void* InSourceAddress);

template<>
UE_DEPRECATED(4.24, "This function is deprecated. Use a SetCurrentValueAt with InIndex set to 0 for the same result.")
LIVELINKMOVIESCENE_API void FLiveLinkStructPropertyBindings::SetCurrentValue<bool>(const UScriptStruct& InStruct, void* InSourceAddress, TCallTraits<bool>::ParamType InValue);


template<> LIVELINKMOVIESCENE_API bool FLiveLinkStructPropertyBindings::GetCurrentValueAt<bool>(int32 InIndex, const UScriptStruct& InStruct, const void* InSourceAddress);
template<> LIVELINKMOVIESCENE_API void FLiveLinkStructPropertyBindings::SetCurrentValueAt<bool>(int32 InIndex, const UScriptStruct& InStruct, void* InSourceAddress, TCallTraits<bool>::ParamType InValue);

#undef UE_API
