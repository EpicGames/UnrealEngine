// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Channels/MovieSceneChannelProxy.h" // IWYU pragma: keep
#include "LiveLinkTypes.h"
#include "MovieSceneLiveLinkStructProperties.h"
#include "Templates/SubclassOf.h"
#include "MovieSceneLiveLinkSubSection.generated.h"

#define UE_API LIVELINKMOVIESCENE_API

class ULiveLinkRole;
struct FMovieSceneChannelProxyData;

class UMovieScenePropertyTrack;
struct FKeyDataOptimizationParams;

namespace MovieSceneLiveLinkSectionUtils
{
#if WITH_EDITOR
	/**
	 * Add a channel with editor data filled out. Channel mask value will be used to enable / disable the channel
     */
	template<typename ChannelType, typename ExtendedEditorDataType>
	void CreateChannelEditor(const FText& InDisplayName, ChannelType& InChannel, int32 InChannelIndex, ExtendedEditorDataType&& InExtendedEditorDataType, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData)
	{
		FMovieSceneChannelMetaData ChannelEditorData(FName(*(InDisplayName.ToString())), InDisplayName);
		ChannelEditorData.SortOrder = InChannelIndex;
		ChannelEditorData.bCanCollapseToTrack = false;
		ChannelEditorData.bEnabled = OutChannelMask[InChannelIndex];
		OutChannelData.Add(InChannel, ChannelEditorData, InExtendedEditorDataType);
	}
#endif
}

/**
 * Base class to manage recording live link data structure properties
 * If user specifics data need to be managed in a certain way, create your own sub section
 */
UCLASS(MinimalAPI, Abstract)
class UMovieSceneLiveLinkSubSection : public UObject
{
	GENERATED_BODY()

public:

	UE_API UMovieSceneLiveLinkSubSection(const FObjectInitializer& ObjectInitializer);
	

public:

	/** 
	 * Called when creating the section for the first time. Will setup the subject role and the static data
	 * Should create the channels associated with the subsection
	 */
	UE_API virtual void Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData);

	/**
	 * Links the channels of the subsection to the sections's channels proxy
	 */
	virtual int32 CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData) { return 0; }

	/**
	 * Adds keyframe to channels associated to sub section from the incoming frame data
	 */
	virtual void RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData)  {};

	/**
	 * Wraps up the sub section's channels. May optimize keyframes if requested
	 */
	virtual void FinalizeSection(bool bReduceKeys, const FKeyDataOptimizationParams& OptimizationParams)  {};

	/**
	 * Gets the number of channels associated with this sub section
	 */
	UE_API virtual int32 GetChannelCount() const;

	/**
	 * Return true if this sub section can manage the specified role
	 */
	virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const { return false; }
	static UE_API TArray<TSubclassOf<UMovieSceneLiveLinkSubSection>> GetLiveLinkSubSectionForRole(const TSubclassOf<ULiveLinkRole>& RoleToSupport);

	UE_API void SetStaticData(const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData);

	UE_API FLiveLinkPropertyData* GetPropertyData(int32 InPropertyIndex);

public:
	//~ UObject interface
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

public:

	/** Data associated to properties managed by this sub section*/
	UPROPERTY()
	FLiveLinkSubSectionData SubSectionData;

	UPROPERTY()
	TSubclassOf<ULiveLinkRole> SubjectRole;

protected:
	TSharedPtr<FLiveLinkStaticDataStruct> StaticData;
};

#undef UE_API
