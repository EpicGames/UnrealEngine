// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneLiveLinkSubSection.h"


#include "MovieSceneLiveLinkSubSectionAnimation.generated.h"

#define UE_API LIVELINKMOVIESCENE_API

class FMovieSceneLiveLinkTransformHandler;


/**
 * A LiveLinkSubSection managing array of transforms contained in the Animation Frame Data structure
 */
UCLASS(MinimalAPI)
class UMovieSceneLiveLinkSubSectionAnimation : public UMovieSceneLiveLinkSubSection
{
	GENERATED_BODY()

public:

	UE_API UMovieSceneLiveLinkSubSectionAnimation(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData);
	UE_API virtual int32 CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData);
	UE_API virtual void RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData) override;
	UE_API virtual void FinalizeSection(bool bReduceKeys, const FKeyDataOptimizationParams& OptimizationParams) override;

public:

	UE_API virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const override;

private:
	void CreatePropertiesChannel();

protected:
	
	/** Helper struct to manage filling channels from transforms */
	TSharedPtr<FMovieSceneLiveLinkTransformHandler> TransformHandler;
};

#undef UE_API
