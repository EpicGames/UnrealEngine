// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneLiveLinkSubSection.h"


#include "MovieSceneLiveLinkSubSectionBasicRole.generated.h"

#define UE_API LIVELINKMOVIESCENE_API


/**
 * A LiveLinkSubSection managing special properties of the BasicRole
 */
UCLASS(MinimalAPI)
class UMovieSceneLiveLinkSubSectionBasicRole : public UMovieSceneLiveLinkSubSection
{
	GENERATED_BODY()

public:

	UE_API UMovieSceneLiveLinkSubSectionBasicRole(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData) override;
	UE_API virtual int32 CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData) override;
	UE_API virtual void RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData) override;
	UE_API virtual void FinalizeSection(bool bReduceKeys, const FKeyDataOptimizationParams& OptimizationParams) override;

public:

	UE_API virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const override;

private:
	void CreatePropertiesChannel();

protected:

	/** Helper struct to manage filling channels from property array */
	TSharedPtr<IMovieSceneLiveLinkPropertyHandler> PropertyHandler;
};

#undef UE_API
