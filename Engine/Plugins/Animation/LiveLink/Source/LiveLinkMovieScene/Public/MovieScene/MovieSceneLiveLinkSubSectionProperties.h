// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneLiveLinkSubSection.h"


#include "MovieSceneLiveLinkSubSectionProperties.generated.h"

#define UE_API LIVELINKMOVIESCENE_API


/**
 * A LiveLinkSubSection managing properties marked as Interp in the data struct associated with the subject role
 */
UCLASS(MinimalAPI)
class UMovieSceneLiveLinkSubSectionProperties : public UMovieSceneLiveLinkSubSection
{
	GENERATED_BODY()

public:

	UE_API UMovieSceneLiveLinkSubSectionProperties(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData) override;
	UE_API virtual int32 CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData) override;
	UE_API virtual void RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData) override;
	UE_API virtual void FinalizeSection(bool bReduceKeys, const FKeyDataOptimizationParams& OptimizationParams) override;

public:

	UE_API virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const override;

protected:

private:
	void CreatePropertyList(UScriptStruct* InScriptStruct, bool bCheckInterpFlag, const FString& InOwner);
	void CreatePropertiesChannel(UScriptStruct* InScriptStruct);
	bool IsPropertyTypeSupported(const FProperty* InProperty) const;
	int32 CreateChannelProxyInternal(FProperty* InPropertyPtr, FLiveLinkPropertyData& OutPropertyData, int32 InPropertyIndex, int32 GlobalIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData, const FText& InPropertyName);

protected:

	/** Helper struct to manage filling channels for each properties */
	TArray<TSharedPtr<IMovieSceneLiveLinkPropertyHandler>> PropertyHandlers;
};

#undef UE_API
