// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieScenePropertyTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneLiveLinkTrack.generated.h"

class ULiveLinkRole;

/**
* A track for animating FMoveSceneLiveLinkTrack properties.
*/
UCLASS(MinimalAPI)
class UMovieSceneLiveLinkTrack : public UMovieScenePropertyTrack, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:

	UMovieSceneLiveLinkTrack(const FObjectInitializer& ObjectInitializer);

	void SetTrackRole(const TSubclassOf<ULiveLinkRole>& InTrackRole) { TrackRole = InTrackRole; }
	TSubclassOf<ULiveLinkRole> GetTrackRole() const { return TrackRole; }

	// UMovieSceneTrack interface
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	virtual void PostCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const override;

#if WITH_EDITORONLY_DATA
	virtual bool CanRename() const override { return true; }

	virtual void SetDisplayName(const FText& NewDisplayName) override;
#endif

protected:

	UPROPERTY()
	TSubclassOf<ULiveLinkRole> TrackRole;
};