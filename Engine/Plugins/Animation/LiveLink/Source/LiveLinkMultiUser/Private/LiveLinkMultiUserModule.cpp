// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMultiUserModule.h"

#include "IConcertClientTransactionBridge.h"
#include "IConcertSyncClientModule.h"
#include "LiveLinkControllerBase.h"
#include "UObject/Package.h"


namespace LiveLinkMultiUserUtils
{
	ETransactionFilterResult HandleTransactionFiltering(const FConcertTransactionFilterArgs& FilterArgs)
	{
		const UObject* ObjectToFilter = FilterArgs.ObjectToFilter;
		return ObjectToFilter && ObjectToFilter->IsA<ULiveLinkControllerBase>()
			? ETransactionFilterResult::IncludeObject
			: ETransactionFilterResult::UseDefault;
	}
}

void FLiveLinkMultiUserModule::StartupModule()
{
	if (IConcertSyncClientModule::IsAvailable())
	{
		IConcertClientTransactionBridge& TransactionBridge = IConcertSyncClientModule::Get().GetTransactionBridge();
		TransactionBridge.RegisterTransactionFilter( TEXT("LiveLinkTransactionFilter"), FOnFilterTransactionDelegate::CreateStatic(&LiveLinkMultiUserUtils::HandleTransactionFiltering));
	}
}

void FLiveLinkMultiUserModule::ShutdownModule()
{
	if (IConcertSyncClientModule::IsAvailable())
	{
		IConcertClientTransactionBridge& TransactionBridge = IConcertSyncClientModule::Get().GetTransactionBridge();
		TransactionBridge.UnregisterTransactionFilter( TEXT("LiveLinkTransactionFilter"));
	}	
}

IMPLEMENT_MODULE(FLiveLinkMultiUserModule, LiveLinkMultiUser);
