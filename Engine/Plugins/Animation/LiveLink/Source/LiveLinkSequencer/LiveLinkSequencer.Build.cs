// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkSequencer : ModuleRules
	{
		public LiveLinkSequencer(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"LiveLinkComponents",
					"LiveLinkInterface",
					"LiveLinkMovieScene",
					"MovieScene",
					"TakeTrackRecorders",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"HeadMountedDisplay",
					"LevelSequence",
					"LiveLink",
					"MovieSceneTools",
					"MovieSceneTracks",
					"Projects",
					"Sequencer",
					"SequenceRecorder",
					"SerializedRecorderInterface",
					"Slate",
					"SlateCore",
					"TakesCore",
					"TakeMovieScene",
					"TakeRecorderSources",
					"TakeRecorder",
					"TargetPlatform",
					"EditorFramework",
					"UnrealEd",
                }
			); 
		}
	}
}
