// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPluginManager.h"

#include "Brushes/SlateImageBrush.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "ISequencerModule.h"
#include "ITakeRecorderModule.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "LiveLinkSequencerPrivate.h"
#include "MovieSceneLiveLinkControllerMapTrackRecorder.h"
#include "Sequencer/LiveLinkPropertyTrackEditor.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "TakeRecorderSource/TakeRecorderLiveLinkSource.h"
#include "TakeRecorderSources.h"

LLM_DEFINE_TAG(LiveLink_LiveLinkSequencer);
DEFINE_LOG_CATEGORY(LogLiveLinkSequencer);


/**
 * Implements the LiveLink Sequencer module.
 */

#define LOCTEXT_NAMESPACE "LiveLinkSequencerModule"

static const FName TakeRecorderModuleName(TEXT("TakeRecorder"));
static const FName MovieSceneSectionRecorderFactoryName(TEXT("MovieSceneSectionRecorderFactory"));
static const FName MovieSceneTrackRecorderFactoryName("MovieSceneTrackRecorderFactory");
static TArray<TSubclassOf<ULiveLinkRole>> SupportedRecordingRoles;


namespace LiveLinkSequencerModuleUtils
{
	FString InPluginContent(const FString& RelativePath, const ANSICHAR* Extension)
	{
		static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("LiveLink"))->GetContentDir();
		return (ContentDir / RelativePath) + Extension;
	}
}

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( LiveLinkSequencerModuleUtils::InPluginContent( RelativePath, ".png" ), __VA_ARGS__ )


class FLiveLinkSequencerModule : public IModuleInterface
{
public:
	TSharedPtr<FSlateStyleSet> StyleSet;

	TSharedPtr< class ISlateStyle > GetStyleSet() { return StyleSet; }

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(LiveLink_LiveLinkSequencer);

		static FName LiveLinkSequencerStyle(TEXT("LiveLinkSequencerStyle"));
		StyleSet = MakeShared<FSlateStyleSet>(LiveLinkSequencerStyle);

		if (FModuleManager::Get().IsModuleLoaded(TakeRecorderModuleName))
		{
			RegisterTakeRecorderSourceMenuExtender();
		}

		ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FLiveLinkSequencerModule::ModulesChangesCallback);

		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		StyleSet->Set("ClassIcon.TakeRecorderLiveLinkSource", new IMAGE_PLUGIN_BRUSH(TEXT("TakeRecorderLiveLinkSource_16x"), Icon16x16));
		StyleSet->Set("ClassThumbnail.TakeRecorderLiveLinkSource", new IMAGE_PLUGIN_BRUSH(TEXT("TakeRecorderLiveLinkSource_64x"), Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		CreateLiveLinkPropertyTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FLiveLinkPropertyTrackEditor::CreateTrackEditor));

		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &LiveLinkControllerMapTrackRecorderFactory);
	}

	void ModulesChangesCallback(FName ModuleName, EModuleChangeReason ReasonForChange)
	{
		if (ReasonForChange == EModuleChangeReason::ModuleLoaded && ModuleName == TakeRecorderModuleName)
		{
			RegisterTakeRecorderSourceMenuExtender();
		}
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(LiveLink_LiveLinkSequencer);

		UnregisterTakeRecorderSourceMenuExtender();

		FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);

		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule != nullptr)
		{
			SequencerModule->UnRegisterTrackEditor(CreateLiveLinkPropertyTrackEditorHandle);
		}

		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &LiveLinkControllerMapTrackRecorderFactory);
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

private:

	void RegisterTakeRecorderSourceMenuExtender()
	{
		if (FModuleManager::Get().IsModuleLoaded(TakeRecorderModuleName))
		{
			ITakeRecorderModule& TakeRecorderModule = FModuleManager::Get().LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
			SourcesMenuExtension = TakeRecorderModule.RegisterSourcesMenuExtension(FOnExtendSourcesMenu::CreateStatic(ExtendSourcesMenu));
		}
	}

	void UnregisterTakeRecorderSourceMenuExtender()
	{
		if (ITakeRecorderModule* TakeRecorderModule = FModuleManager::Get().GetModulePtr<ITakeRecorderModule>("TakeRecorder"))
		{
			TakeRecorderModule->UnregisterSourcesMenuExtension(SourcesMenuExtension);
		}
	}

	static void ExtendSourcesMenu(TSharedRef<FExtender> Extender, UTakeRecorderSources* Sources)
	{
		Extender->AddMenuExtension("Sources", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateStatic(PopulateSourcesMenu, Sources));
	}

	static void PopulateSourcesMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources)
	{
		FName ExtensionName = "LiveLinkSourceSubMenu";

		MenuBuilder.AddSubMenu(
			NSLOCTEXT("TakeRecorderSources", "LiveLinkList_Label", "From LiveLink"),
			NSLOCTEXT("TakeRecorderSources", "LiveLinkList_Tip", "Add a new recording source from a Live Link Subject"),
			FNewMenuDelegate::CreateStatic(PopulateLiveLinkSubMenu, Sources),
			FUIAction(),
			ExtensionName,
			EUserInterfaceActionType::Button
		);
	}

	static void PopulateLiveLinkSubMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources)
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
			
			const bool bIncludeDisabledSubjects = false;
			const bool bIncludeVirtualSubjects = true;
			TArray<FLiveLinkSubjectKey> Subjects = LiveLinkClient->GetSubjects(bIncludeDisabledSubjects, bIncludeVirtualSubjects);
			for (const FLiveLinkSubjectKey& SubjectKey : Subjects)
			{
				MenuBuilder.AddMenuEntry(FText::FromName(SubjectKey.SubjectName)
					, FText()
					, FSlateIcon()
					, FExecuteAction::CreateLambda([Sources, SubjectKey]
					{
						AddLiveLinkSource(Sources, SubjectKey.SubjectName);
					})
				);
			}
		}
	}

	static void AddLiveLinkSource(UTakeRecorderSources* Sources,  const FName& SubjectName)
	{
		FScopedTransaction Transaction(LOCTEXT("AddLiveLinkSource","Add Live Link Source"));

		Sources->Modify();

		UTakeRecorderLiveLinkSource* NewSource = Sources->AddSource<UTakeRecorderLiveLinkSource>();
		NewSource->SubjectName = SubjectName;
	}

private:

	FDelegateHandle ModulesChangedHandle;
	FDelegateHandle CreateLiveLinkPropertyTrackEditorHandle;
	FDelegateHandle SourcesMenuExtension;

	FMovieSceneLiveLinkControllerMapTrackRecorderFactory LiveLinkControllerMapTrackRecorderFactory;
};

IMPLEMENT_MODULE(FLiveLinkSequencerModule, LiveLinkSequencer);

#undef LOCTEXT_NAMESPACE
