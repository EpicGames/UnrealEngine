// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSequencerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkSequencerSettings)

FName ULiveLinkSequencerSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText ULiveLinkSequencerSettings::GetSectionText() const
{
	return NSLOCTEXT("LiveLinkSequenceEditorSettings", "LiveLinkSequenceEditorSettingsSection", "Live Link Sequence Editor");
}

FName ULiveLinkSequencerSettings::GetSectionName() const
{
	return TEXT("Live Link Sequence Editor");
}

#endif

