// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPropertyTrackEditor.h"

#include "ISequencerSection.h"
#include "MovieScene/MovieSceneLiveLinkSection.h"
#include "Styling/SlateIconFinder.h"
#include "LevelSequence.h"

#include "LiveLinkComponent.h"

/**
* An implementation of live link property sections.
*/
class FLiveLinkSection : public FSequencerSection
{
public:

	/**
	* Creates a new Live Link section.
	*
	* @param InSection The section object which is being displayed and edited.
	* @param InSequencer The sequencer which is controlling this section.
	*/
	FLiveLinkSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
		: FSequencerSection(InSection), WeakSequencer(InSequencer)
	{
	}

public:
	
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding) override;
	virtual bool RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath) override;
protected:

	/** The sequencer which is controlling this section. */
	TWeakPtr<ISequencer> WeakSequencer;

};


#define LOCTEXT_NAMESPACE "FLiveLinkSection"

void FLiveLinkSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding)
{
	auto MakeUIAction = [this](int32 Index)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([this, Index]
				{
					const TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
					if (!SequencerPtr.IsValid())
					{
						return;
					}

					UMovieSceneLiveLinkSection* LiveLinkSection = CastChecked<UMovieSceneLiveLinkSection>(WeakSection.Get());
					
					FScopedTransaction Transaction(LOCTEXT("SetLiveLinkActiveChannelsTransaction", "Set Live LinkActive Channels"));
					LiveLinkSection->Modify();
					TArray<bool> ChannelMask = LiveLinkSection->ChannelMask;
					ChannelMask[Index] = !ChannelMask[Index];
					LiveLinkSection->SetMask(ChannelMask);

					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this, Index]
				{
					UMovieSceneLiveLinkSection* LiveLinkSection = CastChecked<UMovieSceneLiveLinkSection>(WeakSection.Get());

					TArray<bool> ChannelMask = LiveLinkSection->ChannelMask;
					if (ChannelMask[Index])
					{
						return ECheckBoxState::Checked;
					}
					else 
					{
						return ECheckBoxState::Unchecked;
					}
				})
		);
	};

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("LiveLinkChannelsText", "Active Live Link Channels"));
	{
		UMovieSceneLiveLinkSection* LiveLinkSection = CastChecked<UMovieSceneLiveLinkSection>(WeakSection.Get());

		TArray<FMovieSceneChannelMetaData> AllMetaData;
		for (const FMovieSceneChannelEntry& Entry : LiveLinkSection->GetChannelProxy().GetAllEntries())
		{
			TArrayView<const FMovieSceneChannelMetaData> EntryMetaData = Entry.GetMetaData();
			AllMetaData.Reserve(AllMetaData.Num() + EntryMetaData.Num());
			for (const FMovieSceneChannelMetaData& MetaData : EntryMetaData)
			{
				AllMetaData.Add(MetaData);
			}
		}

		//Sort metadata like internal channel sorting to get the right index
		auto SortPredicate = [](FMovieSceneChannelMetaData& A, FMovieSceneChannelMetaData& B)
		{
			if (A.SortOrder == B.SortOrder)
			{
				return A.Name.Compare(B.Name) < 0;
			}
			return A.SortOrder < B.SortOrder;
		};

		Algo::Sort(AllMetaData, SortPredicate);

		for (int32 Index = 0; Index < AllMetaData.Num(); ++Index)
		{
			const FText Name = FText::FromName(AllMetaData[Index].Name);
			const FText Text = FText::Format(LOCTEXT("LiveLinkChannelEnable", "{0}"), Name);
			const FText TooltipText = FText::Format(LOCTEXT("LiveLinkChannelEnableTooltip", "Toggle {0}"), Name);
	
			MenuBuilder.AddMenuEntry(
				Text, TooltipText,
				FSlateIcon(), MakeUIAction(Index), NAME_None, EUserInterfaceActionType::ToggleButton);
		}
	}
	MenuBuilder.EndSection();
}
	
bool FLiveLinkSection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePaths)
{
	if (KeyAreaNamePaths.Num() > 0)
	{
		UMovieSceneLiveLinkSection* LiveLinkSection = CastChecked<UMovieSceneLiveLinkSection>(WeakSection.Get());
		TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
		const FScopedTransaction Transaction(LOCTEXT("DeleteLiveLinkChannel", "Delete Live Link channel"));
		if (LiveLinkSection->TryModify())
		{
			//Start by getting all metadata and sort them as the channel list
			TArray<FMovieSceneChannelMetaData> AllMetaData;
			for (const FMovieSceneChannelEntry& Entry : LiveLinkSection->GetChannelProxy().GetAllEntries())
			{
				TArrayView<const FMovieSceneChannelMetaData> EntryMetaData = Entry.GetMetaData();
				AllMetaData.Reserve(AllMetaData.Num() + EntryMetaData.Num());
				for (const FMovieSceneChannelMetaData& MetaData : EntryMetaData)
				{
					AllMetaData.Add(MetaData);
				}
			}

			//Sort metadata like internal channel sorting to get the right index
			auto SortPredicate = [](FMovieSceneChannelMetaData& A, FMovieSceneChannelMetaData& B)
			{
				if (A.SortOrder == B.SortOrder)
				{
					return A.Name.Compare(B.Name) < 0;
				}
				return A.SortOrder < B.SortOrder;
			};

			Algo::Sort(AllMetaData, SortPredicate);

			//Now find the index of each channel selected and update it's mask
			TArray<bool> ChannelMask = LiveLinkSection->ChannelMask;
			for (FName KeyAreaName : KeyAreaNamePaths)
			{
				const int32 ChannelIndex = AllMetaData.IndexOfByPredicate([KeyAreaName](const FMovieSceneChannelMetaData& Other) { return Other.Name == KeyAreaName; });
				if (ChannelIndex != INDEX_NONE && LiveLinkSection->ChannelMask.IsValidIndex(ChannelIndex))
				{
					ChannelMask[ChannelIndex] = false;
				}
			}

			LiveLinkSection->SetMask(ChannelMask);
			SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
			return true;
		}
	}
	return false;
}
#undef LOCTEXT_NAMESPACE


#define LOCTEXT_NAMESPACE "LiveLinkPropertyTrackEditor"


TSharedRef<ISequencerTrackEditor> FLiveLinkPropertyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FLiveLinkPropertyTrackEditor(InSequencer));
}

FText FLiveLinkPropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("LiveLinkPropertyTrackEditor_DisplayName", "Live Link");
}

void FLiveLinkPropertyTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{

}

/* ISequencerTrackEditor interface
*****************************************************************************/

void FLiveLinkPropertyTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddLiveLinkTrack", "Live Link Track"),
		LOCTEXT("AddLiveLinkTrackTooltip", "Adds a new track that exposes Live Link Sources."),
		FSlateIconFinder::FindIconForClass(ULiveLinkComponent::StaticClass()),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FLiveLinkPropertyTrackEditor::HandleAddLiveLinkTrackMenuEntryExecute),
			FCanExecuteAction::CreateRaw(this, &FLiveLinkPropertyTrackEditor::HandleAddLiveLinkTrackMenuEntryCanExecute)
		)
	);
}

TSharedRef<ISequencerSection> FLiveLinkPropertyTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	return MakeShared<FLiveLinkSection>(SectionObject, GetSequencer());
}

bool FLiveLinkPropertyTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneLiveLinkTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FLiveLinkPropertyTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneLiveLinkTrack::StaticClass());
}

const FSlateBrush* FLiveLinkPropertyTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(ULiveLinkComponent::StaticClass()).GetIcon();
}


/* FLiveLinkTrackEditor callbacks
*****************************************************************************/

void FLiveLinkPropertyTrackEditor::HandleAddLiveLinkTrackMenuEntryExecute()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	UMovieSceneTrack* Track = FocusedMovieScene->FindTrack<UMovieSceneLiveLinkTrack>();
	if (!Track)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddLiveLinkTrack_Transaction", "Add Live Link Track"));
		FocusedMovieScene->Modify();
		UMovieSceneLiveLinkTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneLiveLinkTrack>();
		ensure(NewTrack);

		NewTrack->SetDisplayName(LOCTEXT("LiveLinkTrackName", "Live Link"));

		if (GetSequencer().IsValid())
		{
			GetSequencer()->OnAddTrack(NewTrack, FGuid());
		}
	}
}

bool FLiveLinkPropertyTrackEditor::HandleAddLiveLinkTrackMenuEntryCanExecute() const
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	return ((FocusedMovieScene != nullptr) && (FocusedMovieScene->FindTrack<UMovieSceneLiveLinkTrack>() == nullptr));
}




#undef LOCTEXT_NAMESPACE
