// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KeyframeTrackEditor.h"
#include "MovieScene/MovieSceneLiveLinkTrack.h"

//mz for serialization

/** A property track editor for Live Link properties. */
class FLiveLinkPropertyTrackEditor
	: public FKeyframeTrackEditor<UMovieSceneLiveLinkTrack>
{
public:

	FLiveLinkPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FKeyframeTrackEditor<UMovieSceneLiveLinkTrack>(InSequencer)
	{
	}

	/**
	* Creates an instance of this class.  Called by a sequencer
	*
	* @param OwningSequencer The sequencer instance to be used by this tool
	* @return The new instance of this class
	*/
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;

	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;

	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	virtual const FSlateBrush* GetIconBrush() const override;

protected:

	/** Callback for executing the "Add Live Link" menu entry. */
	void HandleAddLiveLinkTrackMenuEntryExecute();
	bool HandleAddLiveLinkTrackMenuEntryCanExecute() const;
};
