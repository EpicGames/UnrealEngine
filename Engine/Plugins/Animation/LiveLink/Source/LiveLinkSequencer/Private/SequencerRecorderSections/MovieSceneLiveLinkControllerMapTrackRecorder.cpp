// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLiveLinkControllerMapTrackRecorder.h"

#include "LiveLinkComponentController.h"
#include "LiveLinkControllerBase.h"
#include "LiveLinkSequencerSettings.h"
#include "MovieSceneLiveLinkControllerTrackRecorder.h"
#include "UObject/UObjectIterator.h"
#include "TrackRecorders/MovieSceneTrackRecorderSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkControllerMapTrackRecorder)

bool FMovieSceneLiveLinkControllerMapTrackRecorderFactory::CanRecordProperty(UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const
{
	// This track recorder can record the ControllerMap property of the LiveLink Component Controller
	if ((InObjectToRecord->GetClass() == ULiveLinkComponentController::StaticClass()) && (InPropertyToRecord->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkComponentController, ControllerMap)))
	{
		return true;
	}
	return false;
}

UMovieSceneTrackRecorder* FMovieSceneLiveLinkControllerMapTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneLiveLinkControllerMapTrackRecorder>();
}

TSubclassOf<UMovieSceneLiveLinkControllerTrackRecorder> UMovieSceneLiveLinkControllerMapTrackRecorder::GetRecorderClassForLiveLinkController(const TSubclassOf<ULiveLinkControllerBase> ControllerClass)
{
	// If the project settings specify a default track recorder class to use for the input controller class, return that one
	const TSubclassOf<UMovieSceneLiveLinkControllerTrackRecorder>* DefaultTrackRecorderClass = GetDefault<ULiveLinkSequencerSettings>()->DefaultTrackRecordersForController.Find(ControllerClass);
	if (DefaultTrackRecorderClass != nullptr && DefaultTrackRecorderClass->Get() != nullptr)
	{
		// Verify that the selected track recorder class truly supports the input controller class
		if ((*DefaultTrackRecorderClass)->GetDefaultObject<UMovieSceneLiveLinkControllerTrackRecorder>()->IsLiveLinkControllerSupported(ControllerClass))
		{
			return *DefaultTrackRecorderClass;
		}
	}

	// If no default track recorder class was specified, return the first track recorder class that supports the input controller class
	for (TObjectIterator<UClass> Itt; Itt; ++Itt)
	{
		if (Itt->IsChildOf(UMovieSceneLiveLinkControllerTrackRecorder::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			if (Itt->GetDefaultObject<UMovieSceneLiveLinkControllerTrackRecorder>()->IsLiveLinkControllerSupported(ControllerClass))
			{
				return TSubclassOf<UMovieSceneLiveLinkControllerTrackRecorder>(*Itt);
			}
		}
	}
	
	// Return nullptr if there are no track recorders that support the input controller class
	return nullptr;
}

void UMovieSceneLiveLinkControllerMapTrackRecorder::CreateTrackImpl()
{
	// Create a track recorder for each LiveLink controller in the controller map
	if (ULiveLinkComponentController* LiveLinkComponent = Cast<ULiveLinkComponentController>(ObjectToRecord.Get()))
	{
		for (TPair<TSubclassOf<ULiveLinkRole>, TObjectPtr<ULiveLinkControllerBase>>& ControllerPair : LiveLinkComponent->ControllerMap)
		{
			if (ULiveLinkControllerBase* Controller = ControllerPair.Value)
			{
				// Create a new track recorder for the current LiveLink controller, call its CreateTrack() method, and add it to the array of controller recorders
				TSubclassOf<UMovieSceneLiveLinkControllerTrackRecorder> TrackRecorderClass = GetRecorderClassForLiveLinkController(Controller->GetClass());

				if (TrackRecorderClass)
				{
					const EObjectFlags RecorderObjectFlags = GetMaskedFlags(RF_Public | RF_Transactional);
					UMovieSceneLiveLinkControllerTrackRecorder* NewTrackRecorder = NewObject<UMovieSceneLiveLinkControllerTrackRecorder>(this, TrackRecorderClass, NAME_None, RecorderObjectFlags);
					NewTrackRecorder->SetLiveLinkController(Controller);
					NewTrackRecorder->CreateTrack(OwningTakeRecorderSource, ObjectToRecord.Get(), MovieScene.Get(), Settings.Get(), ObjectGuid);

					ControllerRecorders.Add(NewTrackRecorder);
				}
			}
		}
	}
}

void UMovieSceneLiveLinkControllerMapTrackRecorder::SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame)
{
	for (UMovieSceneLiveLinkControllerTrackRecorder* ControllerRecorder : ControllerRecorders)
	{
		ControllerRecorder->SetSectionStartTimecode(InSectionStartTimecode, InSectionFirstFrame);
	}
}

void UMovieSceneLiveLinkControllerMapTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	for (UMovieSceneLiveLinkControllerTrackRecorder* ControllerRecorder : ControllerRecorders)
	{
		ControllerRecorder->RecordSample(CurrentTime);
	}
}

void UMovieSceneLiveLinkControllerMapTrackRecorder::StopRecordingImpl()
{
	for (UMovieSceneLiveLinkControllerTrackRecorder* ControllerRecorder : ControllerRecorders)
	{
		ControllerRecorder->StopRecording();
	}
}

void UMovieSceneLiveLinkControllerMapTrackRecorder::FinalizeTrackImpl()
{
	for (UMovieSceneLiveLinkControllerTrackRecorder* ControllerRecorder : ControllerRecorders)
	{
		ControllerRecorder->FinalizeTrack();
	}
}

