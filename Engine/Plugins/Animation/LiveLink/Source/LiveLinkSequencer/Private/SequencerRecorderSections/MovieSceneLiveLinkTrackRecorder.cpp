// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLiveLinkTrackRecorder.h"

#include "Channels/MovieSceneChannelTraits.h"
#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "ILiveLinkClient.h"
#include "LevelSequence.h"
#include "LiveLinkRole.h"
#include "LiveLinkSequencerPrivate.h"
#include "Misc/App.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieScene/MovieSceneLiveLinkTrack.h"
#include "MovieSceneFolder.h"
#include "MovieSceneTakeTrack.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderSourceHelpers.h"
#include "TakeRecorderTimeProcessing.h"

#include "Trace/Trace.inl"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLiveLinkTrackRecorder)

static TAutoConsoleVariable<int32> CVarSequencerAlwaysUseRecordLiveLinkTimecode(
	TEXT("Sequencer.AlwaysRecordLiveLinkTimecode"),
	0, TEXT("This CVAR is no longer used please set the Always Use Timmecode value individually on the Live Link Source."),
	ECVF_Default);


static void AlwaysRecordLiveLinkTimecodeSinkFunction()
{
	static int32 CachedAlwaysRecordLiveLinkTimecode = 0;
	int32 AlwaysRecordLiveLinkTimecode = CVarSequencerAlwaysUseRecordLiveLinkTimecode.GetValueOnGameThread();

	if (AlwaysRecordLiveLinkTimecode != CachedAlwaysRecordLiveLinkTimecode)
	{
		CachedAlwaysRecordLiveLinkTimecode = AlwaysRecordLiveLinkTimecode;
		UE_LOGF(LogLiveLinkSequencer, Warning, "Sequencer.AlwaysRecordLiveLinkTimecode is no longer in use, please set the Always Use Timecode value on the Live Link Source.");
	}
}

static FAutoConsoleVariableSink CVarAlwaysRecoredLiveLinkSink(FConsoleCommandDelegate::CreateStatic(&AlwaysRecordLiveLinkTimecodeSinkFunction));

void UMovieSceneLiveLinkTrackRecorder::CreateTrack(UMovieScene* InMovieScene, const FName& InSubjectName, bool bInSaveSubjectSettings, bool bInAlwaysUseTimecode, bool bInDiscardSamplesBeforeStart, UMovieSceneTrackRecorderSettings* InSettingsObject)
{
	MovieScene = InMovieScene;
	SubjectName = InSubjectName;
	bSaveSubjectSettings = bInSaveSubjectSettings;
	bUseSourceTimecode = bInAlwaysUseTimecode;
	bDiscardSamplesBeforeStart = bInDiscardSamplesBeforeStart;
	bRecordTimecode = GetDefault<UTakeRecorderProjectSettings>()->Settings.bRecordTimecode;

	CreateTracks();
}

UMovieSceneLiveLinkTrack* UMovieSceneLiveLinkTrackRecorder::DoesLiveLinkTrackExist(const FName& TrackName, const TSubclassOf<ULiveLinkRole>& InTrackRole)
{
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (Track->IsA(UMovieSceneLiveLinkTrack::StaticClass()))
		{
			UMovieSceneLiveLinkTrack* TestLiveLinkTrack = CastChecked<UMovieSceneLiveLinkTrack>(Track);
			if (TestLiveLinkTrack && TestLiveLinkTrack->GetPropertyName() == TrackName && TestLiveLinkTrack->GetTrackRole() == InTrackRole)
			{
				return TestLiveLinkTrack;
			}
		}
	}
	return nullptr;
}


void UMovieSceneLiveLinkTrackRecorder::CreateTracks()
{
	LiveLinkTrack = nullptr;
	MovieSceneSection.Reset();

	FramesToProcess.Empty();
	RecordedTimes.Empty();

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	FText Error;
	
	if (LiveLinkClient == nullptr)
	{
		UE_LOGF(LogLiveLinkSequencer, Warning, "Error: Could not create live link track. LiveLink module is not available.");
		return;
	}

	if (SubjectName == NAME_None)
	{
		UE_LOGF(LogLiveLinkSequencer, Warning, "Error: Could not create live link track. Desired subject name is empty.");
		return;
	}

	//Find the subject key associated with the desired subject name. Only one subject with the same name can be enabled.
	const bool bIncludeDisabledSubjects = false;
	const bool bIncludeVirtualSubjects = true;
	TArray<FLiveLinkSubjectKey> EnabledSubjects = LiveLinkClient->GetSubjects(bIncludeDisabledSubjects, bIncludeVirtualSubjects);
	const FLiveLinkSubjectKey* DesiredSubjectKey = EnabledSubjects.FindByPredicate([this](const FLiveLinkSubjectKey& InOther) { return SubjectName == InOther.SubjectName; });
	if (DesiredSubjectKey == nullptr)
	{
		UE_LOGF(LogLiveLinkSequencer, Warning, "Error: Could not create live link track. Could not find an enabled subject with subject name '%ls'.", *SubjectName.ToString());
		return;
	}

	// Keep track if we're recording a virtual subject to handle recorded frames differently
	bIsVirtualSubject = LiveLinkClient->IsVirtualSubject(*DesiredSubjectKey);

	TSharedPtr<FLiveLinkStaticDataStruct> StaticData = MakeShared<FLiveLinkStaticDataStruct>();
	const bool bRegistered = LiveLinkClient->RegisterForSubjectFrames(SubjectName
																	, FOnLiveLinkSubjectStaticDataAdded::FDelegate::CreateUObject(this, &UMovieSceneLiveLinkTrackRecorder::OnStaticDataReceived)
																	, FOnLiveLinkSubjectFrameDataAdded::FDelegate::CreateUObject(this, &UMovieSceneLiveLinkTrackRecorder::OnFrameDataReceived)
																	, OnStaticDataReceivedHandle
																	, OnFrameDataReceivedHandle
																	, SubjectRole
																	, StaticData.Get());

	if(bRegistered && StaticData->IsValid())
	{
		LiveLinkTrack = DoesLiveLinkTrackExist(SubjectName, SubjectRole);
		if (!LiveLinkTrack.IsValid())
		{
			LiveLinkTrack = MovieScene->AddTrack<UMovieSceneLiveLinkTrack>();
			LiveLinkTrack->SetTrackRole(SubjectRole);
		}
		else
		{
			LiveLinkTrack->RemoveAllAnimationData();
		}

		LiveLinkTrack->SetPropertyNameAndPath(SubjectName, SubjectName.ToString());

		MovieSceneSection = Cast<UMovieSceneLiveLinkSection>(LiveLinkTrack->CreateNewSection());
		if (MovieSceneSection != nullptr)
		{
			MovieSceneSection->SetIsActive(false);
			LiveLinkTrack->AddSection(*MovieSceneSection);

			FLiveLinkSubjectPreset SubjectPreset;
			if (bSaveSubjectSettings)
			{
				SubjectPreset = LiveLinkClient->GetSubjectPreset(*DesiredSubjectKey, MovieSceneSection.Get());

				//Nulling out VirtualSubject will make it look like a 'live' subject when playing back.
				//Subject settings will be lost though. That's a drawback of recording virtual subject for now.
				SubjectPreset.VirtualSubject = nullptr;
			}
			else
			{
				//When we don't save defaults, fill in a preset to match the subject. SourceGuid is left out voluntarily. It will be filled when the sequencer is playing back the track.
				SubjectPreset.Key.Source.Invalidate();
				SubjectPreset.Key.SubjectName = SubjectName;
				SubjectPreset.Role = SubjectRole;
				SubjectPreset.bEnabled = true;
			}

			//Initialize the LiveLink Section. This will spawn required sub sections to manage data for this role
			MovieSceneSection->Initialize(SubjectPreset, StaticData);

			MovieSceneSection->CreateChannelProxy();
		}
		else
		{
			UE_LOGF(LogLiveLinkSequencer, Warning, "Error Creating LiveLink MovieScene Section for subject '%ls' with role '%ls", *SubjectName.ToString(), *SubjectRole->GetFName().ToString());
		}
	}
	else
	{
		UE_LOGF(LogLiveLinkSequencer, Warning, "Error: Could not register to SubjectName '%ls' from LiveLink client.", *(SubjectName.ToString()));
	}
}

void UMovieSceneLiveLinkTrackRecorder::SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) 
{
	float Time = 0.0f;
	SecondsDiff = FPlatformTime::Seconds() - Time;

	if (MovieSceneSection.IsValid())
	{
		MovieSceneSection->TimecodeSource = FMovieSceneTimecodeSource(InSectionStartTimecode);

		FTakeRecorderParameters Parameters;
		Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
		Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		RecordStartFrame = Parameters.Project.bStartAtCurrentTimecode ? FFrameRate::TransformTime(FFrameTime(InSectionStartTimecode.ToFrameNumber(DisplayRate)), DisplayRate, TickResolution).FloorToFrame() : MovieScene->GetPlaybackRange().GetLowerBoundValue();
	}
}

void UMovieSceneLiveLinkTrackRecorder::StopRecordingImpl()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	if (LiveLinkClient && MovieSceneSection.IsValid())
	{
		LiveLinkClient->UnregisterSubjectFramesHandle(SubjectName, OnStaticDataReceivedHandle, OnFrameDataReceivedHandle);
	}
}

void UMovieSceneLiveLinkTrackRecorder::FinalizeTrackImpl()
{
	if (MovieSceneSection.IsValid())
	{
		FTakeRecorderParameters Parameters;
		Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;

		FKeyDataOptimizationParams Params;
		Params.bAutoSetInterpolation = true;
		Params.Tolerance = Parameters.User.ReduceKeysTolerance;
		MovieSceneSection->FinalizeSection(bReduceKeys, Params);

		TOptional<TRange<FFrameNumber> > DefaultSectionLength = MovieSceneSection->GetAutoSizeRange();
		if (DefaultSectionLength.IsSet())
		{
			MovieSceneSection->SetRange(DefaultSectionLength.GetValue());
		}

		MovieSceneSection->SetIsActive(true);
	}
}

void UMovieSceneLiveLinkTrackRecorder::ProcessRecordedTimes(ULevelSequence* InLevelSequence)
{
	if (!MovieSceneSection.IsValid() || !bRecordTimecode)
	{
		return;
	}
	check(InLevelSequence);

	TOptional<TRange<FFrameNumber>> FrameRange = MovieSceneSection->GetRange();
	UMovieSceneTakeTrack* TakeTrack = Cast<UMovieSceneTakeTrack>(MovieScene->FindTrack(UMovieSceneTakeTrack::StaticClass()));
	if (!TakeTrack)
	{
		TakeTrack = Cast<UMovieSceneTakeTrack>(MovieScene->AddTrack(UMovieSceneTakeTrack::StaticClass()));
	}
	UE::TakesCore::ProcessRecordedTimes(InLevelSequence, TakeTrack, FrameRange, RecordedTimes);
}

void UMovieSceneLiveLinkTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	FTakeRecorderParameters Parameters;
	Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
	Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ILiveLinkClient* LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	if (LiveLinkClient && MovieSceneSection.IsValid())
	{
		//we know all section have same tick resolution
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		const FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();

		bool bSyncedOrForced = bUseSourceTimecode || LiveLinkClient->IsSubjectTimeSynchronized(SubjectName);

		// If this is a virtual subject then we'll just evaluate it directly and add it to the FramesToProcess array
		if (bIsVirtualSubject)
		{
			FLiveLinkSubjectFrameData EvaluatedFrame;
			LiveLinkClient->EvaluateFrame_AnyThread(SubjectName, SubjectRole, EvaluatedFrame);
			FramesToProcess.Emplace(MoveTemp(EvaluatedFrame.FrameData));
		}

		if (FramesToProcess.Num() > 0)
		{
			const TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();
			for (const FLiveLinkFrameDataStruct& Frame : FramesToProcess)
			{
				FFrameNumber FrameNumber;
				FQualifiedFrameTime LiveLinkFrameTime;
				if (bSyncedOrForced && CurrentFrameTime.IsSet())
				{
					LiveLinkFrameTime = bUseSourceTimecode ? Frame.GetBaseData()->MetaData.SceneTime : *CurrentFrameTime;

					if (!Parameters.Project.bStartAtCurrentTimecode)
					{
						//Get StartTime on Section in TimeCode FrameRate
						//Convert that to LiveLink FrameRate and subtract out from LiveLink Frame to get section starting from zero.
						//Finally convert that to the actual MovieScene Section FrameRate(TickResolution).
						const FQualifiedFrameTime TimeProviderStartFrameTime = FQualifiedFrameTime(MovieSceneSection->TimecodeSource.Timecode, CurrentFrameTime.GetValue().Rate);
						const FFrameNumber FrameNumberStart = TimeProviderStartFrameTime.ConvertTo(LiveLinkFrameTime.Rate).FrameNumber;
						LiveLinkFrameTime.Time.FrameNumber -= FrameNumberStart;
					}

					FFrameTime FrameTime = LiveLinkFrameTime.ConvertTo(TickResolution);
					FrameNumber = FrameTime.FrameNumber;

					UE_LOGF(LogLiveLinkSequencer, VeryVerbose, "LiveLinkFrameTime: [%ls] at %ls for subject '%ls'.",
						   *(FTimecode::FromFrameTime(LiveLinkFrameTime.Time, LiveLinkFrameTime.Rate).ToString()),
						   *(LiveLinkFrameTime.Rate.ToPrettyText().ToString()),
						   *(SubjectName.ToString()));
				}
				else
				{
					const double Second = Frame.GetBaseData()->WorldTime.GetOffsettedTime() - SecondsDiff;				
					FrameNumber = (Second * TickResolution).FloorToFrame();
					FrameNumber += RecordStartFrame;

					LiveLinkFrameTime = CurrentFrameTime ? *CurrentFrameTime : FQualifiedFrameTime(FrameNumber, CurrentFrameTime.IsSet() ? CurrentFrameTime->Rate : DisplayRate);
					UE_LOGF(LogLiveLinkSequencer, VeryVerbose, "LiveLinkFrameTime (Unsynced): %f, for subject '%ls'.", Frame.GetBaseData()->WorldTime.GetOffsettedTime(), *(SubjectName.ToString()));
				}

				// For clarity, only record values that are after the start frame since frames could have been buffered before recording started.
				if (FrameNumber >= MovieSceneSection->GetInclusiveStartFrame() || !bDiscardSamplesBeforeStart)
				{
					MovieSceneSection->RecordFrame(FrameNumber, Frame);
					RecordedTimes.Add(TPair<FQualifiedFrameTime, FQualifiedFrameTime>(
						FQualifiedFrameTime(FrameNumber, TickResolution),
						LiveLinkFrameTime));
				}
				else
				{
					UE_LOGF(LogLiveLinkSequencer, Warning, "Discarded buffered frame: [%ls] outside of start frame: [%ls] for subject '%ls'.",
						   *(FTimecode::FromFrameTime(ConvertFrameTime(FrameNumber, TickResolution, DisplayRate), DisplayRate).ToString()),
						   *(FTimecode::FromFrameTime(ConvertFrameTime(MovieSceneSection->GetInclusiveStartFrame(), TickResolution, DisplayRate), DisplayRate)).ToString(),
						   *(SubjectName.ToString()));
				}
			}

			//Empty out frames that were processed
			FramesToProcess.Reset();
		} 
	}
}

void UMovieSceneLiveLinkTrackRecorder::AddContentsToFolder(UMovieSceneFolder* InFolder)
{
	if (LiveLinkTrack.IsValid())
	{
		InFolder->AddChildTrack(LiveLinkTrack.Get());
	}
}

void UMovieSceneLiveLinkTrackRecorder::OnStaticDataReceived(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InSubjectRole, const FLiveLinkStaticDataStruct& InStaticData)
{
	UE_LOGF(LogLiveLinkSequencer, Warning, "Static data changed for subject '%ls' while recording. This is not supported and could cause problems with associated frame data", *(SubjectName.ToString()));
}

void UMovieSceneLiveLinkTrackRecorder::OnFrameDataReceived(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InSubjectRole, const FLiveLinkFrameDataStruct& InFrameData)
{
	if (InSubjectKey.SubjectName.Name != SubjectName)
	{
		UE_LOGF(LogLiveLinkSequencer, Warning, "Received frame for Subject '%ls' but was expecting subject '%ls'", *(InSubjectKey.SubjectName.Name.ToString()), *(SubjectName.ToString()));
		return;
	}

	if (InSubjectRole != SubjectRole)
	{
		UE_LOGF(LogLiveLinkSequencer, Warning, "Received frame for Subject '%ls' for role '%ls' but was expecting role '%ls'"
			, *InSubjectKey.SubjectName.ToString()
			, *InSubjectRole.GetDefaultObject()->GetDisplayName().ToString()
			, *SubjectRole.GetDefaultObject()->GetDisplayName().ToString());

		return;
	}

	//We need to make our own copy of the incoming frame to process it when record is called
	FLiveLinkFrameDataStruct CopiedFrame;
	CopiedFrame.InitializeWith(InFrameData);

	FramesToProcess.Emplace(MoveTemp(CopiedFrame));
}

bool UMovieSceneLiveLinkTrackRecorder::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) 
{
	UE_LOGF(LogLiveLinkSequencer, Warning, "Loading recorded file for live link tracks is not supported.");
	return false;
}

