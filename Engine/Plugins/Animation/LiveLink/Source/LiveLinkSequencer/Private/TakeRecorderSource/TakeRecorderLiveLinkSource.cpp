// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeRecorderLiveLinkSource.h"
#include "MovieSceneLiveLinkTrackRecorder.h"
#include "LevelSequence.h"
#include "TakeMetaData.h"
#include "TakeRecorderSources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderLiveLinkSource)

UTakeRecorderLiveLinkSource::UTakeRecorderLiveLinkSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, bReduceKeys(false)
	, SubjectName(NAME_None)
	, bSaveSubjectSettings(true)
	, bUseSourceTimecode(false)
	, bDiscardSamplesBeforeStart(true)
{
	TrackTint = FColor(74, 108, 164);
}

TArray<UTakeRecorderSource*> UTakeRecorderLiveLinkSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) 
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();
	TrackRecorder = NewObject<UMovieSceneLiveLinkTrackRecorder>();
	TrackRecorder->CreateTrack(MovieScene, SubjectName, bSaveSubjectSettings, bUseSourceTimecode, bDiscardSamplesBeforeStart, nullptr);

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderLiveLinkSource::StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence)
{
	if (TrackRecorder)
	{
		TrackRecorder->SetReduceKeys(bReduceKeys);
		TrackRecorder->SetSectionStartTimecode(InSectionStartTimecode, InSectionFirstFrame);
	}
}

void UTakeRecorderLiveLinkSource::TickRecording(const FQualifiedFrameTime& CurrentSequenceTime)
{
	if(TrackRecorder)
	{
		TrackRecorder->RecordSample(CurrentSequenceTime);
	}
}

void UTakeRecorderLiveLinkSource::StopRecording(class ULevelSequence* InSequence)
{
	if (TrackRecorder)
	{
		TrackRecorder->StopRecording();
	}
}

TArray<UTakeRecorderSource*> UTakeRecorderLiveLinkSource::PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled)
{
	if (TrackRecorder)
	{
		TrackRecorder->FinalizeTrack();
		if (!bCancelled)
		{
			TrackRecorder->ProcessRecordedTimes(InSequence);
		}
	}
	
	TrackRecorder = nullptr;
	return TArray<UTakeRecorderSource*>();
}

FText UTakeRecorderLiveLinkSource::GetDisplayTextImpl() const
{
	return FText::FromName(SubjectName);
}

FText UTakeRecorderLiveLinkSource::GetAddSourceDisplayTextImpl() const
{
    return NSLOCTEXT("UTakeRecorderLiveLinkSource", "TakeRecorderDisplayName", "Live Link");
}

void UTakeRecorderLiveLinkSource::AddContentsToFolder(UMovieSceneFolder* InFolder)
{
	TrackRecorder->AddContentsToFolder(InFolder);
}

bool UTakeRecorderLiveLinkSource::CanAddSource(UTakeRecorderSources* InSources) const
{
	for (UTakeRecorderSource* Source : InSources->GetSources())
	{
		if (Source->IsA<UTakeRecorderLiveLinkSource>() )
		{
			if (UTakeRecorderLiveLinkSource* OtherSource = Cast<UTakeRecorderLiveLinkSource>(Source) )
			{
				if (OtherSource->SubjectName == SubjectName)
				{
					return false;
				}
			}
		}
	}
	return true;
}

FString UTakeRecorderLiveLinkSource::GetSubsceneTrackName(ULevelSequence* InSequence) const
{
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		return FString::Printf(TEXT("%s_%s"), *SubjectName.ToString(), *TakeMetaData->GenerateAssetPath("{slate}"));
	}
	else if (SubjectName != NAME_None)
	{
		return SubjectName.ToString();
	}

	return TEXT("LiveLink");
}

FString UTakeRecorderLiveLinkSource::GetSubsceneAssetName(ULevelSequence* InSequence) const
{
	if (UTakeMetaData* TakeMetaData = InSequence->FindMetaData<UTakeMetaData>())
	{
		return FString::Printf(TEXT("%s_%s"), *SubjectName.ToString(), *TakeMetaData->GenerateAssetPath("{slate}_{take}"));
	}
	else if (SubjectName != NAME_None)
	{
		return SubjectName.ToString();
	}

	return TEXT("LiveLink");
}

