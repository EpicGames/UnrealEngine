// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"
#include "TakeRecorderLiveLinkSource.generated.h"

class UMovieSceneLiveLinkTrackRecorder;

USTRUCT(BlueprintType)
struct FLiveLinkSubjectProperty
{
	GENERATED_BODY()

	FLiveLinkSubjectProperty()
		: SubjectName(NAME_None)
		, bEnabled(true)
	{
	}

	FLiveLinkSubjectProperty(const FName& InName, const bool bInEnabled)
	{
		SubjectName = InName;
		bEnabled = bInEnabled;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subjects")
	FName SubjectName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subjects")
	bool bEnabled;
};


UCLASS(BlueprintType)
class ULiveLinkSubjectProperties : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subjects")
	TArray<FLiveLinkSubjectProperty> Properties;
};

/** A recording source that records LiveLink */
UCLASS(Category="Live Link", config=EditorSettings)
class UTakeRecorderLiveLinkSource : public UTakeRecorderSource
{

public:
	GENERATED_BODY()

	UTakeRecorderLiveLinkSource(const FObjectInitializer& ObjInit);

	/** Whether to perform key-reduction algorithms as part of the recording */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Source")
	bool bReduceKeys;

	/** Name of the subject to record */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subjects")
	FName SubjectName;

	/** Whether we should save subject settings in the the live link section. If not, we'll create one with subject information with no settings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subjects")
	bool bSaveSubjectSettings;

	/**
	 * Whether the livelink subject's timecode or the system time should be used for the recording.
	 * @note If set, the livelink subject's timecode will be used even if it does not match the engine timecode.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subjects")
	bool bUseSourceTimecode;

	/** If true discard livelink samples with timecode that occurs before the start of recording*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subjects", meta=(EditCondition = bUseSourceTimecode))
	bool bDiscardSamplesBeforeStart;

	/**
	* The track recorder we created.
	*/
	UPROPERTY()
	TObjectPtr<UMovieSceneLiveLinkTrackRecorder> TrackRecorder;

private:

	// UTakeRecorderSource
	virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	virtual void StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence) override;
	virtual void TickRecording(const FQualifiedFrameTime& CurrentSequenceTime) override;
	virtual void StopRecording(class ULevelSequence* InSequence) override;
	virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, class ULevelSequence* InRootSequence, const bool bCancelled) override;

	virtual FText GetDisplayTextImpl() const override;
	virtual void AddContentsToFolder(class UMovieSceneFolder* InFolder) override;
	virtual bool CanAddSource(UTakeRecorderSources* InSources) const override;
	virtual bool SupportsSubscenes() const override { return true; }
	virtual FString GetSubsceneTrackName(ULevelSequence* InSequence) const override;
	virtual FString GetSubsceneAssetName(ULevelSequence* InSequence) const override;
	virtual FText GetAddSourceDisplayTextImpl() const override;
	// ~UTakeRecorderSource

};
