// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "Templates/SubclassOf.h"

#include "LiveLinkSequencerSettings.generated.h"

#define UE_API LIVELINKSEQUENCER_API

class ULiveLinkControllerBase;
class UMovieSceneLiveLinkControllerTrackRecorder;


/**
 * Settings for LiveLink Sequence Editor
 */
UCLASS(MinimalAPI, config=Game)
class ULiveLinkSequencerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	//~ Begin UDeveloperSettings interface
	UE_API virtual FName GetCategoryName() const;
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FName GetSectionName() const override;
#endif
	//~ End UDeveloperSettings interface

public:

	/** Default Track Recorder class to use for the specified LiveLink controller */
	UPROPERTY(config, EditAnywhere, Category = "LiveLink", meta = (AllowAbstract = "false"))
	TMap<TSubclassOf<ULiveLinkControllerBase>, TSubclassOf<UMovieSceneLiveLinkControllerTrackRecorder>> DefaultTrackRecordersForController;
};

#undef UE_API
