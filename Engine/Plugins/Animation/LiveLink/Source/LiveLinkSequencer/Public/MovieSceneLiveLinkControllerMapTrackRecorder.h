// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackRecorders/IMovieSceneTrackRecorderFactory.h"
#include "TrackRecorders/MovieSceneTrackRecorder.h"


#include "MovieSceneLiveLinkControllerMapTrackRecorder.generated.h"

#define UE_API LIVELINKSEQUENCER_API

class ULiveLinkControllerBase;
class UMovieSceneLiveLinkControllerTrackRecorder;

/** Movie Scene track recorder factory for the LiveLink Component's Controller Map */
class FMovieSceneLiveLinkControllerMapTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneLiveLinkControllerMapTrackRecorderFactory() = default;

	//~ Begin IMovieSceneTrackRecorderFactory interface
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override { return false; }
	UE_API virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override;

	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }

	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneLiveLinkControllerTrackRecorderFactory", "DisplayName", "LiveLink ControllerMap Track Recorder"); }
	//~ End IMovieSceneTrackRecorderFactory interface
};

/** Movie Scene track recorder for LiveLink Component's Controller Map */
UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneLiveLinkControllerMapTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
public:
	virtual ~UMovieSceneLiveLinkControllerMapTrackRecorder() = default;

	//~ Begin UMovieSceneTrackRecorder Interface
	UE_API virtual void CreateTrackImpl() override;
	UE_API virtual void SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) override;
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void StopRecordingImpl() override;
	UE_API virtual void FinalizeTrackImpl() override;
	//~ End UMovieSceneTrackRecorder Interface

private:
	/** Return the desired track recorder class that supports the input LiveLink controller class */
	UE_API TSubclassOf<UMovieSceneLiveLinkControllerTrackRecorder> GetRecorderClassForLiveLinkController(const TSubclassOf<ULiveLinkControllerBase> ControllerClass);

private:
	/** Array of track recorders that will record each of the LiveLink controller's in the Controller Map */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMovieSceneLiveLinkControllerTrackRecorder>> ControllerRecorders;
};

#undef UE_API
