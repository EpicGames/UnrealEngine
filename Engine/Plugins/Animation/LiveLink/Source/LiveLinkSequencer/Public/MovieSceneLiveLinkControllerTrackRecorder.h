// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackRecorders/MovieSceneTrackRecorder.h"

#include "MovieSceneLiveLinkControllerTrackRecorder.generated.h"

class ULiveLinkControllerBase;

/** Abstract based for movie scene track recorders that can record LiveLink Controllers */
UCLASS(MinimalAPI, Abstract)
class UMovieSceneLiveLinkControllerTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()

public:
	/** Returns true if this track recorder class supports the input LiveLink controller class, false otherwise */
	virtual bool IsLiveLinkControllerSupported(const TSubclassOf<ULiveLinkControllerBase>& ControllerToSupport) const PURE_VIRTUAL(UMovieSceneLiveLinkControllerTrackRecorder::IsLiveLinkControllerSupported, return false;);

	/** Set the LiveLink controller object that this track recorder will record properties from */
	void SetLiveLinkController(ULiveLinkControllerBase* InLiveLinkController) { LiveLinkControllerToRecord = InLiveLinkController; }

protected:
	/** The LiveLink controller that this track record will record properties from */
	UPROPERTY(Transient)
	TObjectPtr<ULiveLinkControllerBase> LiveLinkControllerToRecord;
};
