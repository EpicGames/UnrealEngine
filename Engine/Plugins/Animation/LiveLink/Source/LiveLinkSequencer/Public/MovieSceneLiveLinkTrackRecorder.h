// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackRecorders/MovieSceneTrackRecorder.h"

#include "MovieScene/MovieSceneLiveLinkSection.h"

#include "MovieSceneLiveLinkTrackRecorder.generated.h"

#define UE_API LIVELINKSEQUENCER_API

class UMovieSceneTrackRecorderSettings;


class UMovieSceneLiveLinkTrack;
class UMovieSceneLiveLinkSectionBase;
class UMotionControllerComponent;
class ULiveLinkComponent;
class ULiveLinkSubjectProperties;
class UMovieSceneLiveLinkSection;
class UMovieSceneLiveLinkTrack;
class ULevelSequence;


UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneLiveLinkTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
public:
	virtual ~UMovieSceneLiveLinkTrackRecorder() = default;

	// UMovieSceneTrackRecorder Interface
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	UE_API virtual void FinalizeTrackImpl() override;
	UE_API virtual void SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return Cast<UMovieSceneSection>(MovieSceneSection.Get()); }
	UE_API virtual void StopRecordingImpl() override;
	virtual void SetSavedRecordingDirectory(const FString& InDirectory)
	{
		Directory = InDirectory;
	}
	UE_API virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) override;


public:
	//we don't call UMovieSceneTrackRecorder::CreateTrack or CreateTrackImpl since that expects an  ObjectToRecord and a GUID which isn't needed.
	UE_API void CreateTrack(UMovieScene* InMovieScene, const FName& InSubjectName, bool bInSaveSubjectSettings, bool bInAlwaysUseTimecode, bool bDiscardSamplesBeforeStart, UMovieSceneTrackRecorderSettings* InSettingsObject);
	UE_API void AddContentsToFolder(UMovieSceneFolder* InFolder);
	void SetReduceKeys(bool bInReduce) { bReduceKeys = bInReduce; }

	/** Tell the recorder to write the take track using the recorded times. */
	UE_API void ProcessRecordedTimes(ULevelSequence *InLevelSequence);
private:

	UE_API UMovieSceneLiveLinkTrack* DoesLiveLinkTrackExist(const FName& TrackName, const TSubclassOf<ULiveLinkRole>& InTrackRole);

	UE_API void CreateTracks();

	UE_API void OnStaticDataReceived(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InSubjectRole, const FLiveLinkStaticDataStruct& InStaticData);
	UE_API void OnFrameDataReceived(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InSubjectRole, const FLiveLinkFrameDataStruct& InFrameData);
private:

	/** Name of Subject To Record */
	FName SubjectName;

	/** Whether we should save subject preset in the the live link section. If not, we'll create one with subject information with no settings */
	bool bSaveSubjectSettings;

	/** Whether or not we use timecode time or world time*/
	bool bUseSourceTimecode;

	/** Whether to discard livelink samples with timecode that occurs before the start of recording*/
	bool bDiscardSamplesBeforeStart;

	/** Role of the subject we will record*/
	TSubclassOf<ULiveLinkRole> SubjectRole;

	/** Cached LiveLink Tracks, section per each maps to SubjectNames */
	TWeakObjectPtr<UMovieSceneLiveLinkTrack> LiveLinkTrack;

	/** Sections to record to on each track*/
	TWeakObjectPtr<UMovieSceneLiveLinkSection> MovieSceneSection;
	
	/** Diff between Engine Time from when starting to record and Platform
	Time which is used by Live Link. Still used if no TimeCode present.*/
	double SecondsDiff; 

	/** The frame at the start of this recording section */
	FFrameNumber RecordStartFrame;

	/** Guid when registered to get LiveLinkData */
	FGuid HandlerGuid;

	/**Cached directory for serializers to save to*/
	FString Directory;

	/** Cached Key Reduction from Live Link Source Properties*/
	bool bReduceKeys;

	/** Whether the Subject is Virtual or not*/
	bool bIsVirtualSubject = false;

	/** Should we record timecode data. */
	bool bRecordTimecode = false;

	/** Delegates registered during recording to receive live link data as it comes in*/
	FDelegateHandle OnStaticDataReceivedHandle;
	FDelegateHandle OnFrameDataReceivedHandle;

	TArray<FLiveLinkFrameDataStruct> FramesToProcess;

	/** Pairs of recorded times with the corresponding timecode.*/
	TArray<TPair<FQualifiedFrameTime, FQualifiedFrameTime>> RecordedTimes;
};

#undef UE_API
